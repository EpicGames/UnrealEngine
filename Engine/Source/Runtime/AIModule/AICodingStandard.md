## AI Team’s coding standard extension 

_In all cases [Epic’s coding standard](https://docs.unrealengine.com/en-US/ProductionPipelines/DevelopmentSetup/CodingStandard/index.html) has higher priority. AI Team’s coding standard needs to remain compatible and act as an extension and clarification where Epic’s is unclear or leaves room for interpretation._

#### Code organization and naming

##### cpp files
* If a source file has implementation of multiple structs/classes, use the following header to mark the beginning of each class/struct to make it super clear when new implementation begins.

```
//----------------------------------------------------------------//
//  FMyTypeName
//----------------------------------------------------------------//
```
* Keep member functions of a given type together, avoid interlacing functions of multiple types.

##### Subsystems 
`WorldSubsystem`-derived classes need to have a `Subsystem` postfix, while other subsystem-y classes that are not related with any of the engine's subsystem classes get a `Manager` postfix.

#### Namespaces 

[Unreal Engine standard here](https://docs.unrealengine.com/en-US/ProductionPipelines/DevelopmentSetup/CodingStandard/index.html#namespaces).

**Nested namespaces** declaration should be done in a single line, like so:

```c++
namespace UE::Mass::Debug
{ 
//...	
} // namespace UE::Mass::Debug
```


#### Debug-time code

Debugging-related member function names should start with Debug prefix.

___

### FAQ
*add things that can be best explained as a simple answer to a simple question. These things can be reflected above as well. This section needs to remain last at all times.* 

**Q**: What do I do when the thing?
**A**: You do the thing.