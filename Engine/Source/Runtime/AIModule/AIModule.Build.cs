// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
    public class AIModule : ModuleRules
    {
        public AIModule(ReadOnlyTargetRules Target) : base(Target)
        {
            PublicDependencyModuleNames.AddRange(
                new string[] {
                    "Core",
                    "CoreUObject",
                    "Engine",
                    "GameplayTags",
                    "GameplayTasks",
                    "NavigationSystem",
                }
                );

            PrivateDependencyModuleNames.AddRange(
                new string[] {
                    "AutoRTFM",
					"InputCore",
					"RHI",
                    "RenderCore",
                }
                );

            if (Target.bBuildEditor == true)
            {
				PrivateDependencyModuleNames.Add("EditorFramework");
                PrivateDependencyModuleNames.Add("UnrealEd");
                PrivateDependencyModuleNames.Add("SlateCore");

                PrivateDependencyModuleNames.Add("AITestSuite");
                CircularlyReferencedDependentModules.Add("AITestSuite");
            }

            if (Target.bCompileRecast)
            {
                PrivateDependencyModuleNames.Add("Navmesh");
                PublicDefinitions.Add("WITH_RECAST=1");
            }
            else
            {
                // Because we test WITH_RECAST in public Engine header files, we need to make sure that modules
                // that import us also have this definition set appropriately.  Recast is a private dependency
                // module, so it's definitions won't propagate to modules that import Engine.
                PublicDefinitions.Add("WITH_RECAST=0");
            }

            SetupGameplayDebuggerSupport(Target);
        }
    }
}
