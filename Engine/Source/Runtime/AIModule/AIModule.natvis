<?xml version="1.0" encoding="utf-8"?> 
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

  <!--
  *
  * BehaviorTree Visualizers
  *
  -->
  <Type Name="FBehaviorTreeSearchUpdate">
    <DisplayString Condition="Mode.Value == 1">Add - {AuxNode}</DisplayString>
    <DisplayString Condition="Mode.Value == 2">Remove - {AuxNode}</DisplayString>
  </Type>

  <Type Name="UBTNode">
    <DisplayString>({TreeAsset->NamePrivate}::{NodeName} [{ExecutionIndex}])</DisplayString>
  </Type>

  <!--
  *
  * Perception Visualizers
  *
  -->
  <Type Name="FAISightQuery">
    <DisplayString>({ObserverId} - {TargetId}: last={bLastResult} imp={Importance})</DisplayString>
  </Type>

</AutoVisualizer>