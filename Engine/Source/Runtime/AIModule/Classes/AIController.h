// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Templates/SubclassOf.h"
#include "EngineDefines.h"
#include "AITypes.h"
#include "GameplayTaskOwnerInterface.h"
#include "GameplayTask.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/Controller.h"
#include "Perception/AIPerceptionListenerInterface.h"
#include "GenericTeamAgentInterface.h"
#include "VisualLogger/VisualLoggerDebugSnapshotInterface.h"
#include "AIController.generated.h"

class FDebugDisplayInfo;
class UAIPerceptionComponent;
class UBehaviorTree;
class UBlackboardComponent;
class UBlackboardData;
class UBrainComponent;
class UCanvas;
class UGameplayTaskResource;
class UGameplayTasksComponent;
class UPathFollowingComponent;

namespace EPathFollowingRequestResult {	enum Type : int; }
namespace EPathFollowingResult { enum Type : int; }
namespace EPathFollowingStatus { enum Type : int; }

#if ENABLE_VISUAL_LOG
struct FVisualLogEntry;
#endif // ENABLE_VISUAL_LOG
struct FPathFindingQuery;
struct FPathFollowingRequestResult;
struct FPathFollowingResult;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FAIMoveCompletedSignature, FAIRequestID, RequestID, EPathFollowingResult::Type, Result);

// the reason for this being namespace instead of a regular enum is
// so that it can be expanded in game-specific code
// @todo this is a bit messy, needs to be refactored
namespace EAIFocusPriority
{
	typedef uint8 Type;

	inline const Type Default = 0;
	inline const Type Move = 1;
	inline const Type Gameplay = 2;

	inline const Type LastFocusPriority = Gameplay;
}

struct FFocusKnowledge
{
	struct FFocusItem
	{
		TWeakObjectPtr<AActor> Actor;
		FVector Position;

		FFocusItem()
		{
			Actor = nullptr;
			Position = FAISystem::InvalidLocation;
		}
	};
	
	TArray<FFocusItem> Priorities;
};

//~=============================================================================
/**
 * AIController is the base class of controllers for AI-controlled Pawns.
 * 
 * Controllers are non-physical actors that can be attached to a pawn to control its actions.
 * AIControllers manage the artificial intelligence for the pawns they control.
 * In networked games, they only exist on the server.
 *
 * @see https://docs.unrealengine.com/latest/INT/Gameplay/Framework/Controller/
 */

UCLASS(ClassGroup = AI, BlueprintType, Blueprintable, MinimalAPI)
class AAIController : public AController, public IAIPerceptionListenerInterface, public IGameplayTaskOwnerInterface, public IGenericTeamAgentInterface, public IVisualLoggerDebugSnapshotInterface
{
	GENERATED_BODY()

	FGameplayResourceSet ScriptClaimedResources;
protected:
	FFocusKnowledge	FocusInformation;

	/** By default AI's logic does not start when controlled Pawn is possessed. Setting this flag to true
	 *	will make AI logic start when pawn is possessed */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AI)
	uint32 bStartAILogicOnPossess : 1;

	/** By default AI's logic gets stopped when controlled Pawn is unpossessed. Setting this flag to false
	 *	will make AI logic persist past losing control over a pawn */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AI)
	uint32 bStopAILogicOnUnposses : 1;

public:
	/** used for alternating LineOfSight traces */
	UPROPERTY()
	mutable uint32 bLOSflag : 1;

	/** Skip extra line of sight traces to extremities of target being checked. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AI)
	uint32 bSkipExtraLOSChecks : 1;

	/** Is strafing allowed during movement? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AI)
	uint32 bAllowStrafe : 1;

	/** Specifies if this AI wants its own PlayerState. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AI)
	uint32 bWantsPlayerState : 1;

	/** Copy Pawn rotation to ControlRotation, if there is no focus point. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AI)
	uint32 bSetControlRotationFromPawnOrientation:1;

	/** If true, the RootComponent's transform will not be updated when SetControlRotation() is called. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AI)
	uint32 bSkipRootComponentUpdate:1;

private:

	/** Component used for moving along a path. */
	UPROPERTY(VisibleDefaultsOnly, Category = AI)
	TObjectPtr<UPathFollowingComponent> PathFollowingComponent;

public:

	/** Component responsible for behaviors. */
	UPROPERTY(BlueprintReadWrite, Category = AI)
	TObjectPtr<UBrainComponent> BrainComponent;

	UPROPERTY(VisibleDefaultsOnly, Category = AI)
	TObjectPtr<UAIPerceptionComponent> PerceptionComponent;

protected:
	/** blackboard */
	UPROPERTY(BlueprintReadOnly, Category = AI, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UBlackboardComponent> Blackboard;

	UPROPERTY()
	TObjectPtr<UGameplayTasksComponent> CachedGameplayTasksComponent;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AI)
	TSubclassOf<UNavigationQueryFilter> DefaultNavigationFilterClass;

public:

	AIMODULE_API AAIController(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	AIMODULE_API virtual void SetPawn(APawn* InPawn) override;

	/** Makes AI go toward specified Goal actor (destination will be continuously updated), aborts any active path following
	 *  @param AcceptanceRadius - finish move if pawn gets close enough
	 *  @param bStopOnOverlap - add pawn's radius to AcceptanceRadius
	 *  @param bUsePathfinding - use navigation data to calculate path (otherwise it will go in straight line)
	 *  @param bCanStrafe - set focus related flag: bAllowStrafe
	 *  @param FilterClass - navigation filter for pathfinding adjustments. If none specified DefaultNavigationFilterClass will be used
	 *  @param bAllowPartialPath - use incomplete path when goal can't be reached
	 *	@note AcceptanceRadius has default value or -1 due to Header Parser not being able to recognize UPathFollowingComponent::DefaultAcceptanceRadius
	 */
	UFUNCTION(BlueprintCallable, Category = "AI|Navigation", Meta = (AdvancedDisplay = "bStopOnOverlap,bCanStrafe,bAllowPartialPath"))
	AIMODULE_API EPathFollowingRequestResult::Type MoveToActor(AActor* Goal, float AcceptanceRadius = -1, bool bStopOnOverlap = true,
		bool bUsePathfinding = true, bool bCanStrafe = true,
		TSubclassOf<UNavigationQueryFilter> FilterClass = {}, bool bAllowPartialPath = true);

	/** Makes AI go toward specified Dest location, aborts any active path following
	 *  @param AcceptanceRadius - finish move if pawn gets close enough
	 *  @param bStopOnOverlap - add pawn's radius to AcceptanceRadius
	 *  @param bUsePathfinding - use navigation data to calculate path (otherwise it will go in straight line)
	 *  @param bProjectDestinationToNavigation - project location on navigation data before using it
	 *  @param bCanStrafe - set focus related flag: bAllowStrafe
	 *  @param FilterClass - navigation filter for pathfinding adjustments. If none specified DefaultNavigationFilterClass will be used
	 *  @param bAllowPartialPath - use incomplete path when goal can't be reached
	 *	@note AcceptanceRadius has default value or -1 due to Header Parser not being able to recognize UPathFollowingComponent::DefaultAcceptanceRadius
	 */
	UFUNCTION(BlueprintCallable, Category = "AI|Navigation", Meta = (AdvancedDisplay = "bStopOnOverlap,bCanStrafe,bAllowPartialPath"))
	AIMODULE_API EPathFollowingRequestResult::Type MoveToLocation(const FVector& Dest, float AcceptanceRadius = -1, bool bStopOnOverlap = true,
		bool bUsePathfinding = true, bool bProjectDestinationToNavigation = false, bool bCanStrafe = true,
		TSubclassOf<UNavigationQueryFilter> FilterClass = {}, bool bAllowPartialPath = true);

	/** Makes AI go toward specified destination
	 *  @param MoveRequest - details about move
	 *  @param OutPath - optional output param, filled in with assigned path
	 *  @return struct holding MoveId and enum code
	 */
	AIMODULE_API virtual FPathFollowingRequestResult MoveTo(const FAIMoveRequest& MoveRequest, FNavPathSharedPtr* OutPath = nullptr);

	/** Passes move request and path object to path following */
	AIMODULE_API virtual FAIRequestID RequestMove(const FAIMoveRequest& MoveRequest, FNavPathSharedPtr Path);

	/** Finds path for given move request
 	 *  @param MoveRequest - details about move
	 *  @param Query - pathfinding query for navigation system
	 *  @param OutPath - generated path
	 */
	AIMODULE_API virtual void FindPathForMoveRequest(const FAIMoveRequest& MoveRequest, FPathFindingQuery& Query, FNavPathSharedPtr& OutPath) const;

	/** Helper function for creating pathfinding query for this agent from move request data */
	AIMODULE_API bool BuildPathfindingQuery(const FAIMoveRequest& MoveRequest, FPathFindingQuery& OutQuery) const;

	/** Helper function for creating pathfinding query for this agent from move request data and starting location */
	AIMODULE_API bool BuildPathfindingQuery(const FAIMoveRequest& MoveRequest, const FVector& StartLocation, FPathFindingQuery& OutQuery) const;

	UE_DEPRECATED_FORGAME(4.13, "This function is now deprecated, please use FindPathForMoveRequest() for adjusting Query or BuildPathfindingQuery() for getting one.")
	AIMODULE_API virtual bool PreparePathfinding(const FAIMoveRequest& MoveRequest, FPathFindingQuery& Query);

	UE_DEPRECATED_FORGAME(4.13, "This function is now deprecated, please use FindPathForMoveRequest() for adjusting pathfinding or path postprocess.")
	AIMODULE_API virtual FAIRequestID RequestPathAndMove(const FAIMoveRequest& MoveRequest, FPathFindingQuery& Query);

	/** if AI is currently moving due to request given by RequestToPause, then the move will be paused */
	AIMODULE_API bool PauseMove(FAIRequestID RequestToPause);

	/** resumes last AI-performed, paused request provided it's ID was equivalent to RequestToResume */
	AIMODULE_API bool ResumeMove(FAIRequestID RequestToResume);

	/** Aborts the move the controller is currently performing */
	AIMODULE_API virtual void StopMovement() override;

	/** Called on completing current movement request */
	AIMODULE_API virtual void OnMoveCompleted(FAIRequestID RequestID, const FPathFollowingResult& Result);

	UE_DEPRECATED_FORGAME(4.13, "This function is now deprecated, please use version with EPathFollowingResultDetails parameter.")
	AIMODULE_API virtual void OnMoveCompleted(FAIRequestID RequestID, EPathFollowingResult::Type Result);

	/** Returns the Move Request ID for the current move */
	AIMODULE_API FAIRequestID GetCurrentMoveRequestID() const;

	/** Blueprint notification that we've completed the current movement request */
	UPROPERTY(BlueprintAssignable, meta = (DisplayName = "MoveCompleted"))
	FAIMoveCompletedSignature ReceiveMoveCompleted;

	TSubclassOf<UNavigationQueryFilter> GetDefaultNavigationFilterClass() const { return DefaultNavigationFilterClass; }

	/** Returns status of path following */
	UFUNCTION(BlueprintCallable, Category = "AI|Navigation")
	AIMODULE_API EPathFollowingStatus::Type GetMoveStatus() const;

	/** Returns true if the current PathFollowingComponent's path is partial (does not reach desired destination). */
	UFUNCTION(BlueprintCallable, Category = "AI|Navigation")
	AIMODULE_API bool HasPartialPath() const;

	/** Returns position of current path segment's end. */
	UFUNCTION(BlueprintCallable, Category = "AI|Navigation")
	AIMODULE_API FVector GetImmediateMoveDestination() const;

	/** Updates state of movement block detection. */
	UFUNCTION(BlueprintCallable, Category = "AI|Navigation")
	AIMODULE_API void SetMoveBlockDetection(bool bEnable);
	
	/** Starts executing behavior tree. */
	UFUNCTION(BlueprintCallable, Category = "AI")
	AIMODULE_API virtual bool RunBehaviorTree(UBehaviorTree* BTAsset);

	/** Set the control rotation. The RootComponent's rotation will also be updated to match
	  * if bSkipRootComponentUpdate is false and RootComponent->bAbsoluteRotation is true. */
	AIMODULE_API virtual void SetControlRotation(const FRotator& NewRotation) override;

protected:
	AIMODULE_API virtual void CleanupBrainComponent();

	/** Merges the remaining points of InitialPath, with the points of InOutMergedPath. The resulting merged path is outputted into InOutMergedPath. */
	AIMODULE_API virtual void MergePaths(const FNavPathSharedPtr& InitialPath, FNavPathSharedPtr& InOutMergedPath) const;

public:
	/**
	 * Makes AI use the specified Blackboard asset & creates a Blackboard Component if one does not already exist.
	 * @param	BlackboardAsset			The Blackboard asset to use.
	 * @param	BlackboardComponent		The Blackboard component that was used or created to work with the passed-in Blackboard Asset.
	 * @return true if we successfully linked the blackboard asset to the blackboard component.
	 */
	UFUNCTION(BlueprintCallable, Category = "AI")
	AIMODULE_API bool UseBlackboard(UBlackboardData* BlackboardAsset, UBlackboardComponent*& BlackboardComponent);

	/** does this AIController allow given UBlackboardComponent sync data with it */
	AIMODULE_API virtual bool ShouldSyncBlackboardWith(const UBlackboardComponent& OtherBlackboardComponent) const;

	UFUNCTION(BlueprintCallable, Category = "AI|Tasks")
	AIMODULE_API void ClaimTaskResource(TSubclassOf<UGameplayTaskResource> ResourceClass);
	
	UFUNCTION(BlueprintCallable, Category = "AI|Tasks")
	AIMODULE_API void UnclaimTaskResource(TSubclassOf<UGameplayTaskResource> ResourceClass);

protected:
	UFUNCTION(BlueprintImplementableEvent)
	AIMODULE_API void OnUsingBlackBoard(UBlackboardComponent* BlackboardComp, UBlackboardData* BlackboardAsset);

	AIMODULE_API virtual bool InitializeBlackboard(UBlackboardComponent& BlackboardComp, UBlackboardData& BlackboardAsset);

public:
	/** Retrieve the final position that controller should be looking at. */
	UFUNCTION(BlueprintCallable, Category = "AI")
	AIMODULE_API FVector GetFocalPoint() const;

	AIMODULE_API FVector GetFocalPointForPriority(EAIFocusPriority::Type InPriority) const;

	/** Retrieve the focal point this controller should focus to on given actor. */
	UFUNCTION(BlueprintCallable, Category = "AI")
	AIMODULE_API virtual FVector GetFocalPointOnActor(const AActor *Actor) const;

	/** Set the position that controller should be looking at. */
	UFUNCTION(BlueprintCallable, Category = "AI", meta = (DisplayName = "SetFocalPoint", ScriptName = "SetFocalPoint", Keywords = "focus"))
	AIMODULE_API void K2_SetFocalPoint(FVector FP);

	/** Set Focus for actor, will set FocalPoint as a result. */
	UFUNCTION(BlueprintCallable, Category = "AI", meta = (DisplayName = "SetFocus", ScriptName = "SetFocus"))
	AIMODULE_API void K2_SetFocus(AActor* NewFocus);

	/** Get the focused actor. */
	UFUNCTION(BlueprintCallable, Category = "AI")
	AIMODULE_API AActor* GetFocusActor() const;

	inline AActor* GetFocusActorForPriority(EAIFocusPriority::Type InPriority) const { return FocusInformation.Priorities.IsValidIndex(InPriority) ? FocusInformation.Priorities[InPriority].Actor.Get() : nullptr; }

	/** Clears Focus, will also clear FocalPoint as a result */
	UFUNCTION(BlueprintCallable, Category = "AI", meta = (DisplayName = "ClearFocus", ScriptName = "ClearFocus"))
	AIMODULE_API void K2_ClearFocus();


	/**
	 * Computes a launch velocity vector to toss a projectile and hit the given destination.
	 * Performance note: Potentially expensive. Nonzero CollisionRadius and bOnlyTraceUp=false are the more expensive options.
	 *
	 * @param OutTossVelocity - out param stuffed with the computed velocity to use
	 * @param Start - desired start point of arc
	 * @param End - desired end point of arc
	 * @param TossSpeed - Initial speed of the theoretical projectile. Assumed to only change due to gravity for the entire lifetime of the projectile
	 * @param CollisionSize (optional) - is the size of bounding box of the tossed actor (defaults to (0,0,0)
	 * @param bOnlyTraceUp  (optional) - when true collision checks verifying the arc will only be done along the upward portion of the arc
	 * @return - true if a valid arc was computed, false if no valid solution could be found
	 */
	AIMODULE_API bool SuggestTossVelocity(FVector& OutTossVelocity, FVector Start, FVector End, float TossSpeed, bool bPreferHighArc, float CollisionRadius = 0, bool bOnlyTraceUp = false);

	//~ Begin AActor Interface
	AIMODULE_API virtual void Tick(float DeltaTime) override;
	AIMODULE_API virtual void PostInitializeComponents() override;
	AIMODULE_API virtual void PostRegisterAllComponents() override;
	//~ End AActor Interface

	//~ Begin AController Interface
protected:
	AIMODULE_API virtual void OnPossess(APawn* InPawn) override;
	AIMODULE_API virtual void OnUnPossess() override;

public:
	AIMODULE_API virtual bool ShouldPostponePathUpdates() const override;
	AIMODULE_API virtual void DisplayDebug(UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos) override;

#if ENABLE_VISUAL_LOG
	AIMODULE_API virtual void GrabDebugSnapshot(FVisualLogEntry* Snapshot) const override;
#endif

	AIMODULE_API virtual void Reset() override;

	/**
	 * Checks line to center and top of other actor
	 * @param Other is the actor whose visibility is being checked.
	 * @param ViewPoint is eye position visibility is being checked from.  If vect(0,0,0) passed in, uses current viewtarget's eye position.
	 * @param bAlternateChecks used only in AIController implementation
	 * @return true if controller's pawn can see Other actor.
	 */
	AIMODULE_API virtual bool LineOfSightTo(const AActor* Other, FVector ViewPoint = FVector(ForceInit), bool bAlternateChecks = false) const override;
	//~ End AController Interface

	/** Notifies AIController of changes in given actors' perception */
	AIMODULE_API virtual void ActorsPerceptionUpdated(const TArray<AActor*>& UpdatedActors);

	/** Update direction AI is looking based on FocalPoint */
	AIMODULE_API virtual void UpdateControlRotation(float DeltaTime, bool bUpdatePawn = true);

	/** Set FocalPoint for given priority as absolute position or offset from base. */
	AIMODULE_API virtual void SetFocalPoint(FVector NewFocus, EAIFocusPriority::Type InPriority = EAIFocusPriority::Gameplay);

	/* Set Focus actor for given priority, will set FocalPoint as a result. */
	AIMODULE_API virtual void SetFocus(AActor* NewFocus, EAIFocusPriority::Type InPriority = EAIFocusPriority::Gameplay);

	/** Clears Focus for given priority, will also clear FocalPoint as a result
	 *	@param InPriority focus priority to clear. If you don't know what to use you probably mean EAIFocusPriority::Gameplay*/
	AIMODULE_API virtual void ClearFocus(EAIFocusPriority::Type InPriority);

	AIMODULE_API void SetPerceptionComponent(UAIPerceptionComponent& InPerceptionComponent);
	//----------------------------------------------------------------------//
	// IAIPerceptionListenerInterface
	//----------------------------------------------------------------------//
	virtual UAIPerceptionComponent* GetPerceptionComponent() override { return GetAIPerceptionComponent(); }

	//----------------------------------------------------------------------//
	// INavAgentInterface
	//----------------------------------------------------------------------//
	AIMODULE_API virtual bool IsFollowingAPath() const override;
	AIMODULE_API virtual IPathFollowingAgentInterface* GetPathFollowingAgent() const override;

	//----------------------------------------------------------------------//
	// IGenericTeamAgentInterface
	//----------------------------------------------------------------------//
private:
	FGenericTeamId TeamID;
public:
	AIMODULE_API virtual void SetGenericTeamId(const FGenericTeamId& NewTeamID) override;
	virtual FGenericTeamId GetGenericTeamId() const override { return TeamID; }

	//----------------------------------------------------------------------//
	// IGameplayTaskOwnerInterface
	//----------------------------------------------------------------------//
	virtual UGameplayTasksComponent* GetGameplayTasksComponent(const UGameplayTask& Task) const override { return GetGameplayTasksComponent(); }
	virtual AActor* GetGameplayTaskOwner(const UGameplayTask* Task) const override { return const_cast<AAIController*>(this); }
	virtual AActor* GetGameplayTaskAvatar(const UGameplayTask* Task) const override { return GetPawn(); }
	virtual uint8 GetGameplayTaskDefaultPriority() const { return FGameplayTasks::DefaultPriority - 1; }

	inline UGameplayTasksComponent* GetGameplayTasksComponent() const { return CachedGameplayTasksComponent; }

	// add empty overrides to fix linker errors if project implements a child class without adding GameplayTasks module dependency
	virtual void OnGameplayTaskInitialized(UGameplayTask& Task) override {}
	virtual void OnGameplayTaskActivated(UGameplayTask& Task) override {}
	virtual void OnGameplayTaskDeactivated(UGameplayTask& Task) override {}

	UFUNCTION()
	AIMODULE_API virtual void OnGameplayTaskResourcesClaimed(FGameplayResourceSet NewlyClaimed, FGameplayResourceSet FreshlyReleased);

	//----------------------------------------------------------------------//
	// debug/dev-time 
	//----------------------------------------------------------------------//
	AIMODULE_API virtual FString GetDebugIcon() const;
	
	// Cheat/debugging functions
	static void ToggleAIIgnorePlayers() { bAIIgnorePlayers = !bAIIgnorePlayers; }
	static bool AreAIIgnoringPlayers() { return bAIIgnorePlayers; }

	/** If true, AI controllers will ignore players. */
	static AIMODULE_API bool bAIIgnorePlayers;

public:
	/** Returns PathFollowingComponent subobject **/
	UFUNCTION(BlueprintCallable, Category="AI|Navigation")
	UPathFollowingComponent* GetPathFollowingComponent() const { return PathFollowingComponent; }
	UFUNCTION(BlueprintPure, Category = "AI|Perception")
	UAIPerceptionComponent* GetAIPerceptionComponent() { return PerceptionComponent; }

	const UAIPerceptionComponent* GetAIPerceptionComponent() const { return PerceptionComponent; }

	UBrainComponent* GetBrainComponent() const { return BrainComponent; }
	const UBlackboardComponent* GetBlackboardComponent() const { return Blackboard; }
	UBlackboardComponent* GetBlackboardComponent() { return Blackboard; }

	/** Note that this function does not do any pathfollowing state transfer. 
	 *	Intended to be called as part of initialization/setup process */
	UFUNCTION(BlueprintCallable, Category = "AI|Navigation")
	AIMODULE_API void SetPathFollowingComponent(UPathFollowingComponent* NewPFComponent);
};

//----------------------------------------------------------------------//
// forceinlines
//----------------------------------------------------------------------//
namespace FAISystem
{
	inline bool IsValidControllerAndHasValidPawn(const AController* Controller)
	{
		return Controller != nullptr && Controller->IsPendingKillPending() == false
			&& Controller->GetPawn() != nullptr && Controller->GetPawn()->IsPendingKillPending() == false;
	}
}
