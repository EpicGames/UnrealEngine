// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "AITypes.h"
#include "AIResourceInterface.generated.h"

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UAIResourceInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IAIResourceInterface
{
	GENERATED_IINTERFACE_BODY()

	/** If resource is lockable lock it with indicated priority */
	virtual void LockResource(EAIRequestPriority::Type LockSource) {}

	/** clear resource lock of the given origin */
	virtual void ClearResourceLock(EAIRequestPriority::Type LockSource) {}

	/** Force-clears all locks on resource */
	virtual void ForceUnlockResource() {}
	
	/** check whether resource is currently locked */
	virtual bool IsResourceLocked() const {return false;}
};
