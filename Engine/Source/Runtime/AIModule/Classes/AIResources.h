// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameplayTaskResource.h"
#include "AIResources.generated.h"

UCLASS(meta = (DisplayName = "AI Movement"), MinimalAPI)
class UAIResource_Movement : public UGameplayTaskResource
{
	GENERATED_BODY()

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	AIMODULE_API virtual FString GenerateDebugDescription() const override;
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
};

UCLASS(meta = (DisplayName = "AI Logic"), MinimalAPI)
class UAIResource_Logic : public UGameplayTaskResource
{
	GENERATED_BODY()
};
