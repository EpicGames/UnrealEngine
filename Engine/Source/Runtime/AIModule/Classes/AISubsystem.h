// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Tickable.h"
#include "AISystem.h"
#include "AISubsystem.generated.h"


class UAISystem;

/** A class representing a common interface and behavior for AI subsystems */
UCLASS(config = Engine, defaultconfig, MinimalAPI)
class UAISubsystem : public UObject, public FTickableGameObject
{
	GENERATED_BODY()

private:
	UPROPERTY()
	TObjectPtr<UAISystem> AISystem;
			
public:
	AIMODULE_API UAISubsystem(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	AIMODULE_API virtual UWorld* GetWorld() const override;
	AIMODULE_API virtual void BeginDestroy() override;

	// FTickableGameObject begin
	virtual UWorld* GetTickableGameObjectWorld() const override { return GetWorldFast(); }
	virtual void Tick(float DeltaTime) override {}
	AIMODULE_API virtual ETickableTickType GetTickableTickType() const override;
	AIMODULE_API virtual TStatId GetStatId() const override;
	// FTickableGameObject end

	UWorld* GetWorldFast() const { return AISystem ? AISystem->GetOuterWorld() : GetOuter()->GetWorld(); }
};
