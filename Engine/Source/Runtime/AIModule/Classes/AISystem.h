// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "Engine/World.h"
#include "AI/AISystemBase.h"
#include "Math/RandomStream.h"
#include "AISystem.generated.h"

class UAIAsyncTaskBlueprintProxy;
class UAIHotSpotManager;
class UAIPerceptionSystem;
class UAISystem;
class UBehaviorTreeManager;
class UBlackboardComponent;
class UBlackboardData;
class UEnvQueryManager;
class UNavLocalGridManager;

#define GET_AI_CONFIG_VAR(a) (GetDefault<UAISystem>()->a)

UCLASS(config=Engine, defaultconfig, MinimalAPI)
class UAISystem : public UAISystemBase
{
	GENERATED_BODY()

protected:
	/** Class that will be used to spawn the perception system, can be game-specific */
	UPROPERTY(globalconfig, EditAnywhere, Category = "AISystem", meta = (MetaClass = "/Script/AIModule.AIPerceptionSystem", DisplayName = "Perception System Class"))
	FSoftClassPath PerceptionSystemClassName;

	/** Class that will be used to spawn the hot spot manager, can be game-specific */
	UPROPERTY(globalconfig, EditAnywhere, Category = "AISystem", meta = (MetaClass = "/Script/AIModule.AIHotSpotManager", DisplayName = "AIHotSpotManager Class"))
	FSoftClassPath HotSpotManagerClassName;

	/** Class that will be used to spawn the env query manager, can be game-specific */
	UPROPERTY(globalconfig, EditAnywhere, Category = "AISystem", meta = (MetaClass = "/Script/AIModule.EnvQueryManager", DisplayName = "EnvQueryManager Class"))
	FSoftClassPath EnvQueryManagerClassName;
public:
	/** Default AI movement's acceptance radius used to determine whether 
 	 * AI reached path's end */
	UPROPERTY(globalconfig, EditDefaultsOnly, Category = "Movement")
	float AcceptanceRadius; 

	/** Value used for pathfollowing's internal code to determine whether AI reached path's point. 
	 *	@note this value is not used for path's last point. @see AcceptanceRadius*/
	UPROPERTY(globalconfig, EditDefaultsOnly, Category = "Movement")
	float PathfollowingRegularPathPointAcceptanceRadius;
	
	/** Similarly to PathfollowingRegularPathPointAcceptanceRadius used by pathfollowing's internals
	 *	but gets applied only when next point on a path represents a begining of navigation link */
	UPROPERTY(globalconfig, EditDefaultsOnly, Category = "Movement")
	float PathfollowingNavLinkAcceptanceRadius;
	
	/** If true, overlapping the goal will be counted by default as finishing a move */
	UPROPERTY(globalconfig, EditDefaultsOnly, Category = "Movement")
	bool bFinishMoveOnGoalOverlap;

	/** Sets default value for rather move tasks accept partial paths or not */
	UPROPERTY(globalconfig, EditDefaultsOnly, Category = "Movement")
	bool bAcceptPartialPaths;

	/** Sets default value for rather move tasks allow strafing or not */
	UPROPERTY(globalconfig, EditDefaultsOnly, Category = "Movement")
	bool bAllowStrafing;

	/** if enable will make EQS not complaint about using Controllers as queriers. Default behavior (false) will 
	 *	in places automatically convert controllers to pawns, and complain if code user bypasses the conversion or uses
	 *	pawn-less controller */
	UPROPERTY(globalconfig, EditDefaultsOnly, Category = "EQS")
	bool bAllowControllersAsEQSQuerier;

	/** if set, GameplayDebuggerPlugin will be loaded on module's startup */
	UPROPERTY(globalconfig, EditDefaultsOnly, Category = "AISystem")
	bool bEnableDebuggerPlugin;

	/** If set, actors will be forgotten by the perception system when their stimulus has expired.
	 *	If not set, the perception system will remember the actor even if they are no longer perceived and their
	 *	stimuli has exceeded its max age */
	UPROPERTY(globalconfig, EditDefaultsOnly, Category = "AISystem")
	bool bForgetStaleActors;

	/** If set to true will result in automatically adding the SelfActor key to new Blackboard assets. It will 
	 *	also result in making sure all the BB assets loaded do have the SelfKey entry, via PostLoad */
	UPROPERTY(globalconfig, EditDefaultsOnly, Category = "Blackboard")
	bool bAddBlackboardSelfKey = true;

	UPROPERTY(globalconfig, EditDefaultsOnly, Category = "Behavior Tree")
	bool bClearBBEntryOnBTEQSFail = true;
	
	/** If enabled, blackboard based decorators will set key to 'Invalid' on creation or when selected key no longer exists (instead of using the first key of the blackboard). */
	UPROPERTY(globalconfig, EditDefaultsOnly, Category = "Behavior Tree")
	bool bBlackboardKeyDecoratorAllowsNoneAsValue = false;

	/** If set, new BTs will use this BB as default. */
	UPROPERTY(globalconfig, EditDefaultsOnly, Category = "Behavior Tree")
	TSoftObjectPtr<UBlackboardData> DefaultBlackboard;

	/** Which collision channel to use for sight checks by default */
	UPROPERTY(globalconfig, EditDefaultsOnly, Category = "PerceptionSystem")
	TEnumAsByte<ECollisionChannel> DefaultSightCollisionChannel;

protected:
	/** Behavior tree manager used by game */
	UPROPERTY(Transient)
	TObjectPtr<UBehaviorTreeManager> BehaviorTreeManager;

	/** Environment query manager used by game */
	UPROPERTY(Transient)
	TObjectPtr<UEnvQueryManager> EnvironmentQueryManager;

	UPROPERTY(Transient)
	TObjectPtr<UAIPerceptionSystem> PerceptionSystem;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UAIAsyncTaskBlueprintProxy>> AllProxyObjects;

	UPROPERTY(Transient)
	TObjectPtr<UAIHotSpotManager> HotSpotManager;

	UPROPERTY(Transient)
	TObjectPtr<UNavLocalGridManager> NavLocalGrids;

	typedef TMultiMap<TWeakObjectPtr<UBlackboardData>, TWeakObjectPtr<UBlackboardComponent> > FBlackboardDataToComponentsMap;

	/** UBlackboardComponent instances that reference the blackboard data definition */
	FBlackboardDataToComponentsMap BlackboardDataToComponentsMap;

	FDelegateHandle ActorSpawnedDelegateHandle;

	FDelegateHandle PawnBeginPlayDelegateHandle;

	/** random number stream to be used by all things AI. WIP */
	static AIMODULE_API FRandomStream RandomStream;
	
public:
	AIMODULE_API UAISystem(const FObjectInitializer& ObjectInitializer);

	AIMODULE_API virtual void BeginDestroy() override;
	
	AIMODULE_API virtual void PostInitProperties() override;

	// UAISystemBase begin		
	AIMODULE_API virtual void InitializeActorsForPlay(bool bTimeGotReset) override;
	AIMODULE_API virtual void WorldOriginLocationChanged(FIntVector OldOriginLocation, FIntVector NewOriginLocation) override;
	AIMODULE_API virtual void CleanupWorld(bool bSessionEnded = true, bool bCleanupResources = true) override;
	UE_DEPRECATED(5.1, "NewWorld was unused and not always calculated correctly and we expect it is not needed; let us know on UDN if it is necessary.")
	AIMODULE_API virtual void CleanupWorld(bool bSessionEnded, bool bCleanupResources, UWorld* NewWorld) override;
	AIMODULE_API virtual void StartPlay() override;
	// UAISystemBase end

	/** Behavior tree manager getter */
	inline UBehaviorTreeManager* GetBehaviorTreeManager() { return BehaviorTreeManager; }
	/** Behavior tree manager const getter */
	inline const UBehaviorTreeManager* GetBehaviorTreeManager() const { return BehaviorTreeManager; }

	/** Environment Query manager getter */
	inline UEnvQueryManager* GetEnvironmentQueryManager() { return EnvironmentQueryManager; }
	/** Environment Query manager const getter */
	inline const UEnvQueryManager* GetEnvironmentQueryManager() const { return EnvironmentQueryManager; }

	inline UAIPerceptionSystem* GetPerceptionSystem() { return PerceptionSystem; }
	inline const UAIPerceptionSystem* GetPerceptionSystem() const { return PerceptionSystem; }

	inline UAIHotSpotManager* GetHotSpotManager() { return HotSpotManager; }
	inline const UAIHotSpotManager* GetHotSpotManager() const { return HotSpotManager; }

	inline UNavLocalGridManager* GetNavLocalGridManager() { return NavLocalGrids; }
	inline const UNavLocalGridManager* GetNavLocalGridManager() const { return NavLocalGrids; }

	inline static UAISystem* GetCurrentSafe(UWorld* World) 
	{ 
		return World != nullptr ? Cast<UAISystem>(World->GetAISystem()) : NULL;
	}

	inline static UAISystem* GetCurrent(UWorld& World)
	{
		return Cast<UAISystem>(World.GetAISystem());
	}

	inline UWorld* GetOuterWorld() const { return Cast<UWorld>(GetOuter()); }

	virtual UWorld* GetWorld() const override { return GetOuterWorld(); }
	
	inline void AddReferenceFromProxyObject(UAIAsyncTaskBlueprintProxy* BlueprintProxy) { AllProxyObjects.AddUnique(BlueprintProxy); }

	inline void RemoveReferenceToProxyObject(UAIAsyncTaskBlueprintProxy* BlueprintProxy) { AllProxyObjects.RemoveSwap(BlueprintProxy); }

	//----------------------------------------------------------------------//
	// cheats
	//----------------------------------------------------------------------//
	UFUNCTION(exec)
	AIMODULE_API virtual void AIIgnorePlayers();

	UFUNCTION(exec)
	AIMODULE_API virtual void AILoggingVerbose();

	/** insta-runs EQS query for given Target */
	AIMODULE_API void RunEQS(const FString& QueryName, UObject* Target);

	/**
	* Iterator for traversing all UBlackboardComponent instances associated
	* with this blackboard data asset. This is a forward only iterator.
	*/
	struct FBlackboardDataToComponentsIterator
	{
	public:
		FBlackboardDataToComponentsIterator(FBlackboardDataToComponentsMap& BlackboardDataToComponentsMap, class UBlackboardData* BlackboardAsset);

		inline FBlackboardDataToComponentsIterator& operator++()
		{
			++GetCurrentIteratorRef();
			TryMoveIteratorToParentBlackboard();
			return *this;
		}
		inline FBlackboardDataToComponentsIterator operator++(int)
		{
			FBlackboardDataToComponentsIterator Tmp(*this);
			++GetCurrentIteratorRef();
			TryMoveIteratorToParentBlackboard();
			return Tmp;
		}

		inline explicit operator bool() const { return CurrentIteratorIndex < Iterators.Num() && (bool)GetCurrentIteratorRef(); }
		inline bool operator !() const { return !(bool)*this; }

		inline UBlackboardData* Key() const { return GetCurrentIteratorRef().Key().Get(); }
		inline UBlackboardComponent* Value() const { return GetCurrentIteratorRef().Value().Get(); }

	private:
		inline const FBlackboardDataToComponentsMap::TConstKeyIterator& GetCurrentIteratorRef() const { return Iterators[CurrentIteratorIndex]; }
		inline FBlackboardDataToComponentsMap::TConstKeyIterator& GetCurrentIteratorRef() { return Iterators[CurrentIteratorIndex]; }

		void TryMoveIteratorToParentBlackboard()
		{
			if (!GetCurrentIteratorRef() && CurrentIteratorIndex < Iterators.Num() - 1)
			{
				++CurrentIteratorIndex;
				TryMoveIteratorToParentBlackboard(); // keep incrementing until we find a valid iterator.
			}
		}

		int32 CurrentIteratorIndex;

		static const int32 InlineSize = 8;
		TArray<TWeakObjectPtr<UBlackboardData>> IteratorKeysForReference;
		TArray<FBlackboardDataToComponentsMap::TConstKeyIterator, TInlineAllocator<InlineSize>> Iterators;
	};

	/**
	* Registers a UBlackboardComponent instance with this blackboard data asset.
	* This will also register the component for each parent UBlackboardData
	* asset. This should be called after the component has been initialized
	* (i.e. InitializeComponent). The user is responsible for calling
	* UnregisterBlackboardComponent (i.e. UninitializeComponent).
	*/
	AIMODULE_API void RegisterBlackboardComponent(class UBlackboardData& BlackboardAsset, class UBlackboardComponent& BlackboardComp);

	/**
	* Unregisters a UBlackboardComponent instance with this blackboard data
	* asset. This should be called before the component has been uninitialized
	* (i.e. UninitializeComponent).
	*/
	AIMODULE_API void UnregisterBlackboardComponent(class UBlackboardData& BlackboardAsset, class UBlackboardComponent& BlackboardComp);

	/**
	* Creates a forward only iterator for that will iterate all
	* UBlackboardComponent instances that reference the specified
	* BlackboardAsset and it's parents.
	*/
	AIMODULE_API FBlackboardDataToComponentsIterator CreateBlackboardDataToComponentsIterator(class UBlackboardData& BlackboardAsset);

	AIMODULE_API virtual void ConditionalLoadDebuggerPlugin();

	static const FRandomStream& GetRandomStream() { return RandomStream; }
	static void SeedRandomStream(const int32 Seed) { return RandomStream.Initialize(Seed); }

protected:
	AIMODULE_API virtual void OnActorSpawned(AActor* SpawnedActor);
	AIMODULE_API virtual void OnPawnBeginPlay(APawn* Pawn);

	AIMODULE_API void LoadDebuggerPlugin();
};
