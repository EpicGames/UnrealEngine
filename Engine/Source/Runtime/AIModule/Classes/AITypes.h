// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "NavigationSystemTypes.h"
#include "AITypes.generated.h"

class AActor;
typedef TSharedPtr<FMoveRequestCustomData, ESPMode::ThreadSafe> FCustomMoveSharedPtr;

DECLARE_CYCLE_STAT_EXTERN(TEXT("Overall AI Time"), STAT_AI_Overall, STATGROUP_AI, AIMODULE_API);

#define TEXT_AI_LOCATION(v) (FAISystem::IsValidLocation(v) ? *(v).ToString() : TEXT("Invalid"))

namespace FAISystem
{
	extern AIMODULE_API const FRotator InvalidRotation;
	extern AIMODULE_API const FQuat InvalidOrientation;
	inline constexpr FVector InvalidLocation = FVector(FLT_MAX, UE::Math::TVectorConstInit{});
	inline constexpr FVector InvalidDirection = FVector(0, UE::Math::TVectorConstInit{});
	constexpr float InvalidRange = -1.f;
	constexpr float InfiniteInterval = -FLT_MAX;
	constexpr uint32 InvalidUnsignedID = uint32(INDEX_NONE);

	inline bool IsValidLocation(const FVector& TestLocation)
	{
		return -InvalidLocation.X < TestLocation.X && TestLocation.X < InvalidLocation.X
			&& -InvalidLocation.Y < TestLocation.Y && TestLocation.Y < InvalidLocation.Y
			&& -InvalidLocation.Z < TestLocation.Z && TestLocation.Z < InvalidLocation.Z;
	}

	inline bool IsValidDirection(const FVector& TestVector)
	{
		return IsValidLocation(TestVector) == true && TestVector.IsZero() == false;
	}

	inline bool IsValidRotation(const FRotator& TestRotation)
	{
		return TestRotation != InvalidRotation;
	}

	inline bool IsValidOrientation(const FQuat& TestOrientation)
	{
		return TestOrientation != InvalidOrientation;
	}
}

UENUM()
namespace EAIOptionFlag
{
	enum Type : int
	{
		Default,
		Enable UMETA(DisplayName = "Yes"),
		Disable UMETA(DisplayName = "No"),

		MAX UMETA(Hidden)
	};
}

UENUM()
enum class FAIDistanceType : uint8
{
	Distance3D,
	Distance2D,
	DistanceZ,

	MAX UMETA(Hidden)
};

namespace FAISystem
{
	inline bool PickAIOption(EAIOptionFlag::Type Option, bool DefaultOption)
	{
		return Option == EAIOptionFlag::Default ? DefaultOption : (Option == EAIOptionFlag::Enable);
	}

	inline EAIOptionFlag::Type BoolToAIOption(bool Value)
	{
		return Value ? EAIOptionFlag::Enable : EAIOptionFlag::Disable;
	}
}

namespace EAIForceParam
{
	enum Type
	{
		Force,
		DoNotForce,

		MAX
	};
}

namespace FAIMoveFlag
{
	constexpr bool StopOnOverlap = true;
	constexpr bool UsePathfinding = true;
	constexpr bool IgnorePathfinding = false;
}

namespace EAILogicResuming
{
	enum Type
	{
		Continue,
		RestartedInstead,
	};
}

UENUM()
namespace EAIRequestPriority
{
	enum Type : int
	{
		/** Actions requested by Level Designers by placing AI-hinting elements on the map. */
		SoftScript,
		/** Actions AI wants to do due to its internal logic. */
		Logic,
		/** Actions LDs really want AI to perform. */
		HardScript,
		/** Actions being result of game-world mechanics, like hit reactions, death, falling, etc. In general things not depending on what AI's thinking. */
		Reaction,
		/** Ultimate priority, to be used with caution, makes AI perform given action regardless of anything else (for example disabled reactions). */
		Ultimate,

		MAX UMETA(Hidden)
	};
}

namespace EAIRequestPriority
{
	constexpr int32 Lowest = EAIRequestPriority::Logic;
};

UENUM()
namespace EAILockSource
{
	enum Type : int
	{
		Animation,
		Logic,
		Script,
		Gameplay,

		MAX UMETA(Hidden)
	};
}

/**
 *	TCounter needs to supply following functions:
 *		default constructor
 *		typedef X Type; where X is an integer type to be used as ID's internal type
 *		TCounter::Type GetNextAvailableID() - returns next available ID and advances the internal counter
 *		uint32 GetSize() const - returns number of unique IDs created so far
 *		OnIndexForced(TCounter::Type Index) - called when given Index has been force-used. Counter may need to update "next available ID"
 */

template<typename TCounter>
struct FAINamedID
{
	const typename TCounter::Type Index;
	const FName Name;
private:
	static AIMODULE_API TCounter Counter;
protected:
	static TCounter& GetCounter() 
	{ 
		return Counter;
	}

	// back-door for forcing IDs
	FAINamedID(const FName& InName, typename TCounter::Type InIndex)
		: Index(InIndex), Name(InName)
	{
		GetCounter().OnIndexForced(InIndex);
	}

public:
	FAINamedID(const FName& InName)
		: Index(GetCounter().GetNextAvailableID()), Name(InName)
	{}

	FAINamedID(const FAINamedID& Other)
		: Index(Other.Index), Name(Other.Name)
	{}

	FAINamedID& operator=(const FAINamedID& Other)
	{
		new(this) FAINamedID(Other);
		return *this;
	}

	FAINamedID()
		: Index(typename TCounter::Type(-1)), Name(TEXT("Invalid"))
	{}

	operator typename TCounter::Type() const { return Index; }
	bool IsValid() const { return Index != InvalidID().Index; }

	static uint32 GetSize() { return GetCounter().GetSize(); }

	static FAINamedID<TCounter> InvalidID()
	{
		static const FAINamedID<TCounter> InvalidIDInstance;
		return InvalidIDInstance;
	}

	friend inline uint32 GetTypeHash(const FAINamedID& ID)
	{
		return GetTypeHash(ID.Index);
	}
};

template<typename TCounter>
struct FAIGenericID
{
	const typename TCounter::Type Index;
private:
	static AIMODULE_API TCounter Counter;
protected:
	static TCounter& GetCounter()
	{
		return Counter;
	}

	explicit FAIGenericID(typename TCounter::Type InIndex)
		: Index(InIndex)
	{}

public:

	FAIGenericID(const FAIGenericID& Other)
		: Index(Other.Index)
	{}

	FAIGenericID& operator=(const FAIGenericID& Other)
	{
		new(this) FAIGenericID(Other);
		return *this;
	}

	FAIGenericID()
		: Index(typename TCounter::Type(-1))
	{}

	static FAIGenericID GetNextID() { return FAIGenericID(GetCounter().GetNextAvailableID()); }

	UE_DEPRECATED(5.6, "AIGenericId is not convertible to its base type anymore.")
	operator typename TCounter::Type() const { return Index; }

	bool IsValid() const { return Index != InvalidID().Index; }

	bool operator==(const FAIGenericID Other) const { return Index == Other.Index; }
	bool operator!=(const FAIGenericID Other) const { return Index != Other.Index; }

	static uint32 GetSize() { return GetCounter().GetSize(); }

	static FAIGenericID<TCounter> InvalidID()
	{
		static const FAIGenericID<TCounter> InvalidIDInstance;
		return InvalidIDInstance;
	}

	friend inline uint32 GetTypeHash(const FAIGenericID& ID)
	{
		return GetTypeHash(ID.Index);
	}
};

template<typename TCounterType>
struct FAIBasicCounter
{
	typedef TCounterType Type;
protected:
	Type NextAvailableID;
public:
	FAIBasicCounter() : NextAvailableID(Type(0)) {}
	Type GetNextAvailableID() { return NextAvailableID++; }
	uint32 GetSize() const { return uint32(NextAvailableID); }
	void OnIndexForced(Type ForcedIndex) { NextAvailableID = FMath::Max<Type>(ForcedIndex + 1, NextAvailableID); }
};

//////////////////////////////////////////////////////////////////////////
struct FAIResCounter : FAIBasicCounter<uint8>
{};
typedef FAINamedID<FAIResCounter> FAIResourceID;

//////////////////////////////////////////////////////////////////////////
struct FAIResourcesSet
{
	static constexpr uint32 NoResources = 0;
	static constexpr uint32 AllResources = uint32(-1);
	static constexpr uint8 MaxFlags = 32;
private:
	uint32 Flags;
public:
	FAIResourcesSet(uint32 ResourceSetDescription = NoResources) : Flags(ResourceSetDescription) {}
	FAIResourcesSet(const FAIResourceID& Resource) : Flags(0) 
	{
		AddResource(Resource);
	}

	FAIResourcesSet& AddResourceIndex(uint8 ResourceIndex) { Flags |= (1 << ResourceIndex); return *this; }
	FAIResourcesSet& RemoveResourceIndex(uint8 ResourceIndex) { Flags &= ~(1 << ResourceIndex); return *this; }
	bool ContainsResourceIndex(uint8 ResourceID) const { return (Flags & (1 << ResourceID)) != 0; }

	FAIResourcesSet& AddResource(const FAIResourceID& Resource) { AddResourceIndex(Resource.Index); return *this; }
	FAIResourcesSet& RemoveResource(const FAIResourceID& Resource) { RemoveResourceIndex(Resource.Index); return *this; }
	bool ContainsResource(const FAIResourceID& Resource) const { return ContainsResourceIndex(Resource.Index); }

	bool IsEmpty() const { return Flags == 0; }
	void Clear() { Flags = 0; }
};

/** structure used to define which subsystem requested locking of a specific AI resource (like movement, logic, etc.) */
struct FAIResourceLock
{
	/** @note feel free to change the type if you need to support more then 16 lock sources */
	typedef uint16 FLockFlags;
	
	AIMODULE_API FAIResourceLock();
	
	AIMODULE_API void SetLock(EAIRequestPriority::Type LockPriority);
	AIMODULE_API void ClearLock(EAIRequestPriority::Type LockPriority);

	/** set whether we should use resource lock count.  clears all existing locks. */
	AIMODULE_API void SetUseResourceLockCount(bool inUseResourceLockCount);

	/** force-clears all locks */
	AIMODULE_API void ForceClearAllLocks();

	inline bool IsLocked() const
	{
		return Locks != 0;
	}

	inline bool IsLockedBy(EAIRequestPriority::Type LockPriority) const
	{
		return (Locks & (1 << LockPriority)) != 0;
	}

	/** Answers the question if given priority is allowed to use this resource.
	 *	@Note that if resource is locked with priority LockPriority this function will
	 *	return false as well */
	inline bool IsAvailableFor(EAIRequestPriority::Type LockPriority) const
	{
		for (int32 Priority = EAIRequestPriority::MAX - 1; Priority >= LockPriority; --Priority)
		{
			if ((Locks & (1 << Priority)) != 0)
			{
				return false;
			}
		}
		return true;
	}

	AIMODULE_API FString GetLockPriorityName() const;

	void operator+=(const FAIResourceLock& Other)
	{
		Locks |= Other.Locks;		
	}

	bool operator==(const FAIResourceLock& Other) const
	{
		return Locks == Other.Locks;
	}

private:
	TArray<uint8> ResourceLockCount;
	FLockFlags Locks;
	bool bUseResourceLockCount;
};

namespace FAIResources
{
	extern AIMODULE_API const FAIResourceID InvalidResource;
	extern AIMODULE_API const FAIResourceID Movement;
	extern AIMODULE_API const FAIResourceID Logic;
	extern AIMODULE_API const FAIResourceID Perception;
	
	AIMODULE_API void RegisterResource(const FAIResourceID& Resource);
	AIMODULE_API const FAIResourceID& GetResource(int32 ResourceIndex);
	AIMODULE_API int32 GetResourcesCount();
	AIMODULE_API FString GetSetDescription(FAIResourcesSet ResourceSet);
}


USTRUCT(BlueprintType)
struct FAIRequestID
{
	GENERATED_USTRUCT_BODY()
		
private:
	static constexpr uint32 AnyRequestID = 0;
	static constexpr uint32 InvalidRequestID = uint32(-1);

	UPROPERTY()
	uint32 RequestID;

public:
	FAIRequestID(uint32 InRequestID = InvalidRequestID) : RequestID(InRequestID)
	{}

	/** returns true if given ID is identical to stored ID or any of considered
	 *	IDs is FAIRequestID::AnyRequest*/
	inline bool IsEquivalent(uint32 OtherID) const 
	{
		return OtherID != InvalidRequestID && this->IsValid() && (RequestID == OtherID || RequestID == AnyRequestID || OtherID == AnyRequestID);
	}

	inline bool IsEquivalent(FAIRequestID Other) const
	{
		return IsEquivalent(Other.RequestID);
	}

	inline bool IsValid() const
	{
		return RequestID != InvalidRequestID;
	}

	inline uint32 GetID() const { return RequestID; }

	void operator=(uint32 OtherID)
	{
		RequestID = OtherID;
	}

	operator uint32() const
	{
		return RequestID;
	}

	FString ToString() const
	{
		return FString::FromInt(int32(RequestID));
	}

	static AIMODULE_API const FAIRequestID AnyRequest;
	static AIMODULE_API const FAIRequestID CurrentRequest;
	static AIMODULE_API const FAIRequestID InvalidRequest;
};

//////////////////////////////////////////////////////////////////////////

class UNavigationQueryFilter;

USTRUCT()
struct FAIMoveRequest
{
	GENERATED_USTRUCT_BODY()

	AIMODULE_API FAIMoveRequest();
	AIMODULE_API FAIMoveRequest(const AActor* InGoalActor);
	AIMODULE_API FAIMoveRequest(const FVector& InGoalLocation);

	FAIMoveRequest& SetNavigationFilter(TSubclassOf<UNavigationQueryFilter> Filter) { FilterClass = Filter; return *this; }
	FAIMoveRequest& SetUsePathfinding(bool bPathfinding) { bUsePathfinding = bPathfinding; return *this; }
	FAIMoveRequest& SetAllowPartialPath(bool bAllowPartial) { bAllowPartialPath = bAllowPartial; return *this; }
	FAIMoveRequest& SetRequireNavigableEndLocation(bool bRequire) { bRequireNavigableEndLocation = bRequire; return *this; }
	/** Defines if the underlying pathfind query should limit its exploration based on the navigation cost
	 * @param bApply				if set - the pathfind query cost will be limited based on the heuristic between the start and end location
	 * @param InCostLimitFactor		this multiplier is used to compute a max node cost allowed to the open list (cost limit = CostLimitFactor*InitialHeuristicEstimate)
	 * @param InMinimumCostLimit	minimum cost limit clamping value (in cost units) used to allow large deviation in short paths
	 */
	FAIMoveRequest& SetApplyCostLimitFromHeuristic(bool bApply, float InCostLimitFactor = FLT_MAX, float InMinimumCostLimit = 0.f) { bApplyCostLimitFromHeuristic = bApply; CostLimitFactor = InCostLimitFactor; MinimumCostLimit = InMinimumCostLimit; return *this; }
	FAIMoveRequest& SetProjectGoalLocation(bool bProject) { bProjectGoalOnNavigation = bProject; return *this; }

	FAIMoveRequest& SetCanStrafe(bool bStrafe) { bCanStrafe = bStrafe; return *this; }
	FAIMoveRequest& SetReachTestIncludesAgentRadius(bool bIncludeRadius) { bReachTestIncludesAgentRadius = bIncludeRadius; return *this; }
	FAIMoveRequest& SetReachTestIncludesGoalRadius(bool bIncludeRadius) { bReachTestIncludesGoalRadius = bIncludeRadius; return *this; }
	FAIMoveRequest& SetAcceptanceRadius(float Radius) { AcceptanceRadius = Radius; return *this; }
	FAIMoveRequest& SetUserData(const FCustomMoveSharedPtr& InUserData) { UserData = InUserData; return *this; }
	FAIMoveRequest& SetUserFlags(int32 InUserFlags) { UserFlags = InUserFlags; return *this; }
	FAIMoveRequest& SetStartFromPreviousPath(const bool bInStartFromPreviousPath) { bStartFromPreviousPath = bInStartFromPreviousPath; return *this; }

	/** the request should be either set up to move to a location, of go to a valid actor */
	bool IsValid() const { return bInitialized && (!bMoveToActor || GoalActor.IsValid()); }

	bool IsMoveToActorRequest() const { return bMoveToActor; }
	AActor* GetGoalActor() const { return bMoveToActor ? GoalActor.Get() : nullptr; }
	FVector GetGoalLocation() const { return GoalLocation; }
	/** retrieves request's requested destination location, GoalActor's location 
	 *	or GoalLocation, depending on the request itself */
	AIMODULE_API FVector GetDestination() const;

	bool IsUsingPathfinding() const { return bUsePathfinding; }
	bool IsUsingPartialPaths() const { return bAllowPartialPath; }
	bool IsNavigableEndLocationRequired() const { return bRequireNavigableEndLocation; }
	bool IsProjectingGoal() const { return bProjectGoalOnNavigation; }
	bool IsApplyingCostLimitFromHeuristic() const { return bApplyCostLimitFromHeuristic; }
	float GetCostLimitFactor() const { return CostLimitFactor; }
	float GetMinimumCostLimit() const { return MinimumCostLimit; }
	TSubclassOf<UNavigationQueryFilter> GetNavigationFilter() const { return FilterClass; }

	bool CanStrafe() const { return bCanStrafe; }
	bool IsReachTestIncludingAgentRadius() const { return bReachTestIncludesAgentRadius; }
	bool IsReachTestIncludingGoalRadius() const { return bReachTestIncludesGoalRadius; }
	float GetAcceptanceRadius() const { return AcceptanceRadius; }
	const FCustomMoveSharedPtr& GetUserData() const { return UserData; }
	int32 GetUserFlags() const { return UserFlags; }

	const bool ShouldStartFromPreviousPath() const { return bStartFromPreviousPath; }
	AIMODULE_API void SetGoalActor(const AActor* InGoalActor);
	AIMODULE_API void SetGoalLocation(const FVector& InGoalLocation);

	AIMODULE_API bool UpdateGoalLocation(const FVector& NewLocation) const;
	AIMODULE_API FString ToString() const;

protected:

	/** move goal: actor */
	UPROPERTY()
	TWeakObjectPtr<AActor> GoalActor;

	/** move goal: location */
	mutable FVector GoalLocation;

	/** pathfinding: navigation filter to use */
	TSubclassOf<UNavigationQueryFilter> FilterClass;

	/** move goal is an actor */
	uint32 bInitialized : 1;

	/** move goal is an actor */
	uint32 bMoveToActor : 1;

	/** pathfinding: if set - regular pathfinding will be used, if not - direct path between two points */
	uint32 bUsePathfinding : 1;

	/** pathfinding: allow using incomplete path going toward goal but not reaching it */
	uint32 bAllowPartialPath : 1;

	/** pathfinding: if set - require the end location to be linked to the navigation data*/
	uint32 bRequireNavigableEndLocation : 1;

	/** pathfinding: if set - the pathfind query cost will be limited based on the heuristic between the start and end location (c.f. CostLimitFactor and MinimumCostLimit). */
	uint32 bApplyCostLimitFromHeuristic : 1;

	/** pathfinding: goal location will be projected on navigation data before use */
	uint32 bProjectGoalOnNavigation : 1;

	/** pathfinding: the request will start from the end of the previous path (if any), and the generated path will be merged with the remaining points of the previous path */
	uint32 bStartFromPreviousPath : 1;

	/** pathfollowing: acceptance radius needs to be increased by agent radius (stop on overlap vs exact point) */
	uint32 bReachTestIncludesAgentRadius : 1;

	/** pathfollowing: acceptance radius needs to be increased by goal actor radius */
	uint32 bReachTestIncludesGoalRadius : 1;

	/** pathfollowing: keep focal point at move goal */
	uint32 bCanStrafe : 1;

	/** pathfollowing: required distance to goal to complete move */
	float AcceptanceRadius;

	/** pathfinding: this multiplier is used to compute a max node cost allowed to the open list (cost limit = CostLimitFactor*InitialHeuristicEstimate) */
	float CostLimitFactor;

	/** pathfinding: minimum cost limit clamping value (in cost units) used to allow large deviation in short paths */
	float MinimumCostLimit;

	/** custom user data: structure */
	FCustomMoveSharedPtr UserData;

	/** custom user data: flags */
	int32 UserFlags;
};

UENUM()
enum class EGenericAICheck : uint8
{
	Less,
	LessOrEqual,
	Equal,
	NotEqual,
	GreaterOrEqual,
	Greater,
	IsTrue,

	MAX UMETA(Hidden)
};

USTRUCT()
struct FIntervalCountdown
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Time)
	float Interval;

	float TimeLeft;

	explicit FIntervalCountdown(const float InInterval = -1.f) : Interval(InInterval), TimeLeft(0) {}

	void Set(const float InNewTimeLeft)
	{
		TimeLeft = InNewTimeLeft;
	}

	/**	@return True if time's up */
	bool Tick(const float TimeDelta)
	{
		TimeLeft -= TimeDelta;
		return TimeLeft <= 0;
	}

	/** resets the countdown back to Interval */
	void Reset()
	{
		TimeLeft = Interval;
	}

	/** Note that this function returns meaningful results only if Interval > 0*/
	float GetElapsedTime() const
	{
		ensure(Interval > 0);
		// using -TimeLeft because at this point TimeLeft is negative
		return (Interval - TimeLeft);
	}

	/** @return If Interval > 0 returns time accumulated since resetting. Oterwise returns FallbackValue */
	float GetElapsedTimeWithFallback(const float FallbackValue) const
	{
		return Interval > 0 ? GetElapsedTime() : FallbackValue;
	}
};
