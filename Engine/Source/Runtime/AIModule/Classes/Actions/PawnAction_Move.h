// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PawnAction_Move.generated.h"

UENUM()
namespace EPawnActionMoveMode
{
	enum Type : int
	{
		UsePathfinding,
		StraightLine,
	};
}
