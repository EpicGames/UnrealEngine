// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BehaviorTree/BTNode.h"
#include "BTAuxiliaryNode.generated.h"

struct FBTAuxiliaryMemory : public FBTInstancedNodeMemory
{
	float NextTickRemainingTime;
	float AccumulatedDeltaTime;
};

/** 
 * Auxiliary nodes are supporting nodes, that receive notification about execution flow and can be ticked
 *
 * Because some of them can be instanced for specific AI, following virtual functions are not marked as const:
 *  - OnBecomeRelevant
 *  - OnCeaseRelevant
 *  - TickNode
 *
 * If your node is not being instanced (default behavior), DO NOT change any properties of object within those functions!
 * Template nodes are shared across all behavior tree components using the same tree asset and must store
 * their runtime properties in provided NodeMemory block (allocation size determined by GetInstanceMemorySize() )
 *
 */

UCLASS(Abstract, MinimalAPI)
class UBTAuxiliaryNode : public UBTNode
{
	GENERATED_UCLASS_BODY()

	/** wrapper for node instancing: OnBecomeRelevant */
	AIMODULE_API void WrappedOnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const;

	/** wrapper for node instancing: OnCeaseRelevant */
	AIMODULE_API void WrappedOnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const;

	/** wrapper for node instancing: TickNode
	  * @param OwnerComp	The behavior tree owner of this node
	  * @param NodeMemory	The instance memory of the current node
	  * @param DeltaSeconds		DeltaTime since last call
	  * @param NextNeededDeltaTime		In out parameter, if this node needs a smaller DeltaTime it is the node's responsibility to change it
	  * @returns	True if it actually done some processing or false if it was skipped because of not ticking or in between time interval */
	AIMODULE_API bool WrappedTickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds, float& NextNeededDeltaTime) const;

	AIMODULE_API virtual void DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const override;
	AIMODULE_API virtual uint16 GetSpecialMemorySize() const override;

	/** fill in data about tree structure */
	AIMODULE_API void InitializeParentLink(uint8 InChildIndex);

	/** @return parent task node */
	AIMODULE_API const UBTNode* GetMyNode() const;

	/** @return index of child in parent's array or MAX_uint8 */
	uint8 GetChildIndex() const;

	/** Get The next needed deltatime for this node
	  * @param OwnerComp	The behavior tree owner of this node
	  * @param NodeMemory	The instance memory of the current node
	  * @return The next needed DeltaTime */
	AIMODULE_API float GetNextNeededDeltaTime(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const;

protected:

	/** if set, OnBecomeRelevant will be used */
	uint8 bNotifyBecomeRelevant:1;

	/** if set, OnCeaseRelevant will be used */
	uint8 bNotifyCeaseRelevant:1;

	/** if set, OnTick will be used */
	uint8 bNotifyTick : 1;

	/** if set, conditional tick will use remaining time from node's memory */
	uint8 bTickIntervals : 1;

	/** child index in parent node */
	uint8 ChildIndex;

	/** called when auxiliary node becomes active
	 * this function should be considered as const (don't modify state of object) if node is not instanced!  
	 * bNotifyBecomeRelevant must be set to true for this function to be called 
	 * Calling INIT_AUXILIARY_NODE_NOTIFY_FLAGS in the constructor of the node will set this flag automatically */
	AIMODULE_API virtual void OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory);

	/** called when auxiliary node becomes inactive
	 * this function should be considered as const (don't modify state of object) if node is not instanced!  
	 * bNotifyCeaseRelevant must be set to true for this function to be called 
	 * Calling INIT_AUXILIARY_NODE_NOTIFY_FLAGS in the constructor of the node will set this flag automatically */
	AIMODULE_API virtual void OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory);

	/** tick function
	 * this function should be considered as const (don't modify state of object) if node is not instanced!   
	 * bNotifyTick must be set to true for this function to be called 
	 * Calling INIT_AUXILIARY_NODE_NOTIFY_FLAGS in the constructor of the node will set this flag automatically */
	AIMODULE_API virtual void TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds);

	/** sets next tick time */
	AIMODULE_API void SetNextTickTime(uint8* NodeMemory, float RemainingTime) const;

	/** gets remaining time for next tick */
	AIMODULE_API float GetNextTickRemainingTime(uint8* NodeMemory) const;
	
	template<typename TickNode,	typename OnBecomeRelevant, typename OnCeaseRelevant>
	void InitNotifyFlags(TickNode, OnBecomeRelevant, OnCeaseRelevant)
	{
		bNotifyTick = !std::is_same_v<decltype(&UBTAuxiliaryNode::TickNode), TickNode>;
		bNotifyBecomeRelevant = !std::is_same_v<decltype(&UBTAuxiliaryNode::OnBecomeRelevant), OnBecomeRelevant>;
		bNotifyCeaseRelevant = !std::is_same_v<decltype(&UBTAuxiliaryNode::OnCeaseRelevant), OnCeaseRelevant>;
	}
};

#define INIT_AUXILIARY_NODE_NOTIFY_FLAGS() \
	do { \
	using NodeType = TRemovePointer<decltype(this)>::Type; \
	InitNotifyFlags(&NodeType::TickNode, &NodeType::OnBecomeRelevant, &NodeType::OnCeaseRelevant); \
	} while (false)

inline uint8 UBTAuxiliaryNode::GetChildIndex() const
{
	return ChildIndex;
}
