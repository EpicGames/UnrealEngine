// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "BehaviorTree/BTNode.h"
#include "BTCompositeNode.generated.h"

class UBTCompositeNode;
class UBTDecorator;
class UBTService;
class UBTTaskNode;

DECLARE_DELEGATE_RetVal_ThreeParams(int32, FGetNextChildDelegate, FBehaviorTreeSearchData& /*search data*/, int32 /*last child index*/, EBTNodeResult::Type /*last result*/);

struct FBTCompositeMemory
{
	/** index of currently active child node */
	int8 CurrentChild;

	/** child override for next selection */
	int8 OverrideChild;
};

UENUM()
enum class EBTChildIndex : uint8
{
	FirstNode,
	TaskNode,
};

UENUM()
namespace EBTDecoratorLogic
{
	// keep in sync with DescribeLogicOp() in BTCompositeNode.cpp

	enum Type : int
	{
		Invalid,
		/** Test decorator conditions. */
		Test,
		/** logic op: AND */
		And,
		/** logic op: OR */
		Or,
		/** logic op: NOT */
		Not,
	};
}

USTRUCT(BlueprintType)
struct FBTDecoratorLogic
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TEnumAsByte<EBTDecoratorLogic::Type> Operation;

	UPROPERTY()
	uint16 Number;

	FBTDecoratorLogic() : Operation(EBTDecoratorLogic::Invalid), Number(0) {}
	FBTDecoratorLogic(uint8 InOperation, uint16 InNumber) : Operation(InOperation), Number(InNumber) {}
};

USTRUCT(BlueprintType)
struct FBTCompositeChild
{
	GENERATED_USTRUCT_BODY()

	/** child node */
	UPROPERTY(BlueprintReadOnly, Category = BehaviorTree)
	TObjectPtr<UBTCompositeNode> ChildComposite = nullptr;

	UPROPERTY(BlueprintReadOnly, Category = BehaviorTree)
	TObjectPtr<UBTTaskNode> ChildTask = nullptr;

	/** execution decorators */
	UPROPERTY(BlueprintReadOnly, Category = BehaviorTree)
	TArray<TObjectPtr<UBTDecorator>> Decorators;

	/** logic operations for decorators */
	UPROPERTY(BlueprintReadOnly, Category = BehaviorTree)
	TArray<FBTDecoratorLogic> DecoratorOps;
};

UCLASS(Abstract, MinimalAPI)
class UBTCompositeNode : public UBTNode
{
	GENERATED_UCLASS_BODY()

	/** child nodes */
	UPROPERTY(BlueprintReadOnly, Category = BehaviorTree)
	TArray<FBTCompositeChild> Children;

	/** service nodes */
	UPROPERTY(BlueprintReadOnly, Category = BehaviorTree)
	TArray<TObjectPtr<UBTService>> Services;

	AIMODULE_API ~UBTCompositeNode();

	/** fill in data about tree structure */
	AIMODULE_API void InitializeComposite(uint16 InLastExecutionIndex);

	/** find next child branch to execute */
	AIMODULE_API int32 FindChildToExecute(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type& LastResult) const;

	/** get index of child node (handle subtrees) */
	AIMODULE_API int32 GetChildIndex(FBehaviorTreeSearchData& SearchData, const UBTNode& ChildNode) const;
	/** get index of child node */
	AIMODULE_API int32 GetChildIndex(const UBTNode& ChildNode) const;

	/** called before passing search to child node */
	AIMODULE_API void OnChildActivation(FBehaviorTreeSearchData& SearchData, const UBTNode& ChildNode) const;
	AIMODULE_API void OnChildActivation(FBehaviorTreeSearchData& SearchData, int32 ChildIndex) const;

	/**
	 * Notification called after child has finished search
	 * @param SearchData for any new addition or removal of extra aux nodes/ descriptor
	 * @param ChildNode is the children being deactivated
	 * @param NodeResult the raison of the deactivation
	 * @param bRequestedFromValidInstance the new requested search start is within the current active instances
	 */
	AIMODULE_API void OnChildDeactivation(FBehaviorTreeSearchData& SearchData, const UBTNode& ChildNode, EBTNodeResult::Type& NodeResult, const bool bRequestedFromValidInstance) const;
	/**
	 * Notification called after child has finished search
	 * @param SearchData for any new addition or removal of extra aux nodes/ descriptor
	 * @param ChildIndex of the child node being deactivated
	 * @param NodeResult the raison of the deactivation
	 * @param bRequestedFromValidInstance the new request search start is within the current active instances
	 */
	AIMODULE_API void OnChildDeactivation(FBehaviorTreeSearchData& SearchData, int32 ChildIndex, EBTNodeResult::Type& NodeResult, const bool bRequestedFromValidInstance) const;

	/** called when start enters this node */
	AIMODULE_API void OnNodeActivation(FBehaviorTreeSearchData& SearchData) const;

	/** called when search leaves this node */
	AIMODULE_API void OnNodeDeactivation(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type& NodeResult) const;

	/** called when search needs to reactivate this node */
	AIMODULE_API void OnNodeRestart(FBehaviorTreeSearchData& SearchData) const;

	/** notify about task execution start */
	AIMODULE_API void ConditionalNotifyChildExecution(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, const UBTNode& ChildNode, EBTNodeResult::Type& NodeResult) const;

	/** size of instance memory */
	AIMODULE_API virtual uint16 GetInstanceMemorySize() const override;
	AIMODULE_API virtual void InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const override;
	AIMODULE_API virtual void CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const override;

	/** @return child node at given index */
	UBTNode* GetChildNode(int32 Index) const;

	/** @return children count */
	int32 GetChildrenNum() const;

	/** @return execution index of child node */
	AIMODULE_API uint16 GetChildExecutionIndex(int32 Index, EBTChildIndex ChildMode = EBTChildIndex::TaskNode) const;

	/** @return execution index of last node in child branches */
	uint16 GetLastExecutionIndex() const;

	/** set override for next child index */
	AIMODULE_API virtual void SetChildOverride(FBehaviorTreeSearchData& SearchData, int8 Index) const;

	/** gathers description of all runtime parameters */
	AIMODULE_API virtual void DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const override;

	/** check if child node can execute new subtree */
	AIMODULE_API virtual bool CanPushSubtree(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, int32 ChildIdx) const;

#if WITH_EDITOR
	/** @return allowed flow abort modes for decorators */
	AIMODULE_API virtual bool CanAbortLowerPriority() const;
	AIMODULE_API virtual bool CanAbortSelf() const;
#endif // WITH_EDITOR

	/** find branch containing specified node index */
	AIMODULE_API int32 GetMatchingChildIndex(int32 ActiveInstanceIdx, FBTNodeIndex& NodeIdx) const;

	/** get first execution index of given branch */
	AIMODULE_API uint16 GetBranchExecutionIndex(uint16 NodeInBranchIdx) const;

	/** is child execution allowed by decorators? */
	AIMODULE_API bool DoDecoratorsAllowExecution(UBehaviorTreeComponent& OwnerComp, const int32 InstanceIdx, const int32 ChildIdx) const;

	bool IsApplyingDecoratorScope() const;

	// Deprecated methods
	UE_DEPRECATED(5.0, "This function is deprecated. Please use RequestBranchDeactivation instead.")
	void OnChildDeactivation(FBehaviorTreeSearchData& SearchData, const UBTNode& ChildNode, EBTNodeResult::Type& NodeResult) const { OnChildDeactivation(SearchData, ChildNode, NodeResult, /*bRequestedFromValidInstance*/true); }
	UE_DEPRECATED(5.0, "This function is deprecated. Please use RequestBranchDeactivation instead.")
	void OnChildDeactivation(FBehaviorTreeSearchData& SearchData, int32 ChildIndex, EBTNodeResult::Type& NodeResult) const { OnChildDeactivation(SearchData, ChildIndex, NodeResult, /*bRequestedFromValidInstance*/true ); }

protected:

	/** if set, all decorators in branch below will be removed when execution flow leaves (decorators on this node are not affected) */
	UPROPERTY(EditAnywhere, Category = Composite)
	uint32 bApplyDecoratorScope : 1;

	/** if set, NotifyChildExecution will be called */
	uint32 bUseChildExecutionNotify : 1;

	/** if set, NotifyNodeActivation will be called */
	uint32 bUseNodeActivationNotify : 1;

	/** if set, NotifyNodeDeactivation will be called */
	uint32 bUseNodeDeactivationNotify : 1;

	/** if set, CanNotifyDecoratorsOnActivation will be called */
	uint32 bUseDecoratorsActivationCheck : 1;

	/** if set, CanNotifyDecoratorsOnDeactivation will be called */
	uint32 bUseDecoratorsDeactivationCheck : 1;

	/** if set, CanNotifyDecoratorsOnFailedActivation will be called */
	uint32 bUseDecoratorsFailedActivationCheck : 1;

	/** execution index of last node in child branches */
	uint16 LastExecutionIndex;

	/** called just after child execution, allows to modify result  
	 * bUseChildExecutionNotify must be set to true for this function to be called 
	 * Calling INIT_COMPOSITE_NODE_NOTIFY_FLAGS in the constructor of the node will set this flag automatically */
	AIMODULE_API virtual void NotifyChildExecution(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, int32 ChildIdx, EBTNodeResult::Type& NodeResult) const;

	/** called when start enters this node  
	 * bUseNodeActivationNotify must be set to true for this function to be called 
	 * Calling INIT_COMPOSITE_NODE_NOTIFY_FLAGS in the constructor of the node will set this flag automatically */
	AIMODULE_API virtual void NotifyNodeActivation(FBehaviorTreeSearchData& SearchData) const;

	/** called when start leaves this node 
	 * bUseNodeDeactivationNotify must be set to true for this function to be called  
	 * Calling INIT_COMPOSITE_NODE_NOTIFY_FLAGS in the constructor of the node will set this flag automatically */
	AIMODULE_API virtual void NotifyNodeDeactivation(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type& NodeResult) const;

	/** check if NotifyDecoratorsOnActivation is allowed, requires bUseDecoratorsActivationCheck flag 
	 * bUseDecoratorsActivationCheck must be set to true for this function to be called  
	 * Calling INIT_COMPOSITE_NODE_NOTIFY_FLAGS in the constructor of the node will set this flag automatically */
	AIMODULE_API virtual bool CanNotifyDecoratorsOnActivation(FBehaviorTreeSearchData& SearchData, int32 ChildIdx) const;

	/** check if NotifyDecoratorsOnDeactivation is allowed, requires bUseDecoratorsDeactivationCheck flag   
	* bUseDecoratorsDeactivationCheck must be set to true for this function to be called 
	 * Calling INIT_COMPOSITE_NODE_NOTIFY_FLAGS in the constructor of the node will set this flag automatically */
	AIMODULE_API virtual bool CanNotifyDecoratorsOnDeactivation(FBehaviorTreeSearchData& SearchData, int32 ChildIdx, EBTNodeResult::Type& NodeResult) const;

	/** check if NotifyDecoratorsOnFailedActivation is allowed, requires bUseDecoratorsActivationCheck flag  
	 * bUseDecoratorsFailedActivationCheck must be set to true for this function to be called 
	 * Calling INIT_COMPOSITE_NODE_NOTIFY_FLAGS in the constructor of the node will set this flag automatically */
	AIMODULE_API virtual bool CanNotifyDecoratorsOnFailedActivation(FBehaviorTreeSearchData& SearchData, int32 ChildIdx, EBTNodeResult::Type& NodeResult) const;

	/** runs through decorators on given child node and notify them about activation */
	AIMODULE_API void NotifyDecoratorsOnActivation(FBehaviorTreeSearchData& SearchData, int32 ChildIdx) const;

	/** runs through decorators on given child node and notify them about deactivation */
	AIMODULE_API void NotifyDecoratorsOnDeactivation(FBehaviorTreeSearchData& SearchData, int32 ChildIdx, EBTNodeResult::Type& NodeResult, const bool bIsInSameActiveInstance) const;

	/** runs through decorators on given child node and notify them about failed activation */
	AIMODULE_API void NotifyDecoratorsOnFailedActivation(FBehaviorTreeSearchData& SearchData, int32 ChildIdx, EBTNodeResult::Type& NodeResult) const;

	/** get next child to process and store it in CurrentChild */
	AIMODULE_API int32 GetNextChild(FBehaviorTreeSearchData& SearchData, int32 LastChildIdx, EBTNodeResult::Type LastResult) const;

	/** store delayed execution request */
	AIMODULE_API void RequestDelayedExecution(UBehaviorTreeComponent& OwnerComp, EBTNodeResult::Type LastResult) const;

protected:
	virtual int32 GetNextChildHandler(struct FBehaviorTreeSearchData& SearchData, int32 PrevChild, EBTNodeResult::Type LastResult) const { return BTSpecialChild::ReturnToParent; }

	template<typename NotifyChildExecution,	typename NotifyNodeActivation, typename NotifyNodeDeactivation,
			typename CanNotifyDecoratorsOnActivation, typename CanNotifyDecoratorsOnDeactivation, typename CanNotifyDecoratorsOnFailedActivation>
	void InitNotifyFlags(NotifyChildExecution, NotifyNodeActivation, NotifyNodeDeactivation,
						 CanNotifyDecoratorsOnActivation, CanNotifyDecoratorsOnDeactivation, CanNotifyDecoratorsOnFailedActivation)
	{
		bUseChildExecutionNotify = !std::is_same_v<decltype(&UBTCompositeNode::NotifyChildExecution), NotifyChildExecution>;
		bUseNodeActivationNotify = !std::is_same_v<decltype(&UBTCompositeNode::NotifyNodeActivation), NotifyNodeActivation>;
		bUseNodeDeactivationNotify = !std::is_same_v<decltype(&UBTCompositeNode::NotifyNodeDeactivation), NotifyNodeDeactivation>;
		bUseDecoratorsActivationCheck = !std::is_same_v<decltype(&UBTCompositeNode::CanNotifyDecoratorsOnActivation), CanNotifyDecoratorsOnActivation>;
		bUseDecoratorsDeactivationCheck = !std::is_same_v<decltype(&UBTCompositeNode::CanNotifyDecoratorsOnDeactivation), CanNotifyDecoratorsOnDeactivation>;
		bUseDecoratorsFailedActivationCheck = !std::is_same_v<decltype(&UBTCompositeNode::CanNotifyDecoratorsOnFailedActivation), CanNotifyDecoratorsOnFailedActivation>;
	}
};

#define INIT_COMPOSITE_NODE_NOTIFY_FLAGS() \
	do { \
	using NodeType = TRemovePointer<decltype(this)>::Type; \
	InitNotifyFlags(&NodeType::NotifyChildExecution,\
					&NodeType::NotifyNodeActivation,\
					&NodeType::NotifyNodeDeactivation, \
					&NodeType::CanNotifyDecoratorsOnActivation,\
					&NodeType::CanNotifyDecoratorsOnDeactivation,\
						&NodeType::CanNotifyDecoratorsOnFailedActivation); \
		} while (false)

//////////////////////////////////////////////////////////////////////////
// Inlines

inline UBTNode* UBTCompositeNode::GetChildNode(int32 Index) const
{
	return Children.IsValidIndex(Index) ?
		(Children[Index].ChildComposite ?
			static_cast<UBTNode*>(Children[Index].ChildComposite) :
			static_cast<UBTNode*>(Children[Index].ChildTask)) :
		nullptr;
}

inline int32 UBTCompositeNode::GetChildrenNum() const
{
	return Children.Num();
}

inline uint16 UBTCompositeNode::GetLastExecutionIndex() const
{
	return LastExecutionIndex;
}

inline bool UBTCompositeNode::IsApplyingDecoratorScope() const
{
	return bApplyDecoratorScope;
}
