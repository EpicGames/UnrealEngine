// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "BehaviorTree/BTAuxiliaryNode.h"
#include "BTDecorator.generated.h"

class FBehaviorDecoratorDetails;

enum class EBTDecoratorAbortRequest : uint8
{
	// request execution update when only result of condition changes and active branch of tree can potentially change too
	ConditionResultChanged,

	// request execution update every time as long as condition is still passing
	ConditionPassing,
};

/** 
 * Decorators are supporting nodes placed on parent-child connection, that receive notification about execution flow and can be ticked
 *
 * Because some of them can be instanced for specific AI, following virtual functions are not marked as const:
 *  - OnNodeActivation
 *  - OnNodeDeactivation
 *  - OnNodeProcessed
 *  - OnBecomeRelevant (from UBTAuxiliaryNode)
 *  - OnCeaseRelevant (from UBTAuxiliaryNode)
 *  - TickNode (from UBTAuxiliaryNode)
 *
 * If your node is not being instanced (default behavior), DO NOT change any properties of object within those functions!
 * Template nodes are shared across all behavior tree components using the same tree asset and must store
 * their runtime properties in provided NodeMemory block (allocation size determined by GetInstanceMemorySize() )
 *
 */

UCLASS(Abstract, MinimalAPI)
class UBTDecorator : public UBTAuxiliaryNode
{
	GENERATED_UCLASS_BODY()

	/** wrapper for node instancing: CalculateRawConditionValue */
	AIMODULE_API bool WrappedCanExecute(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const;

	/** wrapper for node instancing: OnNodeActivation  */
	AIMODULE_API void WrappedOnNodeActivation(FBehaviorTreeSearchData& SearchData) const;
	
	/** wrapper for node instancing: OnNodeDeactivation */
	AIMODULE_API void WrappedOnNodeDeactivation(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type NodeResult) const;

	/** wrapper for node instancing: OnNodeProcessed */
	AIMODULE_API void WrappedOnNodeProcessed(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type& NodeResult) const;

	/** @return flow controller's abort mode */
	EBTFlowAbortMode::Type GetFlowAbortMode() const;

	/** @return true if condition should be inversed */
	bool IsInversed() const;

	AIMODULE_API virtual FString GetStaticDescription() const override;

	/** modify current flow abort mode, so it can be used with parent composite */
	AIMODULE_API void UpdateFlowAbortMode();

	/** @return true if current abort mode can be used with parent composite */
	AIMODULE_API bool IsFlowAbortModeValid() const;

protected:

	/** if set, FlowAbortMode can be set to None */
	uint32 bAllowAbortNone : 1;

	/** if set, FlowAbortMode can be set to LowerPriority and Both */
	uint32 bAllowAbortLowerPri : 1;

	/** if set, FlowAbortMode can be set to Self and Both */
	uint32 bAllowAbortChildNodes : 1;

	/** if set, OnNodeActivation will be used */
	uint32 bNotifyActivation : 1;

	/** if set, OnNodeDeactivation will be used */
	uint32 bNotifyDeactivation : 1;

	/** if set, OnNodeProcessed will be used */
	uint32 bNotifyProcessed : 1;

	/** if set, static description will include default description of inversed condition */
	uint32 bShowInverseConditionDesc : 1;

private:
	/** if set, condition check result will be inversed */
	UPROPERTY(Category = Condition, EditAnywhere)
	uint32 bInverseCondition : 1;

protected:
	/** flow controller settings */
	UPROPERTY(Category=FlowControl, EditAnywhere)
	TEnumAsByte<EBTFlowAbortMode::Type> FlowAbortMode;

	AIMODULE_API void SetIsInversed(bool bShouldBeInversed);

	/** called when underlying node is activated
	 * this function should be considered as const (don't modify state of object) if node is not instanced! 
	 * bNotifyActivation must be set to true for this function to be called
	 * Calling INIT_DECORATOR_NODE_NOTIFY_FLAGS in the constructor of the decorator will set this flag automatically */
	AIMODULE_API virtual void OnNodeActivation(FBehaviorTreeSearchData& SearchData);

	/** called when underlying node has finished
	 * this function should be considered as const (don't modify state of object) if node is not instanced! 
	 * bNotifyDeactivation must be set to true for this function to be called
	 * Calling INIT_DECORATOR_NODE_NOTIFY_FLAGS in the constructor of the decorator will set this flag automatically */
	AIMODULE_API virtual void OnNodeDeactivation(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type NodeResult);

	/** called when underlying node was processed (deactivated or failed to activate)
	 * this function should be considered as const (don't modify state of object) if node is not instanced! 
	 * bNotifyProcessed must be set to true for this function to be called 
	 * Calling INIT_DECORATOR_NODE_NOTIFY_FLAGS in the constructor of the decorator will set this flag automatically */
	AIMODULE_API virtual void OnNodeProcessed(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type& NodeResult);

	/** calculates raw, core value of decorator's condition. Should not include calling IsInversed */
	AIMODULE_API virtual bool CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const;

	/** more "flow aware" version of calling RequestExecution(this) on owning behavior tree component
	 *  should be used in external events that may change result of CalculateRawConditionValue */
	AIMODULE_API void ConditionalFlowAbort(UBehaviorTreeComponent& OwnerComp, EBTDecoratorAbortRequest RequestMode) const;

	friend FBehaviorDecoratorDetails;

	template<typename TickNode,	typename OnBecomeRelevant, typename OnCeaseRelevant,
		typename OnNodeActivation, typename OnNodeDeactivation, typename OnNodeProcessed>
	void InitNotifyFlags(TickNode, OnBecomeRelevant, OnCeaseRelevant,
					 OnNodeActivation, OnNodeDeactivation, OnNodeProcessed)
	{
		bNotifyTick = !std::is_same_v<decltype(&UBTDecorator::TickNode), TickNode>;
		bNotifyBecomeRelevant = !std::is_same_v<decltype(&UBTDecorator::OnBecomeRelevant), OnBecomeRelevant>;
		bNotifyCeaseRelevant = !std::is_same_v<decltype(&UBTDecorator::OnCeaseRelevant), OnCeaseRelevant>;
		bNotifyActivation = !std::is_same_v<decltype(&UBTDecorator::OnNodeActivation), OnNodeActivation>;
		bNotifyDeactivation = !std::is_same_v<decltype(&UBTDecorator::OnNodeDeactivation), OnNodeDeactivation>;
		bNotifyProcessed = !std::is_same_v<decltype(&UBTDecorator::OnNodeProcessed), OnNodeProcessed>;
	}
};

#define INIT_DECORATOR_NODE_NOTIFY_FLAGS() \
	do { \
	using NodeType = TRemovePointer<decltype(this)>::Type; \
	InitNotifyFlags(&NodeType::TickNode, \
					&NodeType::OnBecomeRelevant, \
					&NodeType::OnCeaseRelevant, \
					&NodeType::OnNodeActivation, \
					&NodeType::OnNodeDeactivation, \
					&NodeType::OnNodeProcessed); \
	} while (false)

//////////////////////////////////////////////////////////////////////////
// Inlines

inline EBTFlowAbortMode::Type UBTDecorator::GetFlowAbortMode() const
{
	return FlowAbortMode;
}

inline bool UBTDecorator::IsInversed() const
{
	return bInverseCondition;
}
