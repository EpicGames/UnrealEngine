// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/BehaviorTreeTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "BTFunctionLibrary.generated.h"

class AActor;
class UBehaviorTreeComponent;
class UBlackboardComponent;
class UBTNode;

namespace FBTNodeBPImplementationHelper
{
	static const int32 NoImplementation = 0;
	static const int32 Generic = 1 << 0;
	static const int32 AISpecific = 1 << 1;
	static const int32 All = Generic | AISpecific;

	/** checks if given object implements GenericEventName and/or AIEventName BP events, and returns an result as flags set on return integer 
	 *	@return flags set in returned integer indicate kinds of events implemented by given object */
	AIMODULE_API int32 CheckEventImplementationVersion(FName GenericEventName, FName AIEventName, const UObject& Object, const UClass& StopAtClass);
}

UCLASS(meta=(RestrictedToClasses="BTNode"), MinimalAPI)
class UBTFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintPure, Category="AI|BehaviorTree", Meta=(HidePin="NodeOwner", DefaultToSelf="NodeOwner"))
	static AIMODULE_API UBlackboardComponent* GetOwnersBlackboard(UBTNode* NodeOwner);

	UFUNCTION(BlueprintPure, Category = "AI|BehaviorTree", Meta=(HidePin="NodeOwner", DefaultToSelf="NodeOwner"))
	static AIMODULE_API UBehaviorTreeComponent* GetOwnerComponent(UBTNode* NodeOwner);

	UFUNCTION(BlueprintPure, Category="AI|BehaviorTree", Meta=(HidePin="NodeOwner", DefaultToSelf="NodeOwner"))
	static AIMODULE_API UObject* GetBlackboardValueAsObject(UBTNode* NodeOwner, const FBlackboardKeySelector& Key);

	UFUNCTION(BlueprintPure, Category="AI|BehaviorTree", Meta=(HidePin="NodeOwner", DefaultToSelf="NodeOwner"))
	static AIMODULE_API AActor* GetBlackboardValueAsActor(UBTNode* NodeOwner, const FBlackboardKeySelector& Key);

	UFUNCTION(BlueprintPure, Category="AI|BehaviorTree", Meta=(HidePin="NodeOwner", DefaultToSelf="NodeOwner"))
	static AIMODULE_API UClass* GetBlackboardValueAsClass(UBTNode* NodeOwner, const FBlackboardKeySelector& Key);

	UFUNCTION(BlueprintPure, Category="AI|BehaviorTree", Meta=(HidePin="NodeOwner", DefaultToSelf="NodeOwner"))
	static AIMODULE_API uint8 GetBlackboardValueAsEnum(UBTNode* NodeOwner, const FBlackboardKeySelector& Key);

	UFUNCTION(BlueprintPure, Category="AI|BehaviorTree", Meta=(HidePin="NodeOwner", DefaultToSelf="NodeOwner"))
	static AIMODULE_API int32 GetBlackboardValueAsInt(UBTNode* NodeOwner, const FBlackboardKeySelector& Key);

	UFUNCTION(BlueprintPure, Category="AI|BehaviorTree", Meta=(HidePin="NodeOwner", DefaultToSelf="NodeOwner"))
	static AIMODULE_API float GetBlackboardValueAsFloat(UBTNode* NodeOwner, const FBlackboardKeySelector& Key);

	UFUNCTION(BlueprintPure, Category="AI|BehaviorTree", Meta=(HidePin="NodeOwner", DefaultToSelf="NodeOwner"))
	static AIMODULE_API bool GetBlackboardValueAsBool(UBTNode* NodeOwner, const FBlackboardKeySelector& Key);

	UFUNCTION(BlueprintPure, Category="AI|BehaviorTree", Meta=(HidePin="NodeOwner", DefaultToSelf="NodeOwner"))
	static AIMODULE_API FString GetBlackboardValueAsString(UBTNode* NodeOwner, const FBlackboardKeySelector& Key);

	UFUNCTION(BlueprintPure, Category="AI|BehaviorTree", Meta=(HidePin="NodeOwner", DefaultToSelf="NodeOwner"))
	static AIMODULE_API FName GetBlackboardValueAsName(UBTNode* NodeOwner, const FBlackboardKeySelector& Key);

	UFUNCTION(BlueprintPure, Category="AI|BehaviorTree", Meta=(HidePin="NodeOwner", DefaultToSelf="NodeOwner"))
	static AIMODULE_API FVector GetBlackboardValueAsVector(UBTNode* NodeOwner, const FBlackboardKeySelector& Key);

	UFUNCTION(BlueprintPure, Category ="AI|BehaviorTree", Meta=(HidePin="NodeOwner", DefaultToSelf="NodeOwner"))
	static AIMODULE_API FRotator GetBlackboardValueAsRotator(UBTNode* NodeOwner, const FBlackboardKeySelector& Key);

	UFUNCTION(BlueprintCallable, Category="AI|BehaviorTree", Meta=(HidePin="NodeOwner", DefaultToSelf="NodeOwner"))
	static AIMODULE_API void SetBlackboardValueAsObject(UBTNode* NodeOwner, const FBlackboardKeySelector& Key, UObject* Value);

	UFUNCTION(BlueprintCallable, Category="AI|BehaviorTree", Meta=(HidePin="NodeOwner", DefaultToSelf="NodeOwner"))
	static AIMODULE_API void SetBlackboardValueAsClass(UBTNode* NodeOwner, const FBlackboardKeySelector& Key, UClass* Value);

	UFUNCTION(BlueprintCallable, Category="AI|BehaviorTree", Meta=(HidePin="NodeOwner", DefaultToSelf="NodeOwner"))
	static AIMODULE_API void SetBlackboardValueAsEnum(UBTNode* NodeOwner, const FBlackboardKeySelector& Key, uint8 Value);

	UFUNCTION(BlueprintCallable, Category="AI|BehaviorTree", Meta=(HidePin="NodeOwner", DefaultToSelf="NodeOwner"))
	static AIMODULE_API void SetBlackboardValueAsInt(UBTNode* NodeOwner, const FBlackboardKeySelector& Key, int32 Value);

	UFUNCTION(BlueprintCallable, Category="AI|BehaviorTree", Meta=(HidePin="NodeOwner", DefaultToSelf="NodeOwner"))
	static AIMODULE_API void SetBlackboardValueAsFloat(UBTNode* NodeOwner, const FBlackboardKeySelector& Key, float Value);

	UFUNCTION(BlueprintCallable, Category="AI|BehaviorTree", Meta=(HidePin="NodeOwner", DefaultToSelf="NodeOwner"))
	static AIMODULE_API void SetBlackboardValueAsBool(UBTNode* NodeOwner, const FBlackboardKeySelector& Key, bool Value);

	UFUNCTION(BlueprintCallable, Category="AI|BehaviorTree", Meta=(HidePin="NodeOwner", DefaultToSelf="NodeOwner"))
	static AIMODULE_API void SetBlackboardValueAsString(UBTNode* NodeOwner, const FBlackboardKeySelector& Key, FString Value);

	UFUNCTION(BlueprintCallable, Category="AI|BehaviorTree", Meta=(HidePin="NodeOwner", DefaultToSelf="NodeOwner"))
	static AIMODULE_API void SetBlackboardValueAsName(UBTNode* NodeOwner, const FBlackboardKeySelector& Key, FName Value);

	UFUNCTION(BlueprintCallable, Category="AI|BehaviorTree", Meta=(HidePin="NodeOwner", DefaultToSelf="NodeOwner"))
	static AIMODULE_API void SetBlackboardValueAsVector(UBTNode* NodeOwner, const FBlackboardKeySelector& Key, FVector Value);

	/** (DEPRECATED) Use ClearBlackboardValue instead */
	UFUNCTION(BlueprintCallable, Category="AI|BehaviorTree", Meta=(HidePin="NodeOwner", DefaultToSelf="NodeOwner", DeprecatedFunction, DeprecationMessage="Use ClearBlackboardValue instead."))
	static AIMODULE_API void ClearBlackboardValueAsVector(UBTNode* NodeOwner, const FBlackboardKeySelector& Key);

	UFUNCTION(BlueprintCallable, Category="AI|BehaviorTree", Meta = (HidePin = "NodeOwner", DefaultToSelf = "NodeOwner"))
	static AIMODULE_API void SetBlackboardValueAsRotator(UBTNode* NodeOwner, const FBlackboardKeySelector& Key, FRotator Value);

	/** Resets indicated value to "not set" value, based on values type */
	UFUNCTION(BlueprintCallable, Category="AI|BehaviorTree", Meta = (HidePin = "NodeOwner", DefaultToSelf = "NodeOwner"))
	static AIMODULE_API void ClearBlackboardValue(UBTNode* NodeOwner, const FBlackboardKeySelector& Key);

	/** Initialize variables marked as "instance memory" and set owning actor for blackboard operations */
	UFUNCTION(BlueprintCallable, Category="AI|BehaviorTree", Meta=(HidePin="NodeOwner", DefaultToSelf="NodeOwner", DeprecatedFunction, DeprecationMessage="No longer needed"))
	static AIMODULE_API void StartUsingExternalEvent(UBTNode* NodeOwner, AActor* OwningActor);

	/** Save variables marked as "instance memory" and clear owning actor */
	UFUNCTION(BlueprintCallable, Category="AI|BehaviorTree", Meta=(HidePin="NodeOwner", DefaultToSelf="NodeOwner", DeprecatedFunction, DeprecationMessage="No longer needed"))
	static AIMODULE_API void StopUsingExternalEvent(UBTNode* NodeOwner);
};
