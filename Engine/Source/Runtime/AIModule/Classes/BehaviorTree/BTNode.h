// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "BehaviorTree/BehaviorTreeTypes.h"
#include "BehaviorTree/BehaviorTreeComponent.h"
#include "BehaviorTree/BlackboardAssetProvider.h"
#include "GameplayTaskOwnerInterface.h"
#include "Tasks/AITask.h"
#include <type_traits>
#include "BTNode.generated.h"

class AActor;
class ISlateStyle;
class UBehaviorTree;
class UBlackboardData;
class UBTCompositeNode;
class UGameplayTasksComponent;

AIMODULE_API DECLARE_LOG_CATEGORY_EXTERN(LogBehaviorTree, Display, All);

class AAIController;
class UWorld;
class UBehaviorTree;
class UBehaviorTreeComponent;
class UBTCompositeNode;
class UBlackboardData;
struct FBehaviorTreeSearchData;

struct FBTInstancedNodeMemory
{
	int32 NodeIdx;
};

struct FBehaviorTreeNodeDebugContext
{
	/** Behavior Tree component of the behavior tree being debugged. */
	UBehaviorTreeComponent& OwnerComponent;
	/** Combined string of the values returned by DescribeRuntimeValues of the node at the current debug execution step. */
	FString RuntimeDescription;
};

UCLASS(Abstract, config=Game, MinimalAPI)
class UBTNode : public UObject, public IGameplayTaskOwnerInterface
{
	GENERATED_UCLASS_BODY()

	AIMODULE_API virtual UWorld* GetWorld() const override;

	/** fill in data about tree structure */
	AIMODULE_API void InitializeNode(UBTCompositeNode* InParentNode, uint16 InExecutionIndex, uint16 InMemoryOffset, uint8 InTreeDepth);

	/** initialize any asset related data */
	AIMODULE_API virtual void InitializeFromAsset(UBehaviorTree& Asset);
	
	/** initialize memory block. InitializeNodeMemory template function is provided to help initialize the memory. */
	AIMODULE_API virtual void InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const;

	/** cleanup memory block. CleanupNodeMemory template function is provided to help cleanup the memory. */
	AIMODULE_API virtual void CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const;

	/** gathers description of all runtime parameters */
	AIMODULE_API virtual void DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const;

	/** size of instance memory */
	AIMODULE_API virtual uint16 GetInstanceMemorySize() const;

	/** called when node instance is added to tree */
	AIMODULE_API virtual void OnInstanceCreated(UBehaviorTreeComponent& OwnerComp);

	/** called when node instance is removed from tree */
	AIMODULE_API virtual void OnInstanceDestroyed(UBehaviorTreeComponent& OwnerComp);

	/** called on creating subtree to set up memory and instancing */
	AIMODULE_API void InitializeInSubtree(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, int32& NextInstancedIndex, EBTMemoryInit::Type InitType) const;

	/** called on removing subtree to cleanup memory */
	AIMODULE_API void CleanupInSubtree(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const;

	/** size of special, hidden memory block for internal mechanics */
	AIMODULE_API virtual uint16 GetSpecialMemorySize() const;

#if USE_BEHAVIORTREE_DEBUGGER
	/** fill in data about execution order */
	AIMODULE_API void InitializeExecutionOrder(UBTNode* NextNode);

	/** @return next node in execution order */
	UBTNode* GetNextNode() const;
#endif

	template<typename T>
	T* InitializeNodeMemory(uint8* NodeMemory, EBTMemoryInit::Type InitType) const;

	template <typename T>
	void CleanupNodeMemory(uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const;

	template<typename T>
	T* GetNodeMemory(FBehaviorTreeSearchData& SearchData) const;

	template<typename T>
	const T* GetNodeMemory(const FBehaviorTreeSearchData& SearchData) const;

	template<typename T>
	T* GetNodeMemory(FBehaviorTreeInstance& BTInstance) const;

	template<typename T>
	const T* GetNodeMemory(const FBehaviorTreeInstance& BTInstance) const;

	template<typename T>
	T* CastInstanceNodeMemory(uint8* NodeMemory) const;

	/** get special memory block used for hidden shared data (e.g. node instancing) */
	template<typename T>
	T* GetSpecialNodeMemory(uint8* NodeMemory) const;

	/** @return parent node */
	UBTCompositeNode* GetParentNode() const;

	/** @return name of node */
	AIMODULE_API FString GetNodeName() const;

	/** @return execution index */
	uint16 GetExecutionIndex() const;

	/** @return memory offset */
	uint16 GetMemoryOffset() const;

	/** @return depth in tree */
	uint8 GetTreeDepth() const;

	/** sets bIsInjected flag, do NOT call this function unless you really know what you are doing! */
	void MarkInjectedNode();

	/** @return true if node was injected by subtree */
	bool IsInjected() const;

	/** sets bCreateNodeInstance flag, do NOT call this function on already pushed tree instance! */
	void ForceInstancing(bool bEnable);

	/** @return true if node wants to be instanced */
	bool HasInstance() const;

	/** @return true if this object is instanced node */
	bool IsInstanced() const;

	/** @return tree asset */
	UBehaviorTree* GetTreeAsset() const;

	/** @return blackboard asset */
	AIMODULE_API UBlackboardData* GetBlackboardAsset() const;

	/** @return node instance if bCreateNodeInstance was set */
	AIMODULE_API UBTNode* GetNodeInstance(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const;
	AIMODULE_API UBTNode* GetNodeInstance(FBehaviorTreeSearchData& SearchData) const;

	/** @return string containing description of this node instance with all relevant runtime values */
	AIMODULE_API FString GetRuntimeDescription(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity) const;

	/** @return string containing description of this node with all setup values */
	AIMODULE_API virtual FString GetStaticDescription() const;

#if WITH_EDITOR
	/** Get the style set from which GetNodeIconName is queried */
	AIMODULE_API virtual const ISlateStyle& GetNodeIconStyleSet() const;

	/** Get the name of the icon used to display this node in the editor */
	AIMODULE_API virtual FName GetNodeIconName() const;

	/** Get whether this node is using a blueprint for its logic */
	AIMODULE_API virtual bool UsesBlueprint() const;

	/** Validates this node and returns the resulting error message. Empty means no error. */
	AIMODULE_API virtual FString GetErrorMessage() const;

	/** Called after creating new node in behavior tree editor, use for versioning */
	virtual void OnNodeCreated() {}

	AIMODULE_API void PreSave(FObjectPreSaveContext SaveContext) override;

	/** @return the external asset associated with this node that should be open on double click. If set, DebugContext contains the info of the node in the current debug step. */
	AIMODULE_API virtual const UObject* GetAssociatedAsset(TOptional<FBehaviorTreeNodeDebugContext> DebugContext) const;
#endif

	/** Gets called only for instanced nodes(bCreateNodeInstance == true). In practive overridden by BP-implemented BT nodes */
	virtual void SetOwner(AActor* ActorOwner) {}

	// BEGIN IGameplayTaskOwnerInterface
	AIMODULE_API virtual UGameplayTasksComponent* GetGameplayTasksComponent(const UGameplayTask& Task) const override;
	AIMODULE_API virtual AActor* GetGameplayTaskOwner(const UGameplayTask* Task) const override;
	AIMODULE_API virtual AActor* GetGameplayTaskAvatar(const UGameplayTask* Task) const override;
	AIMODULE_API virtual uint8 GetGameplayTaskDefaultPriority() const override;
	AIMODULE_API virtual void OnGameplayTaskInitialized(UGameplayTask& Task) override;
	// END IGameplayTaskOwnerInterface

	AIMODULE_API UBehaviorTreeComponent* GetBTComponentForTask(UGameplayTask& Task) const;
	
	template <class T>
	T* NewBTAITask(UBehaviorTreeComponent& BTComponent)
	{
		check(BTComponent.GetAIOwner());
		bOwnsGameplayTasks = true;
		return UAITask::NewAITask<T>(*BTComponent.GetAIOwner(), *this, TEXT("Behavior"));
	}
	
	template <class T>
	T* NewBTAITask(const UClass& Class, UBehaviorTreeComponent& BTComponent)
	{
		check(BTComponent.GetAIOwner());
		bOwnsGameplayTasks = true;
		return UAITask::NewAITask<T>(Class, *BTComponent.GetAIOwner(), *this, TEXT("Behavior"));
	}

	/** node name */
	UPROPERTY(Category=Description, EditAnywhere, BlueprintReadOnly)
	FString NodeName;
	
private:

	/** source asset */
	UPROPERTY()
	TObjectPtr<UBehaviorTree> TreeAsset;

	/** parent node */
	UPROPERTY()
	TObjectPtr<UBTCompositeNode> ParentNode;

#if USE_BEHAVIORTREE_DEBUGGER
	/** next node in execution order */
	TWeakObjectPtr<UBTNode> NextExecutionNode;
#endif

	/** depth first index (execution order) */
	UPROPERTY(transient)
	uint16 ExecutionIndex;

	/** instance memory offset */
	UPROPERTY(transient)
	uint16 MemoryOffset;

	/** depth in tree */
	UPROPERTY(transient)
	uint8 TreeDepth;

	/** set automatically for node instances. Should never be set manually */
	UPROPERTY(transient)
	uint8 bIsInstanced : 1;

	/** if set, node is injected by subtree. Should never be set manually */
	UPROPERTY(transient)
	uint8 bIsInjected : 1;

protected:

	/** if set, node will be instanced instead of using memory block and template shared with all other BT components */
	UPROPERTY(transient)
	uint8 bCreateNodeInstance : 1;

	/** set to true if task owns any GameplayTasks. Note this requires tasks to be created via NewBTAITask
	 *	Otherwise specific BT task node class is responsible for ending the gameplay tasks on node finish */
	UPROPERTY(transient)
	uint8 bOwnsGameplayTasks : 1;
};

//////////////////////////////////////////////////////////////////////////
// Inlines

inline UBehaviorTree* UBTNode::GetTreeAsset() const
{
	return TreeAsset;
}

inline UBTCompositeNode* UBTNode::GetParentNode() const
{
	return ParentNode;
}

#if USE_BEHAVIORTREE_DEBUGGER
inline UBTNode* UBTNode::GetNextNode() const
{
	return NextExecutionNode.Get();
}
#endif

inline uint16 UBTNode::GetExecutionIndex() const
{
	return ExecutionIndex;
}

inline uint16 UBTNode::GetMemoryOffset() const
{
	return MemoryOffset;
}

inline uint8 UBTNode::GetTreeDepth() const
{
	return TreeDepth;
}

inline void UBTNode::MarkInjectedNode()
{
	bIsInjected = true;
}

inline bool UBTNode::IsInjected() const
{
	return bIsInjected;
}

inline void UBTNode::ForceInstancing(bool bEnable)
{
	// allow only in not initialized trees, side effect: root node always blocked
	check(ParentNode == NULL);

	bCreateNodeInstance = bEnable;
}

inline bool UBTNode::HasInstance() const
{
	return bCreateNodeInstance;
}

inline bool UBTNode::IsInstanced() const
{
	return bIsInstanced;
}

template<typename T>
T* UBTNode::InitializeNodeMemory(uint8* NodeMemory, EBTMemoryInit::Type InitType) const
{
	if (InitType == EBTMemoryInit::Initialize)
	{
		new(NodeMemory) T();
	}
	return CastInstanceNodeMemory<T>(NodeMemory);
}

template <typename T>
void UBTNode::CleanupNodeMemory(uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const
{
	if constexpr (!std::is_trivially_destructible_v<T>)
	{
		if (CleanupType == EBTMemoryClear::Destroy)
		{
			CastInstanceNodeMemory<T>(NodeMemory)->~T();
		}
	}
}

template<typename T>
T* UBTNode::GetNodeMemory(FBehaviorTreeSearchData& SearchData) const
{
	return GetNodeMemory<T>(SearchData.OwnerComp.InstanceStack[SearchData.OwnerComp.GetActiveInstanceIdx()]);
}

template<typename T>
const T* UBTNode::GetNodeMemory(const FBehaviorTreeSearchData& SearchData) const
{
	return GetNodeMemory<T>(SearchData.OwnerComp.InstanceStack[SearchData.OwnerComp.GetActiveInstanceIdx()]);
}

template<typename T>
T* UBTNode::GetNodeMemory(FBehaviorTreeInstance& BTInstance) const
{
	return (T*)(BTInstance.GetInstanceMemory().GetData() + MemoryOffset);
}

template<typename T>
const T* UBTNode::GetNodeMemory(const FBehaviorTreeInstance& BTInstance) const
{
	return (const T*)(BTInstance.GetInstanceMemory().GetData() + MemoryOffset);
}

template<typename T>
T* UBTNode::CastInstanceNodeMemory(uint8* NodeMemory) const
{
	// using '<=' rather than '==' to allow child classes to extend parent's
	// memory class as well (which would make GetInstanceMemorySize return 
	// a value equal or greater to sizeof(T)).
	checkf(sizeof(T) <= GetInstanceMemorySize(), TEXT("Requesting type of %zu bytes but GetInstanceMemorySize returns %u. Make sure GetInstanceMemorySize is implemented properly in %s class hierarchy."), sizeof(T), GetInstanceMemorySize(), *GetFName().ToString());
	return reinterpret_cast<T*>(NodeMemory);
}

template<typename T>
T* UBTNode::GetSpecialNodeMemory(uint8* NodeMemory) const
{
	const int32 SpecialMemorySize = GetSpecialMemorySize();
	return SpecialMemorySize ? (T*)(NodeMemory - ((SpecialMemorySize + 3) & ~3)) : nullptr;
}
