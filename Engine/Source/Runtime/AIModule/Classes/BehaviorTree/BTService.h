// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BehaviorTree/BTAuxiliaryNode.h"
#include "BTService.generated.h"

/** 
 *	Behavior Tree service nodes is designed to perform "background" tasks that update AI's knowledge.
 *
 *  Services are being executed when underlying branch of behavior tree becomes active,
 *  but unlike tasks they don't return any results and can't directly affect execution flow.
 *
 *  Usually they perform periodical checks (see TickNode) and often store results in blackboard.
 *  If any decorator node below requires results of check beforehand, use OnSearchStart function.
 *   Keep in mind that any checks performed there have to be instantaneous!
 *	
 *  Other typical use case is creating a marker when specific branch is being executed
 *  (see OnBecomeRelevant, OnCeaseRelevant), by setting a flag in blackboard.
 *
 *  Because some of them can be instanced for specific AI, following virtual functions are not marked as const:
 *   - OnBecomeRelevant (from UBTAuxiliaryNode)
 *   - OnCeaseRelevant (from UBTAuxiliaryNode)
 *   - TickNode (from UBTAuxiliaryNode)
 *   - OnSearchStart
 *
 *  If your node is not being instanced (default behavior), DO NOT change any properties of object within those functions!
 *  Template nodes are shared across all behavior tree components using the same tree asset and must store
 *  their runtime properties in provided NodeMemory block (allocation size determined by GetInstanceMemorySize() )
 */
 
UCLASS(Abstract, MinimalAPI)
class UBTService : public UBTAuxiliaryNode
{
	GENERATED_UCLASS_BODY()

	AIMODULE_API virtual FString GetStaticDescription() const override;

	AIMODULE_API void NotifyParentActivation(FBehaviorTreeSearchData& SearchData);

protected:

	// Gets the description of our tick interval
	AIMODULE_API FString GetStaticTickIntervalDescription() const;

	// Gets the description for our service
	AIMODULE_API virtual FString GetStaticServiceDescription() const;

	/** defines time span between subsequent ticks of the service */
	UPROPERTY(Category=Service, EditAnywhere, meta=(ClampMin="0.001"))
	float Interval;

	/** adds random range to service's Interval */
	UPROPERTY(Category=Service, EditAnywhere, meta=(ClampMin="0.0"))
	float RandomDeviation;

	/** call Tick event when task search enters this node (SearchStart will be called as well) */
	UPROPERTY(Category = Service, EditAnywhere, AdvancedDisplay, meta=(EditCondition = bCanTickOnSearchStartBeExposed, EditConditionHides))
	uint32 bCallTickOnSearchStart : 1;

	/** if set, next tick time will be always reset to service's interval when node is activated */
	UPROPERTY(Category = Service, EditAnywhere, AdvancedDisplay)
	uint32 bRestartTimerOnEachActivation : 1;

	/** if set, service will be notified about search entering underlying branch */
	uint32 bNotifyOnSearch : 1;

#if WITH_EDITORONLY_DATA
	/** if set, the service exposes the option to call Tick event when task search enters this node */
	UPROPERTY(transient)
	uint32 bCanTickOnSearchStartBeExposed : 1;
#endif // WITH_EDITORONLY_DATA

	/** update next tick interval
	 * this function should be considered as const (don't modify state of object) if node is not instanced!
	 * bNotifyTick must be set to true for this function to be called 
	 * Calling INIT_SERVICE_NODE_NOTIFY_FLAGS in the constructor of the service will set this flag automatically */
	AIMODULE_API virtual void TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds) override;

	/** called when search enters underlying branch
	 * this function should be considered as const (don't modify state of object) if node is not instanced! 
	 * bNotifyOnSearch must be set to true for this function to be called  
	 * Calling INIT_SERVICE_NODE_NOTIFY_FLAGS in the constructor of the service will set this flag automatically */
	AIMODULE_API virtual void OnSearchStart(FBehaviorTreeSearchData& SearchData);

#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
#endif // WITH_EDITOR

	/** set next tick time */
	AIMODULE_API virtual void ScheduleNextTick(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory);

	template<typename TickNode, typename OnBecomeRelevant, typename OnCeaseRelevant, typename OnSearchStart>
	void InitNotifyFlags(TickNode, OnBecomeRelevant, OnCeaseRelevant, OnSearchStart)
	{
		bNotifyTick = !std::is_same_v<decltype(&UBTService::TickNode), TickNode>;
		bNotifyBecomeRelevant = !std::is_same_v<decltype(&UBTService::OnBecomeRelevant), OnBecomeRelevant>;
		bNotifyCeaseRelevant = !std::is_same_v<decltype(&UBTService::OnCeaseRelevant), OnCeaseRelevant>;
		bNotifyOnSearch = !std::is_same_v<decltype(&UBTService::OnSearchStart), OnSearchStart>;
	}
};

#define INIT_SERVICE_NODE_NOTIFY_FLAGS() \
	do { \
	using NodeType = TRemovePointer<decltype(this)>::Type; \
	InitNotifyFlags(&NodeType::TickNode, \
					&NodeType::OnBecomeRelevant, \
					&NodeType::OnCeaseRelevant, \
					&NodeType::OnSearchStart); \
	} while (false)
