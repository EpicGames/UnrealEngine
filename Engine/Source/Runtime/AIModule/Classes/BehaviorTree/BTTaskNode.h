// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BehaviorTree/BTNode.h"
#include "BTTaskNode.generated.h"

class UBTService;

struct FBTTaskMemory : public FBTInstancedNodeMemory
{
	float NextTickRemainingTime = 0.f;
	float AccumulatedDeltaTime = 0.f;
};

/** 
 * Task are leaf nodes of behavior tree, which perform actual actions
 *
 * Because some of them can be instanced for specific AI, following virtual functions are not marked as const:
 *  - ExecuteTask
 *  - AbortTask
 *  - TickTask
 *  - OnMessage
 *
 * If your node is not being instanced (default behavior), DO NOT change any properties of object within those functions!
 * Template nodes are shared across all behavior tree components using the same tree asset and must store
 * their runtime properties in provided NodeMemory block (allocation size determined by GetInstanceMemorySize() )
 *
 */

UCLASS(Abstract, MinimalAPI)
class UBTTaskNode : public UBTNode
{
	GENERATED_UCLASS_BODY()

	/** starts this task, should return Succeeded, Failed or InProgress
	 *  (use FinishLatentTask() when returning InProgress)
	 * this function should be considered as const (don't modify state of object) if node is not instanced! */
	AIMODULE_API virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory);

	AIMODULE_API virtual uint16 GetSpecialMemorySize() const override;

protected:
	/** aborts this task, should return Aborted or InProgress
	 *  (use FinishLatentAbort() when returning InProgress)
	 * this function should be considered as const (don't modify state of object) if node is not instanced! */
	AIMODULE_API virtual EBTNodeResult::Type AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory);

	/** sets next tick time */
	AIMODULE_API void SetNextTickTime(uint8* NodeMemory, float RemainingTime) const;

public:
#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
#endif // WITH_EDITOR
	AIMODULE_API virtual void OnGameplayTaskDeactivated(UGameplayTask& Task) override;

	/** message observer's hook */
	AIMODULE_API void ReceivedMessage(UBrainComponent* BrainComp, const FAIMessage& Message);

	/** wrapper for node instancing: ExecuteTask */
	AIMODULE_API EBTNodeResult::Type WrappedExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const;

	/** wrapper for node instancing: AbortTask */
	AIMODULE_API EBTNodeResult::Type WrappedAbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const;

	/** wrapper for node instancing: TickTask
	  * @param OwnerComp	The behavior tree owner of this node
	  * @param NodeMemory	The instance memory of the current node
	  * @param DeltaSeconds		DeltaTime since last call
	  * @param NextNeededDeltaTime		In out parameter, if this node needs a smaller DeltaTime it is the node's responsibility to change it
	  * @returns	True if it actually done some processing or false if it was skipped because of not ticking or in between time interval */
	AIMODULE_API bool WrappedTickTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds, float& NextNeededDeltaTime) const;

	/** wrapper for node instancing: OnTaskFinished */
	AIMODULE_API void WrappedOnTaskFinished(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTNodeResult::Type TaskResult) const;

	/** helper function: finish latent executing */
	AIMODULE_API void FinishLatentTask(UBehaviorTreeComponent& OwnerComp, EBTNodeResult::Type TaskResult) const;

	/** helper function: finishes latent aborting */
	AIMODULE_API void FinishLatentAbort(UBehaviorTreeComponent& OwnerComp) const;

	/** @return true if task search should be discarded when this task is selected to execute but is already running */
	bool ShouldIgnoreRestartSelf() const;

	/** service nodes */
	UPROPERTY()
	TArray<TObjectPtr<UBTService>> Services;

protected:

	/** if set, task search will be discarded when this task is selected to execute but is already running */
	UPROPERTY(EditAnywhere, Category=Task)
	uint32 bIgnoreRestartSelf : 1;

	/** if set, TickTask will be called */
	uint32 bNotifyTick : 1;

	/** if set, OnTaskFinished will be called */
	uint32 bNotifyTaskFinished : 1;

	/** ticks this task 
	 * this function should be considered as const (don't modify state of object) if node is not instanced! 
	 * bNotifyTick must be set to true for this function to be called
	 * Calling INIT_TASK_NODE_NOTIFY_FLAGS in the constructor of the task will set this flag automatically */
	AIMODULE_API virtual void TickTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds);

	/** message handler, default implementation will finish latent execution/abortion
	 * this function should be considered as const (don't modify state of object) if node is not instanced! */
	AIMODULE_API virtual void OnMessage(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, FName Message, int32 RequestID, bool bSuccess);

	/** called when task execution is finished
	 * this function should be considered as const (don't modify state of object) if node is not instanced! 
	 * bNotifyTaskFinished must be set to true for this function to be called 
	 * Calling INIT_TASK_NODE_NOTIFY_FLAGS in the constructor of the task will set this flag automatically */
	AIMODULE_API virtual void OnTaskFinished(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTNodeResult::Type TaskResult);

	/** register message observer */
	AIMODULE_API void WaitForMessage(UBehaviorTreeComponent& OwnerComp, FName MessageType) const;
	AIMODULE_API void WaitForMessage(UBehaviorTreeComponent& OwnerComp, FName MessageType, int32 RequestID) const;
	
	/** unregister message observers */
	AIMODULE_API void StopWaitingForMessages(UBehaviorTreeComponent& OwnerComp) const;
	
	template<typename TickTask,	typename OnTaskFinished>
	void InitNotifyFlags(TickTask, OnTaskFinished)
	{
		bNotifyTick = !std::is_same_v<decltype(&UBTTaskNode::TickTask), TickTask>;
		bNotifyTaskFinished = !std::is_same_v<decltype(&UBTTaskNode::OnTaskFinished), OnTaskFinished>;
	}

	/** if set, conditional tick will use remaining time from node's memory */
	uint8 bTickIntervals : 1;
};

#define INIT_TASK_NODE_NOTIFY_FLAGS() \
	do { \
	using NodeType = TRemovePointer<decltype(this)>::Type; \
	InitNotifyFlags(&NodeType::TickTask, &NodeType::OnTaskFinished); \
	} while (false)

inline bool UBTTaskNode::ShouldIgnoreRestartSelf() const
{
	return bIgnoreRestartSelf;
}
