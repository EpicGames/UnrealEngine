// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/Blueprint.h"
#include "BehaviorTree/BTCompositeNode.h"
#include "BehaviorTree.generated.h"

class UBlackboardData;
class UBTDecorator;

UCLASS(BlueprintType, MinimalAPI)
class UBehaviorTree : public UObject, public IBlackboardAssetProvider
{
	GENERATED_UCLASS_BODY()

	/** root node of loaded tree */
	UPROPERTY(BlueprintReadOnly, Category = BehaviorTree)
	TObjectPtr<UBTCompositeNode> RootNode;

#if WITH_EDITORONLY_DATA

	/** Graph for Behavior Tree */
	UPROPERTY()
	TObjectPtr<class UEdGraph>	BTGraph;

	/** Info about the graphs we last edited */
	UPROPERTY()
	TArray<FEditedDocumentInfo> LastEditedDocuments;

#endif

	// BEGIN IBlackboardAssetProvider
	/** @return blackboard asset */
	AIMODULE_API virtual UBlackboardData* GetBlackboardAsset() const override;
	// END IBlackboardAssetProvider

	/** blackboard asset for this tree */
	UPROPERTY(BlueprintReadOnly, Category = BehaviorTree)
	TObjectPtr<UBlackboardData> BlackboardAsset;

	/** root level decorators, used by subtrees */
	UPROPERTY(BlueprintReadOnly, Category = BehaviorTree)
	TArray<TObjectPtr<UBTDecorator>> RootDecorators;

	/** logic operators for root level decorators, used by subtrees  */
	UPROPERTY()
	TArray<FBTDecoratorLogic> RootDecoratorOps;

	/** memory size required for instance of this tree */
	uint16 InstanceMemorySize;
};
