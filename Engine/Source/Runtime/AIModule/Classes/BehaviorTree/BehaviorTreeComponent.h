// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EngineDefines.h"
#include "BehaviorTree/BehaviorTreeTypes.h"
#include "GameplayTagContainer.h"
#include "AITypes.h"
#include "BrainComponent.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "BehaviorTreeComponent.generated.h"

class FBehaviorTreeDebugger;
class UBehaviorTree;
class UBTAuxiliaryNode;
class UBTCompositeNode;
class UBTDecorator;
class UBTNode;
class UBTTask_RunBehavior;
class UBTTask_RunBehaviorDynamic;
class UBTTaskNode;

struct FBTNodeExecutionInfo
{
	/** index of first task allowed to be executed */
	FBTNodeIndex SearchStart;

	/** index of last task allowed to be executed */
	FBTNodeIndex SearchEnd;

	/** node to be executed */
	TObjectPtr<const UBTCompositeNode> ExecuteNode;

	/** subtree index */
	uint16 ExecuteInstanceIdx;

	/** result used for resuming execution */
	TEnumAsByte<EBTNodeResult::Type> ContinueWithResult;

	/** if set, tree will try to execute next child of composite instead of forcing branch containing SearchStart */
	uint8 bTryNextChild : 1;

	/** if set, request was not instigated by finishing task/initialization but is a restart (e.g. decorator) */
	uint8 bIsRestart : 1;

	FBTNodeExecutionInfo() : ExecuteNode(NULL), bTryNextChild(false), bIsRestart(false) { }
};

struct FBTPendingExecutionInfo
{
	/** next task to execute */
	TObjectPtr<UBTTaskNode> NextTask;

	/** if set, tree ran out of nodes */
	uint32 bOutOfNodes : 1;

	/** if set, request can't be executed */
	uint32 bLocked : 1;

	FBTPendingExecutionInfo() : NextTask(NULL), bOutOfNodes(false), bLocked(false) {}
	bool IsSet() const { return (NextTask || bOutOfNodes) && !bLocked; }
	bool IsLocked() const { return bLocked; }

	void Lock() { bLocked = true; }
	void Unlock() { bLocked = false; }
};

struct FBTTreeStartInfo
{
	UBehaviorTree* Asset;
	EBTExecutionMode::Type ExecuteMode;
	uint8 bPendingInitialize : 1;

	FBTTreeStartInfo() : Asset(nullptr), ExecuteMode(EBTExecutionMode::Looped), bPendingInitialize(0) {}
	bool IsSet() const { return Asset != nullptr; }
	bool HasPendingInitialize() const { return bPendingInitialize && IsSet(); }
};

enum class EBTBranchAction : uint16
{
	None = 0x0,
	DecoratorEvaluate = 0x1,
	DecoratorActivate_IfNotExecuting = 0x2,
	DecoratorActivate_EvenIfExecuting = 0x4,
	DecoratorActivate = DecoratorActivate_IfNotExecuting | DecoratorActivate_EvenIfExecuting,
	DecoratorDeactivate = 0x8,
	UnregisterAuxNodes = 0x10,
	StopTree_Safe = 0x20,
	StopTree_Forced = 0x40,
	ActiveNodeEvaluate = 0x80,
	SubTreeEvaluate = 0x100,
	ProcessPendingInitialize = 0x200,
	Cleanup = 0x400,
	UninitializeComponent = 0x800,
	StopTree = StopTree_Safe | StopTree_Forced,
	Changing_Topology_Actions = UnregisterAuxNodes | StopTree | ProcessPendingInitialize | Cleanup | UninitializeComponent,
	All = DecoratorEvaluate | DecoratorActivate_IfNotExecuting | DecoratorActivate_EvenIfExecuting | DecoratorDeactivate | Changing_Topology_Actions | ActiveNodeEvaluate | SubTreeEvaluate,
};
ENUM_CLASS_FLAGS(EBTBranchAction);

UCLASS(ClassGroup = AI, meta = (BlueprintSpawnableComponent), MinimalAPI)
class UBehaviorTreeComponent : public UBrainComponent
{
	GENERATED_UCLASS_BODY()

	// UActorComponent overrides
	AIMODULE_API virtual void RegisterComponentTickFunctions(bool bRegister) override;
	AIMODULE_API virtual void SetComponentTickEnabled(bool bEnabled) override;

	// Begin UBrainComponent overrides
	AIMODULE_API virtual void StartLogic() override;
	AIMODULE_API virtual void RestartLogic() override;
	AIMODULE_API virtual void StopLogic(const FString& Reason) override;
	AIMODULE_API virtual void PauseLogic(const FString& Reason) override;
	AIMODULE_API virtual EAILogicResuming::Type ResumeLogic(const FString& Reason) override;

	/** indicates instance has been initialized to work with specific BT asset */
	AIMODULE_API bool TreeHasBeenStarted() const;

public:
	/** DO NOT USE. This constructor is for internal usage only for hot-reload purposes. */
	AIMODULE_API UBehaviorTreeComponent(FVTableHelper& Helper);

	AIMODULE_API virtual bool IsRunning() const override;
	AIMODULE_API virtual bool IsPaused() const override;
	AIMODULE_API virtual void Cleanup() override;
	AIMODULE_API virtual void HandleMessage(const FAIMessage& Message) override;
	// End UBrainComponent overrides

	// Begin UActorComponent overrides
	AIMODULE_API virtual void UninitializeComponent() override;
	// End UActorComponent overrides

	/** starts execution from root */
	AIMODULE_API void StartTree(UBehaviorTree& Asset, EBTExecutionMode::Type ExecuteMode = EBTExecutionMode::Looped);

	/** stops execution */
	AIMODULE_API void StopTree(EBTStopMode::Type StopMode = EBTStopMode::Safe);

	/** restarts execution from root
	 * @param RestartMode to force the reevaluation of the root node, which could skip active nodes that are removed and then readded (default)
	 *        or a complete restart from scratch of the tree (equivalent of StopTree then StartTree)
	 */
	AIMODULE_API void RestartTree(EBTRestartMode RestartMode = EBTRestartMode::ForceReevaluateRootNode);

	/** request execution change */
	AIMODULE_API void RequestExecution(const UBTCompositeNode* RequestedOn, const int32 InstanceIdx, 
		const UBTNode* RequestedBy, const int32 RequestedByChildIndex,
		const EBTNodeResult::Type ContinueWithResult, bool bStoreForDebugger = true);

	/** replaced by the RequestBranchEvaluation from decorator*/
	void RequestExecution(const UBTDecorator* RequestedBy) { check(RequestedBy); RequestBranchEvaluation(*RequestedBy); }

	/** replaced by RequestBranchEvaluation with EBTNodeResult */
	void RequestExecution(EBTNodeResult::Type ContinueWithResult) { RequestBranchEvaluation(ContinueWithResult); }

	/** request unregistration of aux nodes in the specified branch */
	UE_DEPRECATED(5.0, "This function is deprecated. Please use RequestBranchDeactivation instead.")
	AIMODULE_API void RequestUnregisterAuxNodesInBranch(const UBTCompositeNode* Node);

	/** request branch evaluation: helper for active node (ex: tasks) */
	AIMODULE_API void RequestBranchEvaluation(EBTNodeResult::Type ContinueWithResult);

	/** request branch evaluation: helper for decorator */
	AIMODULE_API void RequestBranchEvaluation(const UBTDecorator& RequestedBy);

	/** request branch activation: helper for decorator */
	AIMODULE_API void RequestBranchActivation(const UBTDecorator& RequestedBy, const bool bRequestEvenIfExecuting);

	/** request branch deactivation: helper for decorator */
	AIMODULE_API void RequestBranchDeactivation(const UBTDecorator& RequestedBy);

	/** finish latent execution or abort */
	AIMODULE_API void OnTaskFinished(const UBTTaskNode* TaskNode, EBTNodeResult::Type TaskResult);

	/** setup message observer for given task */
	AIMODULE_API void RegisterMessageObserver(const UBTTaskNode* TaskNode, FName MessageType);
	AIMODULE_API void RegisterMessageObserver(const UBTTaskNode* TaskNode, FName MessageType, FAIRequestID MessageID);
	
	/** remove message observers registered with task */
	AIMODULE_API void UnregisterMessageObserversFrom(const UBTTaskNode* TaskNode);
	AIMODULE_API void UnregisterMessageObserversFrom(const FBTNodeIndex& TaskIdx);

	/** add active parallel task */
	AIMODULE_API void RegisterParallelTask(const UBTTaskNode* TaskNode);

	/** remove parallel task */
	AIMODULE_API void UnregisterParallelTask(const UBTTaskNode* TaskNode, uint16 InstanceIdx);

	/** unregister all aux nodes less important than given index */
	AIMODULE_API void UnregisterAuxNodesUpTo(const FBTNodeIndex& Index);

	/** unregister all aux nodes between given execution index range: FromIndex < AuxIndex < ToIndex */
	AIMODULE_API void UnregisterAuxNodesInRange(const FBTNodeIndex& FromIndex, const FBTNodeIndex& ToIndex);

	/** unregister all aux nodes in branch of tree */
	AIMODULE_API void UnregisterAuxNodesInBranch(const UBTCompositeNode* Node, bool bApplyImmediately = true);

	/** BEGIN UActorComponent overrides */
	AIMODULE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	/** END UActorComponent overrides */

	/** Schedule when will be the next tick, 0.0f means next frame, FLT_MAX means never */
	AIMODULE_API void ScheduleNextTick(float NextDeltaTime);

	/** process execution flow */
	AIMODULE_API void ProcessExecutionRequest();

	/** schedule execution flow update in next tick */
	AIMODULE_API void ScheduleExecutionUpdate();

	/** tries to find behavior tree instance in context */
	AIMODULE_API int32 FindInstanceContainingNode(const UBTNode* Node) const;

	/** tries to find template node for given instanced node */
	AIMODULE_API UBTNode* FindTemplateNode(const UBTNode* Node) const;

	/** @return current tree */
	UBehaviorTree* GetCurrentTree() const;

	/** @return tree from top of instance stack */
	UBehaviorTree* GetRootTree() const;

	/** @return active node */
	const UBTNode* GetActiveNode() const;
	
	/** get index of active instance on stack */
	uint16 GetActiveInstanceIdx() const;

	/** @return node memory */
	AIMODULE_API uint8* GetNodeMemory(const UBTNode* Node, int32 InstanceIdx) const;

	/** @return node memory. Virtual override for when GetNodeMemory fails to find NodeMemory */
	AIMODULE_API virtual uint8* GetNodeMemoryAux(const UBTNode* Node, int32 InstanceIdx) const;

	/** @return true if ExecutionRequest is switching to higher priority node */
	bool IsRestartPending() const;

	/** @return true if waiting for abort to finish */
	bool IsAbortPending() const;

	/** @return true if active node is one of child nodes of given one */
	AIMODULE_API bool IsExecutingBranch(const UBTNode* Node, int32 ChildIndex = -1) const;

	/** @return true if aux node is currently active */
	AIMODULE_API bool IsAuxNodeActive(const UBTAuxiliaryNode* AuxNode) const;
	AIMODULE_API bool IsAuxNodeActive(const UBTAuxiliaryNode* AuxNodeTemplate, int32 InstanceIdx) const;

	/** Returns true if InstanceStack contains any BT runtime instances */
	bool IsInstanceStackEmpty() const
	{
		return (InstanceStack.Num() == 0);
	}

	/** Returns the accumulated delta time for the current tick */
	float GetAccumulatedTickDeltaTime() const
	{
		return AccumulatedTickDeltaTime;
	}

	/** @return status of speficied task */
	AIMODULE_API EBTTaskStatus::Type GetTaskStatus(const UBTTaskNode* TaskNode) const;

	AIMODULE_API virtual FString GetDebugInfoString() const override;
	AIMODULE_API virtual FString DescribeActiveTasks() const;
	AIMODULE_API virtual FString DescribeActiveTrees() const;

	/** @return the cooldown tag end time, 0.0f if CooldownTag is not found */
	UFUNCTION(BlueprintCallable, Category = "AI|Logic")
	AIMODULE_API double GetTagCooldownEndTime(FGameplayTag CooldownTag) const;

	/** add to the cooldown tag's duration */
	UFUNCTION(BlueprintCallable, Category = "AI|Logic")
	AIMODULE_API void AddCooldownTagDuration(FGameplayTag CooldownTag, float CooldownDuration, bool bAddToExistingDuration);

	/** assign subtree to RunBehaviorDynamic task specified by tag. */
	UFUNCTION(BlueprintCallable, Category="AI|Logic")
	AIMODULE_API virtual void SetDynamicSubtree(FGameplayTag InjectTag, UBehaviorTree* BehaviorAsset);

	/** assign subtree to RunBehaviorDynamic task specified by tag. Optional Starting Node can be given if the location in the tree is known to avoid parsing the whole tree.  */
	AIMODULE_API virtual void SetDynamicSubtree(FGameplayTag InjectTag, UBehaviorTree* BehaviorAsset, UBTCompositeNode* OptionalStartingNode);

	/** Will call the given functor on each task node in the current instance stacks. */
	AIMODULE_API void ForEachChildTask(TFunctionRef<void (UBTTaskNode&, const FBehaviorTreeInstance&, int32 InstanceIndex)> Functor);

	/** Will call the given functor on each child node from the given start node. InstanceIndex can be found using FindInstanceContainingNode. */
	AIMODULE_API void ForEachChildTask(UBTCompositeNode& StartNode, int32 InstanceIndex, TFunctionRef<void(UBTTaskNode&, const FBehaviorTreeInstance&, int32 InstanceIndex)> Functor);
// Code for timing BT Search for FramePro
#if !UE_BUILD_SHIPPING
	static AIMODULE_API void EndFrame();
#endif

#if ENABLE_VISUAL_LOG
	AIMODULE_API virtual void DescribeSelfToVisLog(struct FVisualLogEntry* Snapshot) const override;
#endif

#if CSV_PROFILER_STATS
	/** Set a custom CSV tick stat name, must point to a static string */
	void SetCSVTickStatName(const char* InCSVTickStatName) { CSVTickStatName = InCSVTickStatName; }
#endif

	static AIMODULE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

protected:
	/** stack of behavior tree instances */
	TArray<FBehaviorTreeInstance> InstanceStack;

	/** list of known subtree instances */
	TArray<FBehaviorTreeInstanceId> KnownInstances;

	/** instanced nodes */
	UPROPERTY(transient)
	TArray<TObjectPtr<UBTNode>> NodeInstances;

	/** search data being currently used */
	FBehaviorTreeSearchData SearchData;

	/** execution request, search will be performed when current task finish execution/aborting */
	FBTNodeExecutionInfo ExecutionRequest;

	/** result of ExecutionRequest, will be applied when current task finish aborting */
	FBTPendingExecutionInfo PendingExecution;

	struct FBranchActionInfo
	{
		FBranchActionInfo(const EBTBranchAction InAction)
			: Action(InAction)
		{}
		FBranchActionInfo(const UBTNode* InNode, const EBTBranchAction InAction)
			: Node(InNode)
			, Action(InAction)
		{}
		FBranchActionInfo(const UBTNode* InNode, EBTNodeResult::Type InContinueWithResult, const EBTBranchAction InAction)
			: Node(InNode)
			, ContinueWithResult(InContinueWithResult)
			, Action(InAction)
		{}
		TObjectPtr<const UBTNode> Node = nullptr;
		EBTNodeResult::Type ContinueWithResult = EBTNodeResult::Succeeded;
		EBTBranchAction Action;
	};

	/* Type of suspended branch action */
	EBTBranchAction SuspendedBranchActions;

	/** list of all pending branch action requests */
	TArray<FBranchActionInfo> PendingBranchActionRequests;

	/** stored data for starting new tree, waits until previously running finishes aborting */
	FBTTreeStartInfo TreeStartInfo;

	/** message observers mapped by instance & execution index */
	TMultiMap<FBTNodeIndex,FAIMessageObserverHandle> TaskMessageObservers;

	/** behavior cooldowns mapped by tag to last time it was set */
	TMap<FGameplayTag, double> CooldownTagsMap;

#if USE_BEHAVIORTREE_DEBUGGER
	/** search flow for debugger */
	mutable TArray<TArray<FBehaviorTreeDebuggerInstance::FNodeFlowData> > CurrentSearchFlow;
	mutable TArray<TArray<FBehaviorTreeDebuggerInstance::FNodeFlowData> > CurrentRestarts;
	mutable TMap<FName, FString> SearchStartBlackboard;
	mutable TArray<FBehaviorTreeDebuggerInstance> RemovedInstances;

	/** debugger's recorded data */
	mutable TArray<FBehaviorTreeExecutionStep> DebuggerSteps;

	/** set when at least one debugger window is opened */
	static AIMODULE_API int32 ActiveDebuggerCounter;
#endif

	// Code for timing BT Search for FramePro
#if !UE_BUILD_SHIPPING
	static AIMODULE_API bool bAddedEndFrameCallback;
	static AIMODULE_API double FrameSearchTime;
	static AIMODULE_API int32 NumSearchTimeCalls;
#endif

	/** index of last active instance on stack */
	uint16 ActiveInstanceIdx;

	/** loops tree execution */
	uint8 bLoopExecution : 1;

	/** set when execution is waiting for tasks to finish their latent abort (current or parallel's main) */
	uint8 bWaitingForLatentAborts : 1;

	/** set when execution update is scheduled for next tick */
	uint8 bRequestedFlowUpdate : 1;

	/** set when tree stop was called */
	uint8 bRequestedStop : 1;

	/** if set, tree execution is allowed */
	uint8 bIsRunning : 1;

	/** if set, execution requests will be postponed */
	uint8 bIsPaused : 1;

	/** push behavior tree instance on execution stack
	 *	@NOTE: should never be called out-side of BT execution, meaning only BT tasks can push another BT instance! */
	AIMODULE_API bool PushInstance(UBehaviorTree& TreeAsset);

	/** add unique Id of newly created subtree to KnownInstances list and return its index */
	AIMODULE_API uint8 UpdateInstanceId(UBehaviorTree* TreeAsset, const UBTNode* OriginNode, int32 OriginInstanceIdx);

	/** remove instanced nodes, known subtree instances and safely clears their persistent memory */
	AIMODULE_API void RemoveAllInstances();

	/** copy memory block from running instances to persistent memory */
	AIMODULE_API void CopyInstanceMemoryToPersistent();

	/** copy memory block from persistent memory to running instances (rollback) */
	AIMODULE_API void CopyInstanceMemoryFromPersistent();

	/** called when tree runs out of nodes to execute */
	AIMODULE_API virtual void OnTreeFinished();

	/** apply pending node updates from SearchData */
	AIMODULE_API void ApplySearchData(UBTNode* NewActiveNode);

	/** apply pending node updates required for discarded search */
	AIMODULE_API void ApplyDiscardedSearch();

	/** apply updates from specific list */
	UE_DEPRECATED(5.4, "This function is deprecated. Please use ApplyAllSearchUpdates instead.")
	AIMODULE_API void ApplySearchUpdates(const TArray<FBehaviorTreeSearchUpdate>& UpdateList, int32 NewNodeExecutionIndex, bool bPostUpdate = false);

	/** 
	 * Apply updates and post update from specified UpdateList.
	 * @param bDoPostUpdate if true the post updates will also be processed.
	 * @param bAllowTaskUpdates If false Task node updates will not be processed from the UpdateList.
	 */
	AIMODULE_API void ApplyAllSearchUpdates(const TArray<FBehaviorTreeSearchUpdate>& UpdateList, int32 NewNodeExecutionIndex, bool bDoPostUpdate = true, bool bAllowTaskUpdates = true);

	/** abort currently executed task */
	AIMODULE_API void AbortCurrentTask();

	/** execute new task */
	AIMODULE_API void ExecuteTask(UBTTaskNode* TaskNode);

	/** deactivate all nodes up to requested one */
	AIMODULE_API bool DeactivateUpTo(const UBTCompositeNode* Node, uint16 NodeInstanceIdx, EBTNodeResult::Type& NodeResult, int32& OutLastDeactivatedChildIndex);

	/** returns true if execution was waiting on latent aborts and they are all finished;  */
	AIMODULE_API bool TrackPendingLatentAborts();

	/** tracks if there are new tasks using latent abort in progress */
	AIMODULE_API void TrackNewLatentAborts();

	/** return true if the current or any parallel task has a latent abort in progress */
	AIMODULE_API bool HasActiveLatentAborts() const;

	/** apply pending execution from last task search */
	AIMODULE_API void ProcessPendingExecution();

	/** apply pending tree initialization */
	AIMODULE_API void ProcessPendingInitialize();

	/** restore state of tree to state before search */
	AIMODULE_API void RollbackSearchChanges();

	/** make a snapshot for debugger */
	AIMODULE_API void StoreDebuggerExecutionStep(EBTExecutionSnap::Type SnapType);

	/** make a snapshot for debugger from given subtree instance */
	AIMODULE_API void StoreDebuggerInstance(FBehaviorTreeDebuggerInstance& InstanceInfo, uint16 InstanceIdx, EBTExecutionSnap::Type SnapType) const;
	AIMODULE_API void StoreDebuggerRemovedInstance(uint16 InstanceIdx) const;

	/** store search step for debugger */
	AIMODULE_API void StoreDebuggerSearchStep(const UBTNode* Node, uint16 InstanceIdx, EBTNodeResult::Type NodeResult) const;
	AIMODULE_API void StoreDebuggerSearchStep(const UBTNode* Node, uint16 InstanceIdx, bool bPassed) const;

	/** store restarting node for debugger */
	AIMODULE_API void StoreDebuggerRestart(const UBTNode* Node, uint16 InstanceIdx, bool bAllowed);

	/** describe blackboard's key values */
	AIMODULE_API void StoreDebuggerBlackboard(TMap<FName, FString>& BlackboardValueDesc) const;

	/** gather nodes runtime descriptions */
	AIMODULE_API void StoreDebuggerRuntimeValues(TArray<FString>& RuntimeDescriptions, UBTNode* RootNode, uint16 InstanceIdx) const;

	/** update runtime description of given task node in latest debugger's snapshot */
	AIMODULE_API void UpdateDebuggerAfterExecution(const UBTTaskNode* TaskNode, uint16 InstanceIdx) const;

	/** check if debugger is currently running and can gather data */
	static AIMODULE_API bool IsDebuggerActive();

	/** Return NodeA's relative priority in regards to NodeB */
	AIMODULE_API EBTNodeRelativePriority CalculateRelativePriority(const UBTNode* NodeA, const UBTNode* NodeB) const;

	/** Evaluate a branch as current active node is finished */
	AIMODULE_API void EvaluateBranch(EBTNodeResult::Type LastResult);

	/** Evaluate a branch as the decorator conditions have changed */
	AIMODULE_API void EvaluateBranch(const UBTDecorator& RequestedBy);

	/** Activate a branch as the decorator conditions are now passing */
	AIMODULE_API void ActivateBranch(const UBTDecorator& RequestedBy, bool bRequestEvenIfNotExecuting);

	/** Deactivate a branch as the decorator conditions are not passing anymore */
	AIMODULE_API void DeactivateBranch(const UBTDecorator& RequestedBy);

	/** Suspend any branch actions and queue them to be processed later by ResumeBranchActions() */
	AIMODULE_API void SuspendBranchActions(EBTBranchAction BranchActions = EBTBranchAction::All);

	/** Resume branch actions and execute all the queued up ones */
	AIMODULE_API void ResumeBranchActions();

	UE_DEPRECATED(5.1, "This function is deprecated. Please use SuspendBranchActions instead.")
	void SuspendBranchDeactivation() { SuspendBranchActions(); }

	UE_DEPRECATED(5.1, "This function is deprecated. Please use ResumeBranchActions instead.")
	void ResumeBranchDeactivation() { ResumeBranchActions(); }

	struct FBTSuspendBranchActionsScoped
	{
		FBTSuspendBranchActionsScoped(UBehaviorTreeComponent& InBTComp, EBTBranchAction BranchActions = EBTBranchAction::All)
			: BTComp(InBTComp)
		{
			BTComp.SuspendBranchActions(BranchActions);
		}
		~FBTSuspendBranchActionsScoped()
		{
			BTComp.ResumeBranchActions();
		}
		UBehaviorTreeComponent& BTComp;
	};

	void TickNewlyAddedAuxNodesHelper();

	UE_DEPRECATED(5.1, "This struct is deprecated. Please use FBTSuspendBranchActionsScoped instead.")
	typedef FBTSuspendBranchActionsScoped FBTSuspendBranchDeactivationScoped;

	friend UBTNode;
	friend UBTCompositeNode;
	friend UBTTaskNode;
	friend UBTTask_RunBehavior;
	friend UBTTask_RunBehaviorDynamic;
	friend FBehaviorTreeDebugger;
	friend FBehaviorTreeInstance;

private:
	/** Please don't call this function directly instead call ApplyAllSearchUpdates */
	void ApplySearchUpdatesImpl(const TArray<FBehaviorTreeSearchUpdate>& UpdateList, int32 NewNodeExecutionIndex, bool bPostUpdate, bool bAllowTaskUpdates);

protected:
	/** data asset defining the tree */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = AI)
	TObjectPtr<UBehaviorTree> DefaultBehaviorTreeAsset;

	/** Used to tell tickmanager that we want interval ticking */
	bool bTickedOnce = false;

	/** Predicted next DeltaTime*/
	float NextTickDeltaTime = 0.f;

	/** Accumulated DeltaTime if ticked more than predicted next delta time */
	float AccumulatedTickDeltaTime = 0.f;

	/** Current frame DeltaTime */
	float CurrentFrameDeltaTime = 0.f;

	/** GameTime of the last DeltaTime request, used for debugging to output warnings about ticking */
	double LastRequestedDeltaTimeGameTime = 0;

#if CSV_PROFILER_STATS
	/** CSV tick stat name. Can be changed but must point to a static string */
	const char* CSVTickStatName = "BehaviorTreeTick";
#endif
};

//////////////////////////////////////////////////////////////////////////
// Inlines

inline UBehaviorTree* UBehaviorTreeComponent::GetCurrentTree() const
{
	return InstanceStack.Num() ? KnownInstances[InstanceStack[ActiveInstanceIdx].InstanceIdIndex].TreeAsset : NULL;
}

inline UBehaviorTree* UBehaviorTreeComponent::GetRootTree() const
{
	return InstanceStack.Num() ? KnownInstances[InstanceStack[0].InstanceIdIndex].TreeAsset : NULL;
}

inline const UBTNode* UBehaviorTreeComponent::GetActiveNode() const
{
	return InstanceStack.Num() ? InstanceStack[ActiveInstanceIdx].ActiveNode : NULL;
}

inline uint16 UBehaviorTreeComponent::GetActiveInstanceIdx() const
{
	return ActiveInstanceIdx;
}

inline bool UBehaviorTreeComponent::IsRestartPending() const
{
	return ExecutionRequest.ExecuteNode && !ExecutionRequest.bTryNextChild;
}

inline bool UBehaviorTreeComponent::IsAbortPending() const
{
	return bWaitingForLatentAborts || PendingExecution.IsSet();
}
