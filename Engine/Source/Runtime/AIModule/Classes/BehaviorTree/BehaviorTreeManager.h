// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "BehaviorTreeManager.generated.h"

class UBehaviorTree;
class UBehaviorTreeComponent;
class UBTCompositeNode;
class UBTDecorator;

USTRUCT()
struct FBehaviorTreeTemplateInfo
{
	GENERATED_USTRUCT_BODY()

	/** behavior tree asset */
	UPROPERTY()
	TObjectPtr<UBehaviorTree> Asset = nullptr;

	/** initialized template */
	UPROPERTY(transient)
	TObjectPtr<UBTCompositeNode> Template = nullptr;

	/** size required for instance memory */
	uint16 InstanceMemorySize;
};

UCLASS(config=Engine, Transient, MinimalAPI)
class UBehaviorTreeManager : public UObject
{
	GENERATED_UCLASS_BODY()

	/** limit for recording execution steps for debugger */
	UPROPERTY(config)
	int32 MaxDebuggerSteps;

	/** get behavior tree template for given blueprint */
	AIMODULE_API bool LoadTree(UBehaviorTree& Asset, UBTCompositeNode*& Root, uint16& InstanceMemorySize);

	/** get aligned memory size */
	static AIMODULE_API uint16 GetAlignedDataSize(uint16 Size);

	/** helper function for sorting and aligning node memory */
	static AIMODULE_API void InitializeMemoryHelper(const TArray<UBTDecorator*>& Nodes, TArray<uint16>& MemoryOffsets, int32& MemorySize, bool bForceInstancing = false);

	/** cleanup hooks for map loading */
	AIMODULE_API virtual void FinishDestroy() override;

	AIMODULE_API void DumpUsageStats() const;

	/** register new behavior tree component for tracking */
	AIMODULE_API void AddActiveComponent(UBehaviorTreeComponent& Component);

	/** unregister behavior tree component from tracking */
	AIMODULE_API void RemoveActiveComponent(UBehaviorTreeComponent& Component);

	static AIMODULE_API UBehaviorTreeManager* GetCurrent(UWorld* World);
	static AIMODULE_API UBehaviorTreeManager* GetCurrent(UObject* WorldContextObject);

	TConstArrayView<TObjectPtr<UBehaviorTreeComponent>> GetActiveComponents() const 
	{ 
	    return ActiveComponents;
	}

protected:

	/** initialized tree templates */
	UPROPERTY()
	TArray<FBehaviorTreeTemplateInfo> LoadedTemplates;

	UPROPERTY()
	TArray<TObjectPtr<UBehaviorTreeComponent>> ActiveComponents;
};
