// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blackboard/BlackboardKey.h"
#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "InputCoreTypes.h"
#include "Templates/SubclassOf.h"
#include "BehaviorTreeTypes.generated.h"

class FBlackboardDecoratorDetails;
class UBehaviorTree;
class UBehaviorTreeComponent;
class UBlackboardComponent;
class UBlackboardData;
class UBlackboardKeyType;
class UBTAuxiliaryNode;
class UBTCompositeNode;
class UBTNode;
class UBTTaskNode;
struct FBehaviorTreeSearchData;

// Visual logging helper
#define BT_VLOG(Context, Verbosity, Format, ...) UE_VLOG(Context->OwnerComp.IsValid() ? Context->OwnerComp->GetOwner() : NULL, LogBehaviorTree, Verbosity, Format, ##__VA_ARGS__)
#define BT_SEARCHLOG(SearchData, Verbosity, Format, ...) UE_VLOG(SearchData.OwnerComp.GetOwner(), LogBehaviorTree, Verbosity, Format, ##__VA_ARGS__)

// Behavior tree debugger in editor
#define USE_BEHAVIORTREE_DEBUGGER	(1 && WITH_EDITORONLY_DATA)

DECLARE_STATS_GROUP(TEXT("Behavior Tree"), STATGROUP_AIBehaviorTree, STATCAT_Advanced);

DECLARE_CYCLE_STAT_EXTERN(TEXT("BT Tick"),STAT_AI_BehaviorTree_Tick,STATGROUP_AIBehaviorTree, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("BT Load Time"),STAT_AI_BehaviorTree_LoadTime,STATGROUP_AIBehaviorTree, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("BT Search Time"),STAT_AI_BehaviorTree_SearchTime,STATGROUP_AIBehaviorTree, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("BT Execution Time"),STAT_AI_BehaviorTree_ExecutionTime,STATGROUP_AIBehaviorTree, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("BT Auxiliary Update Time"),STAT_AI_BehaviorTree_AuxUpdateTime,STATGROUP_AIBehaviorTree, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("BT Cleanup Time"), STAT_AI_BehaviorTree_Cleanup, STATGROUP_AIBehaviorTree, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("BT Stop Tree Time"), STAT_AI_BehaviorTree_StopTree, STATGROUP_AIBehaviorTree, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Templates"),STAT_AI_BehaviorTree_NumTemplates,STATGROUP_AIBehaviorTree, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Num Instances"),STAT_AI_BehaviorTree_NumInstances,STATGROUP_AIBehaviorTree, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Instance memory"),STAT_AI_BehaviorTree_InstanceMemory,STATGROUP_AIBehaviorTree, AIMODULE_API);

#ifndef AI_BLACKBOARD_KEY_SIZE_8
template <> struct TIsValidVariadicFunctionArg<FBlackboard::FKey>
{
	enum { Value = true };
};
#endif

#if WITH_EDITORONLY_DATA
namespace UE::BehaviorTree::Messages
{
	extern AIMODULE_API FText BlackboardNotSet;
}
#endif // WITH_EDITORONLY_DATA

enum class EBlackboardNotificationResult : uint8
{
	RemoveObserver,
	ContinueObserving
};

// delegate defines
DECLARE_DELEGATE_TwoParams(FOnBlackboardChange, const UBlackboardComponent&, FBlackboard::FKey /*key ID*/);

// using "not checked" user policy (means race detection is disabled) because this delegate is stored in a TMultiMap and causes its reallocation
// from inside delegate's execution. This is incompatible with race detection that needs to access the delegate instance after its execution
using FOnBlackboardChangeNotification = TDelegate<EBlackboardNotificationResult(const UBlackboardComponent&, FBlackboard::FKey keyID), FNotThreadSafeNotCheckedDelegateUserPolicy>;

namespace BTSpecialChild
{
	inline constexpr int32 NotInitialized = -1;	// special value for child indices: needs to be initialized
	inline constexpr int32 ReturnToParent = -2;	// special value for child indices: return to parent node
	
	inline constexpr uint8 OwnedByComposite = MAX_uint8;	// special value for aux node's child index: owned by composite node instead of a task
}

UENUM(BlueprintType)
namespace EBTNodeResult
{
	// keep in sync with DescribeNodeResult()
	enum Type : int
	{
		// finished as success
		Succeeded,
		// finished as failure
		Failed,
		// finished aborting = failure
		Aborted,
		// not finished yet
		InProgress,
	};
}

namespace EBTExecutionMode
{
	enum Type
	{
		SingleRun,
		Looped,
	};
}

namespace EBTStopMode
{
	enum Type
	{
		Safe,
		Forced,
	};
}

enum class EBTRestartMode : uint8
{
	ForceReevaluateRootNode, // (Default) will just request a new execution on the root node and any active nodes that gets re-added will not get CeaseRelevant/BecomeRelevant notification
	CompleteRestart, // essentially equivalent to calling StopTree and then StartTree. Every active node is going to be removed and the execution started from the root
};

namespace EBTMemoryInit
{
	enum Type
	{
		Initialize,		// first time initialization
		RestoreSubtree,	// loading saved data on reentering subtree
	};
}

namespace EBTMemoryClear
{
	enum Type
	{
		Destroy,		// final clear
		StoreSubtree,	// saving data on leaving subtree
	};
}

UENUM()
namespace EBTFlowAbortMode
{
	// keep in sync with DescribeFlowAbortMode()

	enum Type : int
	{
		None				UMETA(DisplayName="Nothing"),
		LowerPriority		UMETA(DisplayName="Lower Priority"),
		Self				UMETA(DisplayName="Self"),
		Both				UMETA(DisplayName="Both"),
	};
}

namespace EBTActiveNode
{
	// keep in sync with DescribeActiveNode()
	enum Type
	{
		Composite,
		ActiveTask,
		AbortingTask,
		InactiveTask,
	};
}

namespace EBTTaskStatus
{
	// keep in sync with DescribeTaskStatus()
	enum Type
	{
		Active,
		Aborting,
		Inactive,
	};
}

namespace EBTNodeUpdateMode
{
	// keep in sync with DescribeNodeUpdateMode()
	enum Type
	{
		Unknown,
		Add,				// add node
		Remove,				// remove node
	};
}

/** wrapper struct for holding a parallel task node and its status */
struct FBehaviorTreeParallelTask
{
	/** worker object */
	TObjectPtr<const UBTTaskNode> TaskNode;

	/** additional mode data used for context switching */
	EBTTaskStatus::Type Status;

	FBehaviorTreeParallelTask() : TaskNode(nullptr) {}
	FBehaviorTreeParallelTask(const UBTTaskNode* InTaskNode, EBTTaskStatus::Type InStatus) : TaskNode(InTaskNode), Status(InStatus) {}

	bool operator==(const FBehaviorTreeParallelTask& Other) const { return TaskNode == Other.TaskNode; }
	bool operator==(const UBTTaskNode* OtherTask) const { return TaskNode == OtherTask; }
};

namespace EBTExecutionSnap
{
	enum Type
	{
		Regular,
		OutOfNodes,
	};
}

namespace EBTDescriptionVerbosity
{
	enum Type
	{
		Basic,
		Detailed,
	};
}

enum class EBTNodeRelativePriority : uint8
{
	Lower,
	Same,
	Higher
};

/** debugger data about subtree instance */
struct FBehaviorTreeDebuggerInstance
{
	struct FNodeFlowData
	{
		uint16 ExecutionIndex;
		uint16 bPassed : 1;
		uint16 bTrigger : 1;
		uint16 bDiscardedTrigger : 1;

		FNodeFlowData() : ExecutionIndex(INDEX_NONE), bPassed(0), bTrigger(0), bDiscardedTrigger(0) {}
	};

	FBehaviorTreeDebuggerInstance() : TreeAsset(nullptr), RootNode(nullptr) {}

	/** behavior tree asset */
	UBehaviorTree* TreeAsset;

	/** root node in template */
	TWeakObjectPtr<UBTCompositeNode> RootNode;

	/** execution indices of active nodes */
	TArray<uint16> ActivePath;

	/** execution indices of active nodes */
	TArray<uint16> AdditionalActiveNodes;

	/** search flow from previous state */
	TArray<FNodeFlowData> PathFromPrevious;

	/** runtime descriptions for each execution index */
	TArray<FString> RuntimeDesc;

	inline bool IsValid() const { return ActivePath.Num() != 0; }
};

/** debugger data about current execution step */
struct FBehaviorTreeExecutionStep
{
	FBehaviorTreeExecutionStep() : TimeStamp(0.), ExecutionStepId(InvalidExecutionId) {}

	/** subtree instance stack */
	TArray<FBehaviorTreeDebuggerInstance> InstanceStack;

	/** blackboard snapshot: value descriptions */
	TMap<FName, FString> BlackboardValues;

	/** Game world's time stamp of this step */
	double TimeStamp;

	inline static constexpr int32 InvalidExecutionId = -1;

	/** Id of execution step */
	int32 ExecutionStepId;

	/** If true, the behavior was paused in this execution step. */
	bool bIsExecutionPaused = false;
};

/** identifier of subtree instance */
struct FBehaviorTreeInstanceId
{
	/** behavior tree asset */
	TObjectPtr<UBehaviorTree> TreeAsset;

	/** root node in template for cleanup purposes */
	TObjectPtr<UBTCompositeNode> RootNode;

	/** execution index path from root */
	TArray<uint16> Path;

	/** persistent instance memory */
	TArray<uint8> InstanceMemory;

	/** index of first node instance (BehaviorTreeComponent.NodeInstances) */
	int32 FirstNodeInstance;

	FBehaviorTreeInstanceId() : TreeAsset(0), RootNode(0), FirstNodeInstance(-1) {}

	bool operator==(const FBehaviorTreeInstanceId& Other) const
	{
		return (TreeAsset == Other.TreeAsset) && (Path == Other.Path);
	}
};

struct FBehaviorTreeSearchData;
DECLARE_DELEGATE_TwoParams(FBTInstanceDeactivation, UBehaviorTreeComponent&, EBTNodeResult::Type);

/** data required for instance of single subtree */
struct FBehaviorTreeInstance
{
	/** root node in template */
	TObjectPtr<UBTCompositeNode> RootNode;

	/** active node in template */
	TObjectPtr<UBTNode> ActiveNode;

	/** active auxiliary nodes */
	TArray<TObjectPtr<UBTAuxiliaryNode>> ActiveAuxNodes;

	/** active parallel tasks */
	TArray<FBehaviorTreeParallelTask> ParallelTasks;

	/** memory: instance */
	TArray<uint8> InstanceMemory;

	/** index of identifier (BehaviorTreeComponent.KnownInstances) */
	uint8 InstanceIdIndex;

	/** active node type */
	TEnumAsByte<EBTActiveNode::Type> ActiveNodeType;

	/** delegate sending a notify when tree instance is removed from active stack */
	FBTInstanceDeactivation DeactivationNotify;

	AIMODULE_API FBehaviorTreeInstance();

	UE_DEPRECATED(5.2, "Copying FBehaviorTreeInstance constructor has been deprecated in favor of move-constructor")
	AIMODULE_API FBehaviorTreeInstance(const FBehaviorTreeInstance& Other);

	UE_DEPRECATED(5.2, "Copying FBehaviorTreeInstance assignement operator has been deprecated in favor of move assignement operator")
	FBehaviorTreeInstance& operator=(const FBehaviorTreeInstance& Other) = default;

	AIMODULE_API FBehaviorTreeInstance(FBehaviorTreeInstance&& Other);
	AIMODULE_API FBehaviorTreeInstance& operator=(FBehaviorTreeInstance&& Other);

	AIMODULE_API FBehaviorTreeInstance(int32 MemorySize);
	AIMODULE_API ~FBehaviorTreeInstance();

#if STATS
	void IncMemoryStats() const;
	void DecMemoryStats() const;
	uint32 GetAllocatedSize() const;
#else
	inline uint32 GetAllocatedSize() const { return 0; }
	inline void IncMemoryStats() const {}
	inline void DecMemoryStats() const {}
#endif // STATS

	/** initialize memory and create node instances */
	void Initialize(UBehaviorTreeComponent& OwnerComp, UBTCompositeNode& Node, int32& InstancedIndex, EBTMemoryInit::Type InitType);

	/** cleanup node instances */
	void Cleanup(UBehaviorTreeComponent& OwnerComp, EBTMemoryClear::Type CleanupType);

	/** check if instance has active node with given execution index */
	bool HasActiveNode(uint16 TestExecutionIndex) const;

	/** deactivate all active aux nodes and remove their requests from SearchData */
	void DeactivateNodes(FBehaviorTreeSearchData& SearchData, uint16 InstanceIndex);

	/** get list of all active auxiliary nodes */
	TArrayView<UBTAuxiliaryNode* const> GetActiveAuxNodes() const { return ActiveAuxNodes; }

	UE_DEPRECATED(5.4, "This function is deprecated. Please use version that takes UBehaviorTreeComponent.")
	void AddToActiveAuxNodes(UBTAuxiliaryNode* AuxNode);

	/** add specified node to the active nodes list */
	void AddToActiveAuxNodes(UBehaviorTreeComponent& OwnerComp, UBTAuxiliaryNode* AuxNode);

	UE_DEPRECATED(5.4, "This function is deprecated. Please use version that takes UBehaviorTreeComponent.")
	void RemoveFromActiveAuxNodes(UBTAuxiliaryNode* AuxNode);

	/** remove specified node from the active nodes list */
	void RemoveFromActiveAuxNodes(UBehaviorTreeComponent& OwnerComp, UBTAuxiliaryNode* AuxNode);

	/** remove all auxiliary nodes from active nodes list */
	void ResetActiveAuxNodes();

	/** iterate on auxiliary nodes and call ExecFunc on each of them. Nodes can not be added or removed during the iteration */
	void ExecuteOnEachAuxNode(TFunctionRef<void(const UBTAuxiliaryNode&)> ExecFunc);

	/** get list of all active parallel tasks */
	TArrayView<const FBehaviorTreeParallelTask> GetParallelTasks() const { return ParallelTasks; }

	/** add new parallel task */
	void AddToParallelTasks(FBehaviorTreeParallelTask&& ParallelTask);

	/** remove parallel task at given index */
	void RemoveParallelTaskAt(int32 TaskIndex);

	/** mark parallel task at given index as pending abort */
	void MarkParallelTaskAsAbortingAt(int32 TaskIndex);

	/** indicates if the provided index is a valid parallel task index */
	bool IsValidParallelTaskIndex(const int32 Index) const { return ParallelTasks.IsValidIndex(Index); }

	/** iterate on parallel tasks and call ExecFunc on each of them. Supports removing the iterated task while processed */
	void ExecuteOnEachParallelTask(TFunctionRef<void(const FBehaviorTreeParallelTask&, const int32)> ExecFunc);

	/** set instance memory */
	void SetInstanceMemory(const TArray<uint8>& Memory);

	/** get instance memory */
	TArrayView<const uint8> GetInstanceMemory() const { return InstanceMemory; }

protected:

	/** worker for updating all nodes */
	void CleanupNodes(UBehaviorTreeComponent& OwnerComp, UBTCompositeNode& Node, EBTMemoryClear::Type CleanupType);

private:
	void AddToActiveAuxNodesImpl(UBTAuxiliaryNode* AuxNode);
	void RemoveFromActiveAuxNodesImpl(UBTAuxiliaryNode* AuxNode);

private:
#if DO_ENSURE
	/** debug flag to detect modifications to the array of nodes while iterating through it */
	bool bIteratingNodes = false;

	/**
	 * debug flag to detect forbidden modifications to the array of parallel tasks while iterating through it
	 * the only allowed modification is to unregister the task on which the exec function is executed
	 * @see ExecuteOnEachParallelTask
	 */
	int32 ParallelTaskIndex = INDEX_NONE;
#endif // DO_ENSURE
};

struct FBTNodeIndex
{
	static constexpr uint16 InvalidIndex = TNumericLimits<uint16>::Max(); // (This is also the same as INDEX_NONE assigned to IndexType!)

	/** index of instance of stack */
	uint16 InstanceIndex;

	/** execution index within instance */
	uint16 ExecutionIndex;

	FBTNodeIndex() : InstanceIndex(InvalidIndex), ExecutionIndex(InvalidIndex) {}
	FBTNodeIndex(uint16 InInstanceIndex, uint16 InExecutionIndex) : InstanceIndex(InInstanceIndex), ExecutionIndex(InExecutionIndex) {}

	bool TakesPriorityOver(const FBTNodeIndex& Other) const;
	bool IsSet() const { return InstanceIndex < InvalidIndex; }

	inline bool operator==(const FBTNodeIndex& Other) const { return Other.ExecutionIndex == ExecutionIndex && Other.InstanceIndex == InstanceIndex; }
	inline bool operator!=(const FBTNodeIndex& Other) const { return !operator==(Other); }
	inline friend uint32 GetTypeHash(const FBTNodeIndex& Other) { return Other.ExecutionIndex ^ Other.InstanceIndex; }

	inline FString Describe() const { return FString::Printf(TEXT("[%d:%d]"), InstanceIndex, ExecutionIndex); }
};

struct FBTNodeIndexRange
{
	/** first node index */
	FBTNodeIndex FromIndex;

	/** last node index */
	FBTNodeIndex ToIndex;

	FBTNodeIndexRange(const FBTNodeIndex& From, const FBTNodeIndex& To) : FromIndex(From), ToIndex(To) {}

	bool IsSet() const { return FromIndex.IsSet() && ToIndex.IsSet(); }

	bool operator==(const FBTNodeIndexRange& Other) const { return Other.FromIndex == FromIndex && Other.ToIndex == ToIndex; }
	bool operator!=(const FBTNodeIndexRange& Other) const { return !operator==(Other); }

	bool Contains(const FBTNodeIndex& Index) const
	{ 
		return Index.InstanceIndex == FromIndex.InstanceIndex && FromIndex.ExecutionIndex <= Index.ExecutionIndex && Index.ExecutionIndex <= ToIndex.ExecutionIndex;
	}

	FString Describe() const { return FString::Printf(TEXT("[%s...%s]"), *FromIndex.Describe(), *ToIndex.Describe()); }
};

/** node update data */
struct FBehaviorTreeSearchUpdate
{
	TObjectPtr<UBTAuxiliaryNode> AuxNode;
	TObjectPtr<UBTTaskNode> TaskNode;

	uint16 InstanceIndex;

	TEnumAsByte<EBTNodeUpdateMode::Type> Mode;

	/** if set, this entry will be applied AFTER other are processed */
	uint8 bPostUpdate : 1;

	/** if set the action was unnecessary (Remove and was already inactive, Add and was already active) */
	mutable uint8 bApplySkipped : 1;

	FBehaviorTreeSearchUpdate() 
		: AuxNode(0), TaskNode(0), InstanceIndex(0), Mode(EBTNodeUpdateMode::Unknown), bPostUpdate(false), bApplySkipped(false)
	{}
	FBehaviorTreeSearchUpdate(const UBTAuxiliaryNode* InAuxNode, uint16 InInstanceIndex, EBTNodeUpdateMode::Type InMode)
		: AuxNode((UBTAuxiliaryNode*)InAuxNode), TaskNode(0), InstanceIndex(InInstanceIndex), Mode(InMode), bPostUpdate(false), bApplySkipped(false)
	{}
	FBehaviorTreeSearchUpdate(const UBTTaskNode* InTaskNode, uint16 InInstanceIndex, EBTNodeUpdateMode::Type InMode)
		: AuxNode(0), TaskNode((UBTTaskNode*)InTaskNode), InstanceIndex(InInstanceIndex), Mode(InMode), bPostUpdate(false), bApplySkipped(false)
	{}
};

/** instance notify data */
struct FBehaviorTreeSearchUpdateNotify
{
	uint16 InstanceIndex;
	TEnumAsByte<EBTNodeResult::Type> NodeResult;

	FBehaviorTreeSearchUpdateNotify() : InstanceIndex(0), NodeResult(EBTNodeResult::Succeeded) {}
	FBehaviorTreeSearchUpdateNotify(uint16 InInstanceIndex, EBTNodeResult::Type InNodeResult) : InstanceIndex(InInstanceIndex), NodeResult(InNodeResult) {}
};

/** node search data */
struct FBehaviorTreeSearchData
{
	/** BT component */
	UBehaviorTreeComponent& OwnerComp;

	/** requested updates of additional nodes (preconditions, services, parallels)
	 *  buffered during search to prevent instant add & remove pairs */
	TArray<FBehaviorTreeSearchUpdate> PendingUpdates;

	/** notifies for tree instances */
	TArray<FBehaviorTreeSearchUpdateNotify> PendingNotifies;

	/** node under which the search was performed */
	FBTNodeIndex SearchRootNode;

	/** first node allowed in search */
	FBTNodeIndex SearchStart;

	/** last node allowed in search */
	FBTNodeIndex SearchEnd;

	/** search unique number */
	int32 SearchId;

	/** active instance index to rollback to */
	int32 RollbackInstanceIdx;

	/** start index of the deactivated branch */
	FBTNodeIndex DeactivatedBranchStart;

	/** end index of the deactivated branch */
	FBTNodeIndex DeactivatedBranchEnd;

	/** saved start index of the deactivated branch for rollback */
	FBTNodeIndex RollbackDeactivatedBranchStart;

	/** saved end index of the deactivated branch for rollback */
	FBTNodeIndex RollbackDeactivatedBranchEnd;

	/** if set, execution request from node in the deactivated branch will be skipped */
	uint32 bFilterOutRequestFromDeactivatedBranch : 1;

	/** if set, current search will be restarted in next tick */
	uint32 bPostponeSearch : 1;

	/** set when task search is in progress */
	uint32 bSearchInProgress : 1;

	/** if set, active node state/memory won't be rolled back */
	uint32 bPreserveActiveNodeMemoryOnRollback : 1;

	/** adds update info to PendingUpdates array, removing all previous updates for this node */
	void AddUniqueUpdate(const FBehaviorTreeSearchUpdate& UpdateInfo);

	/** assign unique Id number */
	void AssignSearchId();

	/** clear state of search */
	void Reset();

	FBehaviorTreeSearchData(UBehaviorTreeComponent& InOwnerComp) 
		: OwnerComp(InOwnerComp), RollbackInstanceIdx(INDEX_NONE)
		, bFilterOutRequestFromDeactivatedBranch(false)
		, bPostponeSearch(false)
		, bSearchInProgress(false)
		, bPreserveActiveNodeMemoryOnRollback(false)
	{}

	FBehaviorTreeSearchData() = delete;

private:

	static int32 NextSearchId;
};

/** property block in blueprint defined nodes */
struct FBehaviorTreePropertyMemory
{
	uint16 Offset;
	uint16 BlockSize;
	
	FBehaviorTreePropertyMemory() {}
	FBehaviorTreePropertyMemory(int32 Value) : Offset((uint32)Value >> 16), BlockSize((uint32)Value & 0xFFFF) {}

	int32 Pack() const { return (int32)(((uint32)Offset << 16) | BlockSize); }
};

/** helper struct for defining types of allowed blackboard entries
 *  (e.g. only entries holding points and objects derived form actor class) */
USTRUCT(BlueprintType)
struct FBlackboardKeySelector
{
	GENERATED_USTRUCT_BODY()

	FBlackboardKeySelector() : SelectedKeyID(FBlackboard::InvalidKey), bNoneIsAllowedValue(false)
	{}

	/** array of allowed types with additional properties (e.g. uobject's base class) 
	  * EditAnywhere is required for FBlackboardSelectorDetails::CacheBlackboardData() */
	UPROPERTY(transient, EditAnywhere, BlueprintReadWrite, Category = Blackboard)
	TArray<TObjectPtr<UBlackboardKeyType>> AllowedTypes;

	/** name of selected key */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = Blackboard)
	FName SelectedKeyName;

	/** class of selected key  */
	UPROPERTY(transient, EditInstanceOnly, BlueprintReadWrite, Category = Blackboard)
	TSubclassOf<UBlackboardKeyType> SelectedKeyType;

protected:
	/** ID of selected key */
	UPROPERTY(transient, EditInstanceOnly, BlueprintReadWrite, Category = Blackboard, meta = (ClampMin = "0", UIMin = "0"))
	int32 SelectedKeyID;

	// Requires BlueprintReadWrite so that blueprint creators (using MakeBlackboardKeySelector) can specify whether or not None is Allowed.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Blackboard, Meta = (Tooltip = ""))
	uint32 bNoneIsAllowedValue:1;

	/** find initial selection. Called when None is not a valid option for this key selector */
	AIMODULE_API void InitSelection(const UBlackboardData& BlackboardAsset);

public:
	/** find ID and class of selected key */
	AIMODULE_API void ResolveSelectedKey(const UBlackboardData& BlackboardAsset);
		
	void AllowNoneAsValue(bool bAllow) { bNoneIsAllowedValue = bAllow; }

	inline FBlackboard::FKey GetSelectedKeyID() const { return FBlackboard::FKey(IntCastChecked<uint16>(SelectedKeyID)); }

	/** helper functions for setting basic filters */
	AIMODULE_API void AddObjectFilter(UObject* Owner, FName PropertyName, TSubclassOf<UObject> AllowedClass);
	AIMODULE_API void AddClassFilter(UObject* Owner, FName PropertyName, TSubclassOf<UObject> AllowedClass);
	AIMODULE_API void AddEnumFilter(UObject* Owner, FName PropertyName, UEnum* AllowedEnum);
	AIMODULE_API void AddNativeEnumFilter(UObject* Owner, FName PropertyName, const FString& AllowedEnumName);
	AIMODULE_API void AddIntFilter(UObject* Owner, FName PropertyName);
	AIMODULE_API void AddFloatFilter(UObject* Owner, FName PropertyName);
	AIMODULE_API void AddBoolFilter(UObject* Owner, FName PropertyName);
	AIMODULE_API void AddVectorFilter(UObject* Owner, FName PropertyName);
	AIMODULE_API void AddRotatorFilter(UObject* Owner, FName PropertyName);
	AIMODULE_API void AddStringFilter(UObject* Owner, FName PropertyName);
	AIMODULE_API void AddNameFilter(UObject* Owner, FName PropertyName);
	AIMODULE_API void AddStructFilter(UObject* Owner, FName PropertyName, const UScriptStruct* AllowedStruct);

	template<typename T>
	void AddStructFilter(UObject* Owner, FName PropertyName)
	{
		AddStructFilter(Owner, PropertyName, TBaseStructure<T>::Get());
	}

	inline bool IsNone() const { return bNoneIsAllowedValue && GetSelectedKeyID() == FBlackboard::InvalidKey; }
	inline bool IsSet() const { return GetSelectedKeyID() != FBlackboard::InvalidKey; }
	inline bool NeedsResolving() const { return GetSelectedKeyID() == FBlackboard::InvalidKey && SelectedKeyName.IsNone() == false; }
	inline void InvalidateResolvedKey() { SelectedKeyID = FBlackboard::InvalidKey; }

#if WITH_EDITOR
	void PreSave(const UObject* Outer, const UBlackboardData& Blackboard, FName PropertyName);
#endif // WITH_EDITOR

	friend FBlackboardDecoratorDetails;
};

UCLASS(Abstract, MinimalAPI)
class UBehaviorTreeTypes : public UObject
{
	GENERATED_BODY()

	static AIMODULE_API FString BTLoggingContext;

public:

	static AIMODULE_API FString DescribeNodeHelper(const UBTNode* Node);

	static AIMODULE_API FString DescribeNodeResult(EBTNodeResult::Type NodeResult);
	static AIMODULE_API FString DescribeFlowAbortMode(EBTFlowAbortMode::Type FlowAbortMode);
	static AIMODULE_API FString DescribeActiveNode(EBTActiveNode::Type ActiveNodeType);
	static AIMODULE_API FString DescribeTaskStatus(EBTTaskStatus::Type TaskStatus);
	static AIMODULE_API FString DescribeNodeUpdateMode(EBTNodeUpdateMode::Type UpdateMode);

	/** returns short name of object's class (BTTaskNode_Wait -> Wait) */
	static AIMODULE_API FString GetShortTypeName(const UObject* Ob);
	
	static FString GetBTLoggingContext() { return BTLoggingContext; }
	
	// @param NewBTLoggingContext the object which name's will be added to some of the BT logging
	// 	pass nullptr to clear
	static AIMODULE_API void SetBTLoggingContext(const UBTNode* NewBTLoggingContext);
};

/** Helper struct to push a node as the new logging context and automatically reset the context on destruction. */
struct FScopedBTLoggingContext
{
	FScopedBTLoggingContext() = delete;
	explicit FScopedBTLoggingContext(const UBTNode* Context)
	{
		UBehaviorTreeTypes::SetBTLoggingContext(Context);
	}

	~FScopedBTLoggingContext()
	{
		UBehaviorTreeTypes::SetBTLoggingContext(nullptr);
	}
};
