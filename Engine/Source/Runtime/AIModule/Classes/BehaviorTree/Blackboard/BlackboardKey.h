// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

namespace FBlackboard
{
	extern AIMODULE_API const FName KeySelf;

#ifdef AI_BLACKBOARD_KEY_SIZE_8
	// this is the legacy BB key size. Add AI_BLACKBOARD_KEY_SIZE_8 to your *.target.cs to enable it.
	using FKey = uint8;
	inline constexpr FKey InvalidKey = FKey(-1);
#else
	//the default BB key size is now 16
	struct FKey
	{
		constexpr FKey() = default;
		FKey(int32 InKey) {Key = IntCastChecked<uint16>(InKey); }
		constexpr FKey(uint16 InKey) : Key(InKey) {}
		constexpr FKey(uint8 InKey) = delete;
		constexpr operator int32() const { return Key; }
		constexpr operator uint16() const { return Key; }
		constexpr operator uint8() const = delete;
		constexpr bool operator==(const FKey& Other) const {return Key == Other.Key;}
		constexpr bool operator!=(const FKey& Other) const {return Key != Other.Key;}
	private:
		friend uint32 GetTypeHash(const FKey& Key);
		uint16 Key = static_cast<uint16>(-1);
	};

	inline constexpr FKey InvalidKey = FKey();

	inline uint32 GetTypeHash(const FKey& Key) { return ::GetTypeHash(Key.Key);}
#endif
}
