// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlackboardKeyEnums.generated.h"

namespace EBlackboardCompare
{
	enum Type
	{
		Less = -1, 	
		Equal = 0, 
		Greater = 1,

		NotEqual = 1,	// important, do not change
	};
}

namespace EBlackboardKeyOperation
{
	enum Type
	{
		Basic,
		Arithmetic,
		Text,
	};
}

UENUM()
namespace EBasicKeyOperation
{
	enum Type : int
	{
		Set				UMETA(DisplayName="Is Set"),
		NotSet			UMETA(DisplayName="Is Not Set"),
	};
}

UENUM()
namespace EArithmeticKeyOperation
{
	enum Type : int
	{
		Equal			UMETA(DisplayName="Is Equal To"),
		NotEqual		UMETA(DisplayName="Is Not Equal To"),
		Less			UMETA(DisplayName="Is Less Than"),
		LessOrEqual		UMETA(DisplayName="Is Less Than Or Equal To"),
		Greater			UMETA(DisplayName="Is Greater Than"),
		GreaterOrEqual	UMETA(DisplayName="Is Greater Than Or Equal To"),
	};
}

UENUM()
namespace ETextKeyOperation
{
	enum Type : int
	{
		Equal			UMETA(DisplayName="Is Equal To"),
		NotEqual		UMETA(DisplayName="Is Not Equal To"),
		Contain			UMETA(DisplayName="Contains"),
		NotContain		UMETA(DisplayName="Not Contains"),
	};
}
