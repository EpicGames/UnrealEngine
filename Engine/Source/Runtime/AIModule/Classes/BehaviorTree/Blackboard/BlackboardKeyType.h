// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "InputCoreTypes.h"
#include "BehaviorTree/BehaviorTreeTypes.h"
#include "BlackboardKeyEnums.h"
#include "BlackboardKeyType.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogBlackboard, Log, All);

class UBlackboardComponent;

struct FBlackboardInstancedKeyMemory
{
	/** index of instanced key in UBlackboardComponent::InstancedKeys */
	int32 KeyIdx;
};

UCLASS(EditInlineNew, Abstract, CollapseCategories, AutoExpandCategories=(Blackboard), MinimalAPI)
class UBlackboardKeyType : public UObject
{
	GENERATED_UCLASS_BODY()

	/** handle dynamic data size */
	AIMODULE_API virtual void PreInitialize(UBlackboardComponent& OwnerComp);

	/** handle instancing if needed */
	AIMODULE_API void InitializeKey(UBlackboardComponent& OwnerComp, FBlackboard::FKey KeyID);

	/** does it match settings in filter? */
	AIMODULE_API virtual bool IsAllowedByFilter(UBlackboardKeyType* FilterOb) const;

	/** extract location from entry, supports instanced keys */
	AIMODULE_API bool WrappedGetLocation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, FVector& Location) const;

	/** extract rotation from entry, supports instanced keys */
	AIMODULE_API bool WrappedGetRotation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, FRotator& Rotation) const;

	/** free value before removing from blackboard, supports instanced keys */
	AIMODULE_API void WrappedFree(UBlackboardComponent& OwnerComp, uint8* MemoryBlock);

	/** sets value to the default, supports instanced keys */
	AIMODULE_API void WrappedClear(const UBlackboardComponent& OwnerComp, uint8* MemoryBlock) const;

	/** check if key has stored value, supports instanced keys */
	AIMODULE_API bool WrappedIsEmpty(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock) const;

	/** various value testing, used by decorators, supports instanced keys */
	AIMODULE_API bool WrappedTestBasicOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, EBasicKeyOperation::Type Op) const;
	AIMODULE_API bool WrappedTestArithmeticOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, EArithmeticKeyOperation::Type Op, int32 OtherIntValue, float OtherFloatValue) const;
	AIMODULE_API bool WrappedTestTextOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, ETextKeyOperation::Type Op, const FString& OtherString) const;

	/** describe params of arithmetic test */
	AIMODULE_API virtual FString DescribeArithmeticParam(int32 IntValue, float FloatValue) const;

	/** convert value to text, supports instanced keys */
	AIMODULE_API FString WrappedDescribeValue(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock) const;

	/** description of params for property view */
	AIMODULE_API virtual FString DescribeSelf() const;

	/** create replacement key for deprecated data */
	AIMODULE_API virtual UBlackboardKeyType* UpdateDeprecatedKey();

	/** @return key instance if bCreateKeyInstance was set */
	AIMODULE_API const UBlackboardKeyType* GetKeyInstance(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock) const;
	AIMODULE_API UBlackboardKeyType* GetKeyInstance(UBlackboardComponent& OwnerComp, const uint8* MemoryBlock) const;

	/** compares two values */
	AIMODULE_API virtual EBlackboardCompare::Type CompareValues(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock,
		const UBlackboardKeyType* OtherKeyOb, const uint8* OtherMemoryBlock) const;

	/** @return true if key wants to be instanced */
	bool HasInstance() const;

	/** @return true if this object is instanced key */
	bool IsInstanced() const;

	/** get ValueSize */
	uint16 GetValueSize() const;

	/** get test supported by this type */
	EBlackboardKeyOperation::Type GetTestOperation() const;

protected:

	/** size of value for this type */
	uint16 ValueSize;

	/** decorator operation supported with this type */
	TEnumAsByte<EBlackboardKeyOperation::Type> SupportedOp;

	/** set automatically for node instances */
	uint8 bIsInstanced : 1;

	/** if set, key will be instanced instead of using memory block */
	uint8 bCreateKeyInstance : 1;

	/** helper function for reading typed data from memory block */
	template<typename T>
	static T GetValueFromMemory(const uint8* MemoryBlock)
	{
		return *((T*)MemoryBlock);
	}

	/** helper function for writing typed data to memory block, returns true if value has changed */
	template<typename T>
	static bool SetValueInMemory(uint8* MemoryBlock, const T& Value)
	{
		const bool bChanged = *((T*)MemoryBlock) != Value;
		*((T*)MemoryBlock) = Value;
		
		return bChanged;
	}

	/** helper function for witting weak object data to memory block, returns true if value has changed */
	template<typename T>
	static bool SetWeakObjectInMemory(uint8* MemoryBlock, const TWeakObjectPtr<T>& Value)
	{
		TWeakObjectPtr<T>* PrevValue = (TWeakObjectPtr<T>*)MemoryBlock;
		const bool bChanged =
			(Value.IsValid(false, true) != PrevValue->IsValid(false, true)) ||
			(Value.IsStale(false, true) != PrevValue->IsStale(false, true)) ||
			(*PrevValue) != Value;

		*((TWeakObjectPtr<T>*)MemoryBlock) = Value;

		return bChanged;
	}

	friend UBlackboardComponent;
	
	/** copy value from other key, works directly on provided memory/properties */
	AIMODULE_API virtual void CopyValues(UBlackboardComponent& OwnerComp, uint8* MemoryBlock, const UBlackboardKeyType* SourceKeyOb, const uint8* SourceBlock);

	/** initialize memory, works directly on provided memory/properties */
	AIMODULE_API virtual void InitializeMemory(UBlackboardComponent& OwnerComp, uint8* MemoryBlock);

	/** free value before removing from blackboard, works directly on provided memory/properties */
	AIMODULE_API virtual void FreeMemory(UBlackboardComponent& OwnerComp, uint8* MemoryBlock);

	/** extract location from entry, works directly on provided memory/properties */
	AIMODULE_API virtual bool GetLocation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, FVector& Location) const;
	
	/** extract rotation from entry, works directly on provided memory/properties */
	AIMODULE_API virtual bool GetRotation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, FRotator& Rotation) const;

	/** sets value to the default, works directly on provided memory/properties */
	AIMODULE_API virtual void Clear(UBlackboardComponent& OwnerComp, uint8* MemoryBlock);

	/** check if key has stored value, works directly on provided memory/properties */
	AIMODULE_API virtual bool IsEmpty(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock) const;

	/** various value testing, works directly on provided memory/properties */
	AIMODULE_API virtual bool TestBasicOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, EBasicKeyOperation::Type Op) const;
	AIMODULE_API virtual bool TestArithmeticOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, EArithmeticKeyOperation::Type Op, int32 OtherIntValue, float OtherFloatValue) const;
	AIMODULE_API virtual bool TestTextOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, ETextKeyOperation::Type Op, const FString& OtherString) const;

	/** convert value to text, works directly on provided memory/properties */
	AIMODULE_API virtual FString DescribeValue(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock) const;
};

//////////////////////////////////////////////////////////////////////////
// Inlines

inline uint16 UBlackboardKeyType::GetValueSize() const
{
	return ValueSize;
}

inline EBlackboardKeyOperation::Type UBlackboardKeyType::GetTestOperation() const
{
	return SupportedOp; 
}

inline bool UBlackboardKeyType::HasInstance() const
{
	return bCreateKeyInstance;
}

inline bool UBlackboardKeyType::IsInstanced() const
{
	return bIsInstanced;
}
