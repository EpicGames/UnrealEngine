// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType.h"
#include "BlackboardKeyType_Bool.generated.h"

class UBlackboardComponent;

UCLASS(EditInlineNew, meta=(DisplayName="Bool"), MinimalAPI)
class UBlackboardKeyType_Bool : public UBlackboardKeyType
{
	GENERATED_UCLASS_BODY()

	typedef bool FDataType;
	static AIMODULE_API const FDataType InvalidValue;

	static AIMODULE_API bool GetValue(const UBlackboardKeyType_Bool* KeyOb, const uint8* RawData);
	static AIMODULE_API bool SetValue(UBlackboardKeyType_Bool* KeyOb, uint8* RawData, bool bValue);

	AIMODULE_API virtual EBlackboardCompare::Type CompareValues(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock,
		const UBlackboardKeyType* OtherKeyOb, const uint8* OtherMemoryBlock) const override;

	UPROPERTY(Category = Blackboard, EditDefaultsOnly)
	bool bDefaultValue = InvalidValue;

protected:
	AIMODULE_API virtual void InitializeMemory(UBlackboardComponent& OwnerComp, uint8* MemoryBlock) override;
	AIMODULE_API virtual FString DescribeValue(const UBlackboardComponent& OwnerComp, const uint8* RawData) const override;
	AIMODULE_API virtual bool TestBasicOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, EBasicKeyOperation::Type Op) const override;
};
