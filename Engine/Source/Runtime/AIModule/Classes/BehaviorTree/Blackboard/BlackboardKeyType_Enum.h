// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType.h"
#include "BlackboardKeyType_Enum.generated.h"

class UBlackboardComponent;

UCLASS(EditInlineNew, meta=(DisplayName="Enum"), MinimalAPI)
class UBlackboardKeyType_Enum : public UBlackboardKeyType
{
	GENERATED_UCLASS_BODY()

	typedef uint8 FDataType;
	static AIMODULE_API const FDataType InvalidValue;

	UPROPERTY(Category=Blackboard, EditDefaultsOnly)
	TObjectPtr<UEnum> EnumType;

	/** name of enum defined in c++ code, will take priority over asset from EnumType property */
	UPROPERTY(Category=Blackboard, EditDefaultsOnly)
	FString EnumName;

	UPROPERTY(Category=Blackboard, EditDefaultsOnly)
	uint8 DefaultValue = 0;

	/** set when EnumName override is valid and active */
	UPROPERTY(Category = Blackboard, VisibleDefaultsOnly)
	uint32 bIsEnumNameValid : 1;

	static AIMODULE_API FDataType GetValue(const UBlackboardKeyType_Enum* KeyOb, const uint8* RawData);
	static AIMODULE_API bool SetValue(UBlackboardKeyType_Enum* KeyOb, uint8* RawData, FDataType Value);

	AIMODULE_API virtual EBlackboardCompare::Type CompareValues(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock,
		const UBlackboardKeyType* OtherKeyOb, const uint8* OtherMemoryBlock) const override;

	AIMODULE_API virtual FString DescribeSelf() const override;
	AIMODULE_API virtual FString DescribeArithmeticParam(int32 IntValue, float FloatValue) const override;
	AIMODULE_API virtual bool IsAllowedByFilter(UBlackboardKeyType* FilterOb) const override;

	AIMODULE_API virtual void InitializeMemory(UBlackboardComponent& OwnerComp, uint8* MemoryBlock) override;

protected:
#if WITH_EDITOR
	AIMODULE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	static AIMODULE_API bool ValidateEnum(const UEnum& EnumType);
#endif

	AIMODULE_API virtual FString DescribeValue(const UBlackboardComponent& OwnerComp, const uint8* RawData) const override;
	AIMODULE_API virtual bool TestArithmeticOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, EArithmeticKeyOperation::Type Op, int32 OtherIntValue, float OtherFloatValue) const override;
};
