// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType.h"
#include "BlackboardKeyType_Float.generated.h"

class UBlackboardComponent;

UCLASS(EditInlineNew, meta=(DisplayName="Float"), MinimalAPI)
class UBlackboardKeyType_Float : public UBlackboardKeyType
{
	GENERATED_UCLASS_BODY()

	typedef float FDataType;
	static AIMODULE_API const FDataType InvalidValue;

	static AIMODULE_API float GetValue(const UBlackboardKeyType_Float* KeyOb, const uint8* RawData);
	static AIMODULE_API bool SetValue(UBlackboardKeyType_Float* KeyOb, uint8* RawData, float Value);

	AIMODULE_API virtual EBlackboardCompare::Type CompareValues(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock,
		const UBlackboardKeyType* OtherKeyOb, const uint8* OtherMemoryBlock) const override;

	AIMODULE_API virtual FString DescribeArithmeticParam(int32 IntValue, float FloatValue) const override;

	UPROPERTY(EditDefaultsOnly, Category=Blackboard)
	float DefaultValue = InvalidValue;

protected:
	AIMODULE_API virtual void InitializeMemory(UBlackboardComponent& OwnerComp, uint8* MemoryBlock) override;
	AIMODULE_API virtual FString DescribeValue(const UBlackboardComponent& OwnerComp, const uint8* RawData) const override;
	AIMODULE_API virtual bool TestArithmeticOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, EArithmeticKeyOperation::Type Op, int32 OtherIntValue, float OtherFloatValue) const override;
};
