// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType.h"
#include "BlackboardKeyType_Name.generated.h"

class UBlackboardComponent;

UCLASS(EditInlineNew, meta=(DisplayName="Name"), MinimalAPI)
class UBlackboardKeyType_Name : public UBlackboardKeyType
{
	GENERATED_UCLASS_BODY()

	typedef FName FDataType;
	static AIMODULE_API const FDataType InvalidValue;

	static AIMODULE_API FName GetValue(const UBlackboardKeyType_Name* KeyOb, const uint8* RawData);
	static AIMODULE_API bool SetValue(UBlackboardKeyType_Name* KeyOb, uint8* RawData, const FName& Value);

	AIMODULE_API virtual EBlackboardCompare::Type CompareValues(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock,
		const UBlackboardKeyType* OtherKeyOb, const uint8* OtherMemoryBlock) const override;

	UPROPERTY(EditDefaultsOnly, Category = Blackboard)
	FName DefaultValue = InvalidValue;

protected:
	AIMODULE_API virtual void InitializeMemory(UBlackboardComponent& OwnerComp, uint8* MemoryBlock) override;
	AIMODULE_API virtual FString DescribeValue(const UBlackboardComponent& OwnerComp, const uint8* RawData) const override;
	AIMODULE_API virtual bool TestTextOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, ETextKeyOperation::Type Op, const FString& OtherString) const override;
};
