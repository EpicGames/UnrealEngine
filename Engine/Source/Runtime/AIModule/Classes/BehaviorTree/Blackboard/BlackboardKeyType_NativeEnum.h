// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType.h"
#include "BlackboardKeyType_NativeEnum.generated.h"

// DEPRECATED, please use UBlackboardKeyType_Enum instead

UCLASS(NotEditInlineNew, HideDropDown, MinimalAPI)
class UBlackboardKeyType_NativeEnum : public UBlackboardKeyType
{
	GENERATED_UCLASS_BODY()

	typedef uint8 FDataType;
	static AIMODULE_API const FDataType InvalidValue;

	UPROPERTY(Category=Blackboard, EditDefaultsOnly)
	FString EnumName;

	UPROPERTY()
	TObjectPtr<UEnum> EnumType;

	AIMODULE_API virtual UBlackboardKeyType* UpdateDeprecatedKey() override;

	static AIMODULE_API uint8 GetValue(const UBlackboardKeyType_NativeEnum* KeyOb, const uint8* RawData);
	static AIMODULE_API bool SetValue(UBlackboardKeyType_NativeEnum* KeyOb, uint8* RawData, uint8 Value);
};
