// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType.h"
#include "BlackboardKeyType_Object.generated.h"

class UBlackboardComponent;

UCLASS(EditInlineNew, meta=(DisplayName="Object"), MinimalAPI)
class UBlackboardKeyType_Object : public UBlackboardKeyType
{
	GENERATED_UCLASS_BODY()

	typedef UObject* FDataType;
	static AIMODULE_API const FDataType InvalidValue;

	UPROPERTY(Category=Blackboard, EditDefaultsOnly, NoClear, meta=(AllowAbstract="1"))
	TObjectPtr<UClass> BaseClass;

	UPROPERTY(Category=Blackboard, EditDefaultsOnly)
	TObjectPtr<UObject> DefaultValue;

	static AIMODULE_API UObject* GetValue(const UBlackboardKeyType_Object* KeyOb, const uint8* RawData);
	static AIMODULE_API bool SetValue(UBlackboardKeyType_Object* KeyOb, uint8* RawData, UObject* Value);

	AIMODULE_API virtual EBlackboardCompare::Type CompareValues(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock,
		const UBlackboardKeyType* OtherKeyOb, const uint8* OtherMemoryBlock) const override;

	AIMODULE_API virtual FString DescribeSelf() const override;
	AIMODULE_API virtual bool IsAllowedByFilter(UBlackboardKeyType* FilterOb) const override;

	AIMODULE_API virtual void InitializeMemory(UBlackboardComponent& OwnerComp, uint8* MemoryBlock) override;

protected:
	AIMODULE_API virtual FString DescribeValue(const UBlackboardComponent& OwnerComp, const uint8* RawData) const override;
	AIMODULE_API virtual bool GetLocation(const UBlackboardComponent& OwnerComp, const uint8* RawData, FVector& Location) const override;
	AIMODULE_API virtual bool GetRotation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, FRotator& Rotation) const override;
	AIMODULE_API virtual bool TestBasicOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, EBasicKeyOperation::Type Op) const override;
};
