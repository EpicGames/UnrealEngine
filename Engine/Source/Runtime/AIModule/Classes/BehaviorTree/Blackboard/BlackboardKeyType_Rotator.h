// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType.h"
#include "BlackboardKeyType_Rotator.generated.h"

class UBlackboardComponent;

UCLASS(EditInlineNew, meta=(DisplayName="Rotator"), MinimalAPI)
class UBlackboardKeyType_Rotator : public UBlackboardKeyType
{
	GENERATED_UCLASS_BODY()
	
	typedef FRotator FDataType; 
	static AIMODULE_API const FDataType InvalidValue;

	static AIMODULE_API FRotator GetValue(const UBlackboardKeyType_Rotator* KeyOb, const uint8* RawData);
	static AIMODULE_API bool SetValue(UBlackboardKeyType_Rotator* KeyOb, uint8* RawData, const FRotator& Value);

	AIMODULE_API virtual EBlackboardCompare::Type CompareValues(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock,
		const UBlackboardKeyType* OtherKeyOb, const uint8* OtherMemoryBlock) const override;

	// If not set, will default to  FAISystem::InvalidRotation
	UPROPERTY(EditDefaultsOnly, Category = Blackboard, meta = (EditCondition = "bUseDefaultValue"))
	FRotator DefaultValue = FRotator::ZeroRotator;

	UPROPERTY()
	bool bUseDefaultValue = false;

protected:
	AIMODULE_API virtual void InitializeMemory(UBlackboardComponent& OwnerComp, uint8* RawData) override;
	AIMODULE_API virtual FString DescribeValue(const UBlackboardComponent& OwnerComp, const uint8* RawData) const override;
	AIMODULE_API virtual bool GetRotation(const UBlackboardComponent& OwnerComp, const uint8* RawData, FRotator& Rotation) const override;
	AIMODULE_API virtual bool IsEmpty(const UBlackboardComponent& OwnerComp, const uint8* RawData) const override;
	AIMODULE_API virtual void Clear(UBlackboardComponent& OwnerComp, uint8* RawData) override;
	AIMODULE_API virtual bool TestBasicOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, EBasicKeyOperation::Type Op) const override;
};
