// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType.h"
#include "BlackboardKeyType_String.generated.h"

class UBlackboardComponent;

UCLASS(EditInlineNew, meta=(DisplayName="String"), MinimalAPI)
class UBlackboardKeyType_String : public UBlackboardKeyType
{
	GENERATED_UCLASS_BODY()

	typedef FString FDataType;
	static AIMODULE_API const FDataType InvalidValue;

	static AIMODULE_API FString GetValue(const UBlackboardKeyType_String* KeyOb, const uint8* RawData);
	static AIMODULE_API bool SetValue(UBlackboardKeyType_String* KeyOb, uint8* RawData, const FString& Value);

	AIMODULE_API virtual EBlackboardCompare::Type CompareValues(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock,
		const UBlackboardKeyType* OtherKeyOb, const uint8* OtherMemoryBlock) const override;

	UPROPERTY()
	FString StringValue;

	UPROPERTY(EditDefaultsOnly, Category = Blackboard)
	FString DefaultValue = InvalidValue;

protected:
	AIMODULE_API virtual void InitializeMemory(UBlackboardComponent& OwnerComp, uint8* MemoryBlock) override;
	AIMODULE_API virtual FString DescribeValue(const UBlackboardComponent& OwnerComp, const uint8* RawData) const override;
	AIMODULE_API virtual bool TestTextOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, ETextKeyOperation::Type Op, const FString& OtherString) const override;
	AIMODULE_API virtual void Clear(UBlackboardComponent& OwnerComp, uint8* MemoryBlock) override;
	AIMODULE_API virtual bool IsEmpty(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock) const override;
	AIMODULE_API virtual void CopyValues(UBlackboardComponent& OwnerComp, uint8* MemoryBlock, const UBlackboardKeyType* SourceKeyOb, const uint8* SourceBlock) override;
};
