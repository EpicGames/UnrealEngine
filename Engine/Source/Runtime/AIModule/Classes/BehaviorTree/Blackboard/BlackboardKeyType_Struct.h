// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BehaviorTree/Blackboard/BlackboardKeyType.h"
#include "StructUtils/InstancedStruct.h"

#include "BlackboardKeyType_Struct.generated.h"

class UBlackboardComponent;

struct FConstStructView;

UCLASS(EditInlineNew, MinimalAPI, meta = (DisplayName = "Struct"))
class UBlackboardKeyType_Struct : public UBlackboardKeyType
{
	GENERATED_BODY()

public:
	using FDataType = FConstStructView;
	static AIMODULE_API const FDataType InvalidValue;

	UBlackboardKeyType_Struct(const FObjectInitializer& ObjectInitializer);

	static AIMODULE_API FConstStructView GetValue(UBlackboardKeyType_Struct* KeyOb, const uint8* RawData);
	static AIMODULE_API bool SetValue(UBlackboardKeyType_Struct* KeyOb, uint8* RawData, FConstStructView Value);

	virtual EBlackboardCompare::Type CompareValues(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, const UBlackboardKeyType* OtherKeyOb, const uint8* OtherMemoryBlock) const override;
	virtual void CopyValues(UBlackboardComponent& OwnerComp, uint8* MemoryBlock, const UBlackboardKeyType* SourceKeyOb, const uint8* SourceBlock) override;

	virtual FString DescribeSelf() const override;
	virtual bool IsAllowedByFilter(UBlackboardKeyType* FilterOb) const override;

	virtual void InitializeMemory(UBlackboardComponent& OwnerComp, uint8* MemoryBlock) override;
	virtual void FreeMemory(UBlackboardComponent& OwnerComp, uint8* MemoryBlock) override;
	virtual void Clear(UBlackboardComponent& OwnerComp, uint8* MemoryBlock) override;

	virtual void PostLoad() override;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	UPROPERTY(Category = Blackboard, EditDefaultsOnly, DisplayName = DefaultValue)
	FInstancedStruct DefaultValue;

protected:
	// Runtime value of the key if it is instanced otherwise is empty.
	UPROPERTY(Transient)
	FInstancedStruct Value;

	void UpdateNeedsInstance();

	virtual FString DescribeValue(const UBlackboardComponent& OwnerComp, const uint8* RawData) const override;
	virtual bool TestBasicOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, EBasicKeyOperation::Type Op) const override;
};
