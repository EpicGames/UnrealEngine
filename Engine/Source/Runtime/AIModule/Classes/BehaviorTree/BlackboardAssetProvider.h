// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "BlackboardAssetProvider.generated.h"


/** Helper interface to allow FBlackboardKeySelector properties on DataAssets (and more).
 *  Used by FBlackboardSelectorDetails to access the related Blackboard based on UObject
 *  hierarchy. The asset containing the Blackboard should broadcast OnBlackboardOwnerChanged
 *  when ever the asset ptr changes. */
UINTERFACE(BlueprintType, MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UBlackboardAssetProvider : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IBlackboardAssetProvider
{
	GENERATED_IINTERFACE_BODY()

#if WITH_EDITOR
	/** Delegate to be called by class implementing IBlackboardAssetProvider when the property containing the returned BlackboardData is changed (i.e. on PostEditChangeProperty). */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FBlackboardOwnerChanged, UObject* /*AssetOwner*/, UBlackboardData* /*Asset*/);
	static AIMODULE_API FBlackboardOwnerChanged OnBlackboardOwnerChanged;
#endif
	/** Returns BlackboardData referenced by the owner object. */
	UFUNCTION(BlueprintCallable, Category = GameplayTags)
	virtual UBlackboardData* GetBlackboardAsset() const PURE_VIRTUAL(IBlackboardAssetProvider::GetBlackboardAsset, return nullptr; );
};


