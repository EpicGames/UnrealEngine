// Copyright Epic Games, Inc. All Rights Reserved.

/** 
 *  Blackboard - holds AI's world knowledge, easily accessible for behavior trees
 *
 *  Properties are stored in byte array, and should be accessed only though
 *  GetValue* / SetValue* functions. They will handle broadcasting change events
 *  for registered observers.
 *
 *  Keys are defined by BlackboardData data asset.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "InputCoreTypes.h"
#include "Templates/SubclassOf.h"
#include "Components/ActorComponent.h"
#include "EngineDefines.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType.h"
#include "AISystem.h"
#include "BehaviorTree/BlackboardData.h"
#include "BlackboardComponent.generated.h"

class UBrainComponent;

namespace EBlackboardDescription
{
	enum Type
	{
		OnlyValue,
		KeyWithValue,
		DetailedKeyWithValue,
		Full,
	};
}


UCLASS(ClassGroup = AI, meta = (BlueprintSpawnableComponent), hidecategories = (Sockets, Collision), MinimalAPI)
class UBlackboardComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	AIMODULE_API UBlackboardComponent(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** BEGIN UActorComponent overrides */
	AIMODULE_API virtual void InitializeComponent() override;
	AIMODULE_API virtual void UninitializeComponent() override;
	/** END UActorComponent overrides */

	/** @return name of key */
	AIMODULE_API FName GetKeyName(FBlackboard::FKey KeyID) const;

	/** @return key ID from name */
	AIMODULE_API FBlackboard::FKey GetKeyID(const FName& KeyName) const;

	/** @return class of value for given key */
	AIMODULE_API TSubclassOf<UBlackboardKeyType> GetKeyType(FBlackboard::FKey KeyID) const;

	/** @return true if the key is marked as instance synced */
	AIMODULE_API bool IsKeyInstanceSynced(FBlackboard::FKey KeyID) const;
	
	/** @return number of entries in data asset */
	AIMODULE_API int32 GetNumKeys() const;

	/** @return true if blackboard have valid data asset */
	bool HasValidAsset() const;

	/** register observer for blackboard key */
	AIMODULE_API FDelegateHandle RegisterObserver(FBlackboard::FKey KeyID, const UObject* NotifyOwner, FOnBlackboardChangeNotification ObserverDelegate);

	/** unregister observer from blackboard key */
	AIMODULE_API void UnregisterObserver(FBlackboard::FKey KeyID, FDelegateHandle ObserverHandle);

	/** unregister all observers associated with given owner */
	AIMODULE_API void UnregisterObserversFrom(const UObject* NotifyOwner);

	/** pause observer change notifications, any new ones will be added to a queue */
	AIMODULE_API void PauseObserverNotifications();

	/** resume observer change notifications and, optionally, process the queued observation list */
	AIMODULE_API void ResumeObserverNotifications(bool bSendQueuedObserverNotifications);

	/** @return associated behavior tree component */
	AIMODULE_API UBrainComponent* GetBrainComponent() const;

	/** @return blackboard data asset */
	AIMODULE_API UBlackboardData* GetBlackboardAsset() const;

	/** caches UBrainComponent pointer to be used in communication */
	AIMODULE_API void CacheBrainComponent(UBrainComponent& BrainComponent);

	/** setup component for using given blackboard asset, returns true if blackboard is properly initialized for specified blackboard data */
	AIMODULE_API bool InitializeBlackboard(UBlackboardData& NewAsset);
	
	/** @return true if component can be used with specified blackboard asset */
	AIMODULE_API virtual bool IsCompatibleWith(const UBlackboardData* TestAsset) const;

	UFUNCTION(BlueprintCallable, Category="AI|Components|Blackboard")
	AIMODULE_API UObject* GetValueAsObject(const FName& KeyName) const;

	UFUNCTION(BlueprintCallable, Category="AI|Components|Blackboard")
	AIMODULE_API UClass* GetValueAsClass(const FName& KeyName) const;

	UFUNCTION(BlueprintCallable, Category="AI|Components|Blackboard")
	AIMODULE_API uint8 GetValueAsEnum(const FName& KeyName) const;

	UFUNCTION(BlueprintCallable, Category="AI|Components|Blackboard")
	AIMODULE_API int32 GetValueAsInt(const FName& KeyName) const;

	UFUNCTION(BlueprintCallable, Category="AI|Components|Blackboard")
	AIMODULE_API float GetValueAsFloat(const FName& KeyName) const;

	UFUNCTION(BlueprintCallable, Category="AI|Components|Blackboard")
	AIMODULE_API bool GetValueAsBool(const FName& KeyName) const;

	UFUNCTION(BlueprintCallable, Category="AI|Components|Blackboard")
	AIMODULE_API FString GetValueAsString(const FName& KeyName) const;
	
	UFUNCTION(BlueprintCallable, Category="AI|Components|Blackboard")
	AIMODULE_API FName GetValueAsName(const FName& KeyName) const;

	UFUNCTION(BlueprintCallable, Category="AI|Components|Blackboard")
	AIMODULE_API FVector GetValueAsVector(const FName& KeyName) const;

	UFUNCTION(BlueprintCallable, Category="AI|Components|Blackboard")
	AIMODULE_API FRotator GetValueAsRotator(const FName& KeyName) const;

	UFUNCTION(BlueprintCallable, Category="AI|Components|Blackboard")
	AIMODULE_API void SetValueAsObject(const FName& KeyName, UObject* ObjectValue);
	
	UFUNCTION(BlueprintCallable, Category="AI|Components|Blackboard")
	AIMODULE_API void SetValueAsClass(const FName& KeyName, UClass* ClassValue);

	UFUNCTION(BlueprintCallable, Category="AI|Components|Blackboard")
	AIMODULE_API void SetValueAsEnum(const FName& KeyName, uint8 EnumValue);

	UFUNCTION(BlueprintCallable, Category="AI|Components|Blackboard")
	AIMODULE_API void SetValueAsInt(const FName& KeyName, int32 IntValue);

	UFUNCTION(BlueprintCallable, Category="AI|Components|Blackboard")
	AIMODULE_API void SetValueAsFloat(const FName& KeyName, float FloatValue);

	UFUNCTION(BlueprintCallable, Category="AI|Components|Blackboard")
	AIMODULE_API void SetValueAsBool(const FName& KeyName, bool BoolValue);

	UFUNCTION(BlueprintCallable, Category="AI|Components|Blackboard")
	AIMODULE_API void SetValueAsString(const FName& KeyName, FString StringValue);

	UFUNCTION(BlueprintCallable, Category="AI|Components|Blackboard")
	AIMODULE_API void SetValueAsName(const FName& KeyName, FName NameValue);

	UFUNCTION(BlueprintCallable, Category="AI|Components|Blackboard")
	AIMODULE_API void SetValueAsVector(const FName& KeyName, FVector VectorValue);

	UFUNCTION(BlueprintCallable, Category = "AI|Components|Blackboard")
	AIMODULE_API void SetValueAsRotator(const FName& KeyName, FRotator VectorValue);

	UFUNCTION(BlueprintCallable, Category="AI|Components|Blackboard", Meta=(
		Tooltip="If the vector value has been set (and not cleared), this function returns true (indicating that the value should be valid).  If it's not set, the vector value is invalid and this function will return false.  (Also returns false if the key specified does not hold a vector.)"))
	AIMODULE_API bool IsVectorValueSet(const FName& KeyName) const;
	AIMODULE_API bool IsVectorValueSet(FBlackboard::FKey KeyID) const;
		
	/** return false if call failed (most probably no such entry in BB) */
	UFUNCTION(BlueprintCallable, Category="AI|Components|Blackboard")
	AIMODULE_API bool GetLocationFromEntry(const FName& KeyName, FVector& ResultLocation) const;
	AIMODULE_API bool GetLocationFromEntry(FBlackboard::FKey KeyID, FVector& ResultLocation) const;

	/** return false if call failed (most probably no such entry in BB) */
	UFUNCTION(BlueprintCallable, Category="AI|Components|Blackboard")
	AIMODULE_API bool GetRotationFromEntry(const FName& KeyName, FRotator& ResultRotation) const;
	AIMODULE_API bool GetRotationFromEntry(FBlackboard::FKey KeyID, FRotator& ResultRotation) const;

	UFUNCTION(BlueprintCallable, Category = "AI|Components|Blackboard")
	AIMODULE_API void ClearValue(const FName& KeyName);
	AIMODULE_API void ClearValue(FBlackboard::FKey KeyID);

	/** Copy content from SourceKeyID to DestinationID and return true if it worked */
	AIMODULE_API bool CopyKeyValue(FBlackboard::FKey SourceKeyID, FBlackboard::FKey DestinationID);

	template<class TDataClass>
	bool IsKeyOfType(const FName& KeyName) const;

	template<class TDataClass>
	bool IsKeyOfType(FBlackboard::FKey KeyID) const;

	template<class TDataClass>
	bool SetValue(const FName& KeyName, typename TDataClass::FDataType Value);

	template<class TDataClass>
	bool SetValue(FBlackboard::FKey KeyID, typename TDataClass::FDataType Value);

	template<class TDataClass>
	typename TDataClass::FDataType GetValue(const FName& KeyName) const;

	template<class TDataClass>
	typename TDataClass::FDataType GetValue(FBlackboard::FKey KeyID) const;

	/** get pointer to raw data for given key */
	inline uint8* GetKeyRawData(const FName& KeyName) { return GetKeyRawData(GetKeyID(KeyName)); }
	inline uint8* GetKeyRawData(FBlackboard::FKey KeyID) { return ValueMemory.Num() && ValueOffsets.IsValidIndex(KeyID) ? (ValueMemory.GetData() + ValueOffsets[KeyID]) : NULL; }

	inline const uint8* GetKeyRawData(const FName& KeyName) const { return GetKeyRawData(GetKeyID(KeyName)); }
	inline const uint8* GetKeyRawData(FBlackboard::FKey KeyID) const { return ValueMemory.Num() && ValueOffsets.IsValidIndex(KeyID) ? (ValueMemory.GetData() + ValueOffsets[KeyID]) : NULL; }

	PRAGMA_DISABLE_DEPRECATION_WARNINGS // re BlackboardAsset
	inline bool IsValidKey(FBlackboard::FKey KeyID) const { check(BlackboardAsset); return BlackboardAsset->IsValidKey(KeyID); }
	AIMODULE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS // re BlackboardAsset

	/** compares blackboard's values under specified keys */
	EBlackboardCompare::Type CompareKeyValues(TSubclassOf<UBlackboardKeyType> KeyType, FBlackboard::FKey KeyA, FBlackboard::FKey KeyB) const;

	AIMODULE_API FString GetDebugInfoString(EBlackboardDescription::Type Mode) const;

	/** get description of value under given key */
	AIMODULE_API FString DescribeKeyValue(const FName& KeyName, EBlackboardDescription::Type Mode) const;
	AIMODULE_API FString DescribeKeyValue(FBlackboard::FKey KeyID, EBlackboardDescription::Type Mode) const;

#if ENABLE_VISUAL_LOG
	/** prepare blackboard snapshot for logs */
	AIMODULE_API virtual void DescribeSelfToVisLog(struct FVisualLogEntry* Snapshot) const;
#endif
	
protected:

	/** cached behavior tree component */
	UPROPERTY(transient)
	TObjectPtr<UBrainComponent> BrainComp;

	/** data asset defining entries. Will be used as part of InitializeComponent 
	 *	call provided BlackboardAsset hasn't been already set (via a InitializeBlackboard 
	 *	call). */
	UPROPERTY(EditDefaultsOnly, Category = AI)
	TObjectPtr<UBlackboardData> DefaultBlackboardAsset;

	/** internal use, current BB asset being used. Will be made private in the future */
	UE_DEPRECATED_FORGAME(4.26, "Directly accessing BlackboardAsset is not longer supported. Use DefaultBlackboardAsset or InitializeBlackboard to set it and GetBlackboardAsset to retrieve it")
	UPROPERTY(transient)
	TObjectPtr<UBlackboardData> BlackboardAsset;

	/** memory block holding all values */
	TArray<uint8> ValueMemory;

	/** offsets in ValueMemory for each key */
	TArray<uint16> ValueOffsets;

	/** instanced keys with custom data allocations */
	UPROPERTY(transient)
	TArray<TObjectPtr<UBlackboardKeyType>> KeyInstances;
	
	/** tracks whether memory used to store values in the blackboard has been fully initialized */
	bool bBlackboardMemoryInitialized = false;

protected:
	struct FOnBlackboardChangeNotificationInfo
	{
		FOnBlackboardChangeNotificationInfo(const FOnBlackboardChangeNotification& InDelegateHandle)
			: DelegateHandle(InDelegateHandle)
		{
		}

		FDelegateHandle GetHandle() const
		{
			return DelegateHandle.GetHandle();
		}

		FOnBlackboardChangeNotification DelegateHandle;
		bool bToBeRemoved = false;
	};


	/** Count of re-entrant observer notifications */
	mutable int32 NotifyObserversRecursionCount = 0;

	/** Count of observers to remove */
	mutable int32 ObserversToRemoveCount = 0;

	/** observers registered for blackboard keys */
	mutable TMultiMap<FBlackboard::FKey, FOnBlackboardChangeNotificationInfo> Observers;
	
	/** observers registered from owner objects */
	mutable TMultiMap<const UObject*, FDelegateHandle> ObserverHandles;

	/** queued key change notification, will be processed on ResumeUpdates call */
	mutable TArray<FBlackboard::FKey> QueuedUpdates;

	/** set when observation notifies are paused and shouldn't be passed to observers */
	uint32 bPausedNotifies : 1;

	/** reset to false every time a new BB asset is assigned to this component */
	uint32 bSynchronizedKeyPopulated : 1;

	/** notifies behavior tree decorators about change in blackboard */
	AIMODULE_API void NotifyObservers(FBlackboard::FKey KeyID) const;

	/** initializes parent chain in asset */
	AIMODULE_API void InitializeParentChain(UBlackboardData* NewAsset);

	/** destroy allocated values */
	AIMODULE_API void DestroyValues();

	/** populates BB's synchronized entries */
	AIMODULE_API void PopulateSynchronizedKeys();

	AIMODULE_API bool ShouldSyncWithBlackboard(UBlackboardComponent& OtherBlackboardComponent) const;

	friend UBlackboardKeyType;
};

//////////////////////////////////////////////////////////////////////////
// Inlines

PRAGMA_DISABLE_DEPRECATION_WARNINGS // re BlackboardAsset

inline bool UBlackboardComponent::HasValidAsset() const
{
	return BlackboardAsset && BlackboardAsset->IsValid();
}

template<class TDataClass>
bool UBlackboardComponent::IsKeyOfType(const FName& KeyName) const
{
	const FBlackboard::FKey KeyID = GetKeyID(KeyName);
	return IsKeyOfType<TDataClass>(KeyID);
}

template<class TDataClass>
bool UBlackboardComponent::IsKeyOfType(FBlackboard::FKey KeyID) const
{
	const FBlackboardEntry* EntryInfo = BlackboardAsset ? BlackboardAsset->GetKey(KeyID) : nullptr;
	return (EntryInfo != nullptr) && (EntryInfo->KeyType != nullptr) && (EntryInfo->KeyType->GetClass() == TDataClass::StaticClass());
}

template<class TDataClass>
bool UBlackboardComponent::SetValue(const FName& KeyName, typename TDataClass::FDataType Value)
{
	const FBlackboard::FKey KeyID = GetKeyID(KeyName);
	return SetValue<TDataClass>(KeyID, Value);
}

template<class TDataClass>
bool UBlackboardComponent::SetValue(FBlackboard::FKey KeyID, typename TDataClass::FDataType Value)
{
	const FBlackboardEntry* EntryInfo = BlackboardAsset ? BlackboardAsset->GetKey(KeyID) : nullptr;
	if ((EntryInfo == nullptr) || (EntryInfo->KeyType == nullptr) || (EntryInfo->KeyType->GetClass() != TDataClass::StaticClass()))
	{
		return false;
	}

	const uint16 DataOffset = EntryInfo->KeyType->HasInstance() ? sizeof(FBlackboardInstancedKeyMemory) : 0;
	uint8* RawData = GetKeyRawData(KeyID) + DataOffset;
	if (RawData)
	{
		UBlackboardKeyType* KeyOb = EntryInfo->KeyType->HasInstance() ? KeyInstances[KeyID] : EntryInfo->KeyType;
		const bool bChanged = TDataClass::SetValue((TDataClass*)KeyOb, RawData, Value);
		if (bChanged)
		{
			NotifyObservers(KeyID);
			if (BlackboardAsset->HasSynchronizedKeys() && IsKeyInstanceSynced(KeyID))
			{
				UAISystem* AISystem = UAISystem::GetCurrentSafe(GetWorld());
				for (auto Iter = AISystem->CreateBlackboardDataToComponentsIterator(*BlackboardAsset); Iter; ++Iter)
				{
					UBlackboardComponent* OtherBlackboard = Iter.Value();
					if (OtherBlackboard != nullptr && ShouldSyncWithBlackboard(*OtherBlackboard))
					{
						UBlackboardData* const OtherBlackboardAsset = OtherBlackboard->GetBlackboardAsset();
						const FBlackboard::FKey OtherKeyID = OtherBlackboardAsset ? OtherBlackboardAsset->GetKeyID(EntryInfo->EntryName) : FBlackboard::InvalidKey;
						if (OtherKeyID != FBlackboard::InvalidKey)
						{
							UBlackboardKeyType* OtherKeyOb = EntryInfo->KeyType->HasInstance() ? OtherBlackboard->KeyInstances[OtherKeyID] : EntryInfo->KeyType;
							uint8* OtherRawData = OtherBlackboard->GetKeyRawData(OtherKeyID) + DataOffset;

							TDataClass::SetValue((TDataClass*)OtherKeyOb, OtherRawData, Value);
							OtherBlackboard->NotifyObservers(OtherKeyID);
						}
					}
				}
			}
		}

		return true;
	}

	return false;
}

template<class TDataClass>
typename TDataClass::FDataType UBlackboardComponent::GetValue(const FName& KeyName) const
{
	const FBlackboard::FKey KeyID = GetKeyID(KeyName);
	return GetValue<TDataClass>(KeyID);
}

template<class TDataClass>
typename TDataClass::FDataType UBlackboardComponent::GetValue(FBlackboard::FKey KeyID) const
{
	const FBlackboardEntry* EntryInfo = BlackboardAsset ? BlackboardAsset->GetKey(KeyID) : nullptr;
	if ((EntryInfo == nullptr) || (EntryInfo->KeyType == nullptr) || (EntryInfo->KeyType->GetClass() != TDataClass::StaticClass()))
	{
		return TDataClass::InvalidValue;
	}

	UBlackboardKeyType* KeyOb = EntryInfo->KeyType->HasInstance() ? KeyInstances[KeyID] : EntryInfo->KeyType;
	const uint16 DataOffset = EntryInfo->KeyType->HasInstance() ? sizeof(FBlackboardInstancedKeyMemory) : 0;

	const uint8* RawData = GetKeyRawData(KeyID) + DataOffset;
	return RawData ? TDataClass::GetValue((TDataClass*)KeyOb, RawData) : TDataClass::InvalidValue;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS // re BlackboardAsset

/**
 *	A helper type that improved performance of reading data from BB
 *	It's meant for a specific use-case:

 *		1.	you have a logical property you want to use both in C++ code
 *			as well as being reflected in the BB
 *		2.	you only ever set this property in native code
 *
 *		If those two are true then add a member variable of type FBlackboardCachedAccessor
 *		like so:

 *			FBlackboardCachedAccessor<UBlackboardKeyType_Bool> BBEnemyInMeleeRangeKey;

 *		and from this point on whenever you set or read the value use this variable.
 *		This will make reading almost free.
 *
 *		Before you use the variable you need to initialize it with appropriate BB asset.
 *		This is best done in AAIController::InitializeBlackboard override, like so:
 *
 *			const FBlackboard::FKey EnemyInMeleeRangeKey = BlackboardAsset.GetKeyID(TEXT("EnemyInMeleeRange"));
 *			BBEnemyInMeleeRangeKey = FBlackboardCachedAccessor<UBlackboardKeyType_Bool>(BlackboardComp, EnemyInMeleeRangeKey);
 *
 *		Best used with numerical and boolean types. No guarantees made when using pointer types.

 *	@note does not automatically support BB component or asset change */
template<typename TBlackboardKey>
struct FBBKeyCachedAccessor
{
private:
	FBlackboard::FKey BBKey;
	typedef typename TBlackboardKey::FDataType FStoredType;
	FStoredType CachedValue;
public:
	FBBKeyCachedAccessor() : BBKey(FBlackboard::InvalidKey), CachedValue(TBlackboardKey::InvalidValue)
	{}

	FBBKeyCachedAccessor(const UBlackboardComponent& BBComponent, FBlackboard::FKey InBBKey)
	{
		ensure(InBBKey != FBlackboard::InvalidKey);
		if (ensure(BBComponent.IsKeyOfType<TBlackboardKey>(InBBKey)))
		{
			BBKey = InBBKey;
			CachedValue = BBComponent.GetValue<TBlackboardKey>(InBBKey);
		}
	}

	template<typename T2>
	inline bool SetValue(UBlackboardComponent& BBComponent, const T2 InValue)
	{
		return SetValue(BBComponent, FStoredType(InValue));
	}

	/** @return True is value has changed*/
	inline bool SetValue(UBlackboardComponent& BBComponent, const FStoredType InValue)
	{
		ensure(BBKey != FBlackboard::InvalidKey);
		if (InValue != CachedValue)
		{
			CachedValue = InValue;
			BBComponent.SetValue<TBlackboardKey>(BBKey, InValue);
			return true;
		}
		return false;
	}

	inline const FStoredType& Get() const
	{
		ensure(BBKey != FBlackboard::InvalidKey);
		return CachedValue;
	}

	template<typename T2>
	inline T2 Get() const
	{
		ensure(BBKey != FBlackboard::InvalidKey);
		return (T2)CachedValue;
	}

	bool IsValid() const { return BBKey != FBlackboard::InvalidKey; }
};
