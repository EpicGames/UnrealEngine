// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "InputCoreTypes.h"
#include "Templates/SubclassOf.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType.h"
#include "Engine/DataAsset.h"
#include "BlackboardData.generated.h"

/** blackboard entry definition */
USTRUCT()
struct FBlackboardEntry
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=Blackboard)
	FName EntryName;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category=Blackboard, Meta=(ToolTip="Optional description to explain what this blackboard entry does."))
	FString EntryDescription;

	UPROPERTY(EditAnywhere, Category=Blackboard)
	FName EntryCategory;
#endif // WITH_EDITORONLY_DATA

	/** key type and additional properties */
	UPROPERTY(EditAnywhere, Instanced, Category = Blackboard, meta = (AllowEditInlineCustomization))
	TObjectPtr<UBlackboardKeyType> KeyType;

	/** if set to true then this field will be synchronized across all instances of this blackboard */
	UPROPERTY(EditAnywhere, Category=Blackboard)
	uint32 bInstanceSynced : 1;

	FBlackboardEntry()
		: KeyType(nullptr), bInstanceSynced(0)
	{}

	AIMODULE_API bool operator==(const FBlackboardEntry& Other) const;
};

UCLASS(BlueprintType, AutoExpandCategories=(Blackboard), MinimalAPI)
class UBlackboardData : public UDataAsset
{
	GENERATED_UCLASS_BODY()
	DECLARE_MULTICAST_DELEGATE_OneParam(FKeyUpdate, UBlackboardData* /*asset*/);

	/** parent blackboard (keys can be overridden) */
	UPROPERTY(EditAnywhere, Category=Parent)
	TObjectPtr<UBlackboardData> Parent;

#if WITH_EDITORONLY_DATA
	/** all keys inherited from parent chain */
	UPROPERTY(VisibleDefaultsOnly, Transient, Category=Parent)
	TArray<FBlackboardEntry> ParentKeys;
#endif

	/** blackboard keys */
	UPROPERTY(EditAnywhere, Category=Blackboard)
	TArray<FBlackboardEntry> Keys;

private:
	UPROPERTY()
	uint32 bHasSynchronizedKeys : 1;

public:

	inline bool HasSynchronizedKeys() const { return bHasSynchronizedKeys; }

	/** @return true if the key is instance synced */
	AIMODULE_API bool IsKeyInstanceSynced(FBlackboard::FKey KeyID) const;
	
	/** @return key ID from name */
	AIMODULE_API FBlackboard::FKey GetKeyID(const FName& KeyName) const;

	/** @return name of key */
	AIMODULE_API FName GetKeyName(FBlackboard::FKey KeyID) const;

	/** @return class of value for given key */
	AIMODULE_API TSubclassOf<UBlackboardKeyType> GetKeyType(FBlackboard::FKey KeyID) const;

	/** @return number of defined keys, including parent chain */
	AIMODULE_API int32 GetNumKeys() const;

	inline FBlackboard::FKey GetFirstKeyID() const { return FirstKeyID; }

	bool IsValidKey(FBlackboard::FKey KeyID) const { return KeyID != FBlackboard::InvalidKey && (int32)KeyID < GetNumKeys(); }

	/** @return key data */
	AIMODULE_API const FBlackboardEntry* GetKey(FBlackboard::FKey KeyID) const;

	const TArray<FBlackboardEntry>& GetKeys() const { return Keys; }

	AIMODULE_API virtual void PostInitProperties() override;
	AIMODULE_API virtual void PostLoad() override;
#if WITH_EDITOR
	AIMODULE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	AIMODULE_API void PropagateKeyChangesToDerivedBlackboardAssets();

	/** @return true if blackboard keys are not conflicting with parent key chain */
	AIMODULE_API bool IsValid() const;

	/** updates persistent key with given name, depending on currently defined entries and parent chain
	 *  @return key type of newly created entry for further setup
	 */
	template<class T>
	T* UpdatePersistentKey(const FName& KeyName)
	{
		T* CreatedKeyType = NULL;

		const FBlackboard::FKey KeyID = InternalGetKeyID(KeyName, DontCheckParentKeys);
		if (KeyID == FBlackboard::InvalidKey && Parent == NULL)
		{
			FBlackboardEntry Entry;
			Entry.EntryName = KeyName;

			CreatedKeyType = NewObject<T>(this);
			Entry.KeyType = CreatedKeyType;		

			Keys.Add(Entry);
			MarkPackageDirty();
			PropagateKeyChangesToDerivedBlackboardAssets();
		}
		else if (KeyID != FBlackboard::InvalidKey && Parent != NULL)
		{
			const FBlackboard::FKey KeyIndex = (int32)KeyID - (int32)FirstKeyID;
			Keys.RemoveAt(KeyIndex);
			MarkPackageDirty();
			PropagateKeyChangesToDerivedBlackboardAssets();
		}

		return CreatedKeyType;
	}

#if WITH_EDITOR
	/** A delegate called on PostEditChangeProperty. Can be used in editor to react to asset changes. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FBlackboardDataChanged, UBlackboardData* /*Asset*/);
	static AIMODULE_API FBlackboardDataChanged OnBlackboardDataChanged;
#endif

	/** delegate called for every loaded blackboard asset
	 *  meant for adding game specific persistent keys */
	static AIMODULE_API FKeyUpdate OnUpdateKeys;

	/** updates parent key cache for editor */
	AIMODULE_API void UpdateParentKeys();

	/** forces update of FirstKeyID, which depends on parent chain */
	AIMODULE_API void UpdateKeyIDs();

	AIMODULE_API void UpdateIfHasSynchronizedKeys();

	/** fix entries with deprecated key types */
	AIMODULE_API void UpdateDeprecatedKeys();

	/** returns true if OtherAsset is somewhere up the parent chain of this asset. Node that it will return false if *this == OtherAsset */
	AIMODULE_API bool IsChildOf(const UBlackboardData& OtherAsset) const;

	/** returns true if OtherAsset is equal to *this, or is it's parent, or *this is OtherAsset's parent */
	bool IsRelatedTo(const UBlackboardData& OtherAsset) const
	{
		return this == &OtherAsset || IsChildOf(OtherAsset) || OtherAsset.IsChildOf(*this)
			|| (Parent && OtherAsset.Parent && Parent->IsRelatedTo(*OtherAsset.Parent));
	}

protected:

	enum EKeyLookupMode
	{
		CheckParentKeys,
		DontCheckParentKeys,
	};

	/** @return first ID for keys of this asset (parent keys goes first) */
	FBlackboard::FKey FirstKeyID;

	/** @return key ID from name */
	AIMODULE_API FBlackboard::FKey InternalGetKeyID(const FName& KeyName, EKeyLookupMode LookupMode) const;
};
