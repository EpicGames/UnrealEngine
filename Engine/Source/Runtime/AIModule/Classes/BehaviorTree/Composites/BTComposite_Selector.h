// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/BTCompositeNode.h"
#include "BTComposite_Selector.generated.h"

/** 
 * Selector composite node.
 * Selector Nodes execute their children from left to right, and will stop executing its children when one of their children succeeds.
 * If a Selector's child succeeds, the Selector succeeds. If all the Selector's children fail, the Selector fails.
 */
UCLASS(MinimalAPI)
class UBTComposite_Selector: public UBTCompositeNode
{
	GENERATED_UCLASS_BODY()

	AIMODULE_API virtual int32 GetNextChildHandler(struct FBehaviorTreeSearchData& SearchData, int32 PrevChild, EBTNodeResult::Type LastResult) const override;

#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
#endif
};
