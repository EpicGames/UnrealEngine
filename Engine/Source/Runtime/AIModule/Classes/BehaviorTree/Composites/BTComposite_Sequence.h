// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/BTCompositeNode.h"
#include "BTComposite_Sequence.generated.h"

/**
 * Sequence composite node.
 * Sequence Nodes execute their children from left to right, and will stop executing its children when one of their children fails.
 * If a child fails, then the Sequence fails. If all the Sequence's children succeed, then the Sequence succeeds.
 */
UCLASS(MinimalAPI)
class UBTComposite_Sequence : public UBTCompositeNode
{
	GENERATED_UCLASS_BODY()

	AIMODULE_API virtual int32 GetNextChildHandler(struct FBehaviorTreeSearchData& SearchData, int32 PrevChild, EBTNodeResult::Type LastResult) const override;

#if WITH_EDITOR
	AIMODULE_API virtual bool CanAbortLowerPriority() const override;
	AIMODULE_API virtual FName GetNodeIconName() const override;
#endif
};
