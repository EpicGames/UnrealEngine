// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/BTCompositeNode.h"
#include "BTComposite_SimpleParallel.generated.h"

namespace EBTParallelChild
{
	enum Type
	{
		MainTask,
		BackgroundTree,
	};
}

UENUM()
namespace EBTParallelMode
{
	// keep in sync with DescribeFinishMode

	enum Type : int
	{
		AbortBackground UMETA(DisplayName="Immediate" , ToolTip="When main task finishes, immediately abort background tree."),
		WaitForBackground UMETA(DisplayName="Delayed" , ToolTip="When main task finishes, wait for background tree to finish."),
	};
}

struct FBTParallelMemory : public FBTCompositeMemory
{
	/** last Id of search, detect infinite loops when there isn't any valid task in background tree */
	int32 LastSearchId;

	/** finish result of main task */
	TEnumAsByte<EBTNodeResult::Type> MainTaskResult;

	/** set when main task is running */
	uint8 bMainTaskIsActive : 1;

	/** try running background tree task even if main task has finished */
	uint8 bForceBackgroundTree : 1;

	/** set when main task needs to be repeated */
	uint8 bRepeatMainTask : 1;
};

/**
 * Simple Parallel composite node.
 * Allows for running two children: one which must be a single task node (with optional decorators), and the other of which can be a complete subtree.
 */
UCLASS(HideCategories=(Composite), MinimalAPI)
class UBTComposite_SimpleParallel : public UBTCompositeNode
{
	GENERATED_UCLASS_BODY()

	/** how background tree should be handled when main task finishes execution */
	UPROPERTY(EditInstanceOnly, Category = Parallel)
	TEnumAsByte<EBTParallelMode::Type> FinishMode;

	/** handle child updates */
	AIMODULE_API virtual int32 GetNextChildHandler(FBehaviorTreeSearchData& SearchData, int32 PrevChild, EBTNodeResult::Type LastResult) const override;

	AIMODULE_API virtual void NotifyChildExecution(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, int32 ChildIdx, EBTNodeResult::Type& NodeResult) const override;
	AIMODULE_API virtual void NotifyNodeDeactivation(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type& NodeResult) const override;
	AIMODULE_API virtual bool CanNotifyDecoratorsOnDeactivation(FBehaviorTreeSearchData& SearchData, int32 ChildIdx, EBTNodeResult::Type& NodeResult) const override;
	AIMODULE_API virtual bool CanPushSubtree(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, int32 ChildIdx) const override;
	AIMODULE_API virtual void SetChildOverride(FBehaviorTreeSearchData& SearchData, int8 Index) const override;
	AIMODULE_API virtual uint16 GetInstanceMemorySize() const override;
	AIMODULE_API virtual void InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const override;
	AIMODULE_API virtual void CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const override;
	AIMODULE_API virtual FString GetStaticDescription() const override;
	AIMODULE_API virtual void DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const override;

	/** helper for showing values of EBTParallelMode enum */
	static AIMODULE_API FString DescribeFinishMode(EBTParallelMode::Type Mode);

#if WITH_EDITOR
	AIMODULE_API virtual bool CanAbortLowerPriority() const override;
	AIMODULE_API virtual bool CanAbortSelf() const override;
	AIMODULE_API virtual FName GetNodeIconName() const override;
#endif // WITH_EDITOR
};
