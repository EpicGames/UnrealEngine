// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "InputCoreTypes.h"
#include "BehaviorTree/Decorators/BTDecorator_BlackboardBase.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType.h"
#include "BTDecorator_Blackboard.generated.h"

class FBlackboardDecoratorDetails;
class UBehaviorTree;
class UBlackboardComponent;
struct FBlackboardEntry;

/**
 *  Decorator for accessing blackboard values
 */

UENUM()
namespace EBTBlackboardRestart
{
	enum Type : int
	{
		ValueChange		UMETA(DisplayName="On Value Change", ToolTip="Restart on every change of observed blackboard value"),
		ResultChange	UMETA(DisplayName="On Result Change", ToolTip="Restart only when result of evaluated condition is changed"),
	};
}

/**
 * Blackboard decorator node.
 * A decorator node that bases its condition on a Blackboard key.
 */
UCLASS(HideCategories=(Condition), MinimalAPI)
class UBTDecorator_Blackboard : public UBTDecorator_BlackboardBase
{
	GENERATED_UCLASS_BODY()

	AIMODULE_API virtual bool CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const override;
	AIMODULE_API virtual EBlackboardNotificationResult OnBlackboardKeyValueChange(const UBlackboardComponent& Blackboard, FBlackboard::FKey ChangedKeyID) override;
	AIMODULE_API virtual void DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const override;
	AIMODULE_API virtual FString GetStaticDescription() const override;

protected:

	/** value for arithmetic operations */
	UPROPERTY(Category=Blackboard, EditAnywhere, meta=(DisplayName="Key Value"))
	int32 IntValue;

	/** value for arithmetic operations */
	UPROPERTY(Category=Blackboard, EditAnywhere, meta=(DisplayName="Key Value"))
	float FloatValue;

	/** value for string operations */
	UPROPERTY(Category=Blackboard, EditAnywhere, meta=(DisplayName="Key Value"))
	FString StringValue;

	/** cached description */
	UPROPERTY()
	FString CachedDescription;

	/** operation type */
	UPROPERTY()
	uint8 OperationType;

	/** when observer can try to request abort? */
	UPROPERTY(Category=FlowControl, EditAnywhere)
	TEnumAsByte<EBTBlackboardRestart::Type> NotifyObserver;

#if WITH_EDITORONLY_DATA

	UPROPERTY(Category=Blackboard, EditAnywhere, meta=(DisplayName="Key Query"))
	TEnumAsByte<EBasicKeyOperation::Type> BasicOperation;

	UPROPERTY(Category=Blackboard, EditAnywhere, meta=(DisplayName="Key Query"))
	TEnumAsByte<EArithmeticKeyOperation::Type> ArithmeticOperation;

	UPROPERTY(Category=Blackboard, EditAnywhere, meta=(DisplayName="Key Query"))
	TEnumAsByte<ETextKeyOperation::Type> TextOperation;

#endif

#if WITH_EDITOR
public:
	/** describe decorator and cache it */
	AIMODULE_API virtual void BuildDescription();
protected:
	AIMODULE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	AIMODULE_API virtual void InitializeFromAsset(UBehaviorTree& Asset) override;

	/**
	 * decorator using enum based key requires to synchronize 'StringValue' used for storage with 'IntValue' used for runtime evaluation.
	 * @note using custom serialization version was not possible since we require blackboard and used types (e.g. UserDefinedEnumerations)
	 * to be loaded and keys to be resolved (i.e. InitializeFromAsset).
	 */
	AIMODULE_API void RefreshEnumBasedDecorator(const FBlackboardEntry& Entry);

	/** take blackboard value and evaluate decorator's condition */
	AIMODULE_API bool EvaluateOnBlackboard(const UBlackboardComponent& BlackboardComp) const;

	friend FBlackboardDecoratorDetails;
};
