// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "InputCoreTypes.h"
#include "BehaviorTree/BehaviorTreeTypes.h"
#include "BehaviorTree/BTDecorator.h"
#include "BTDecorator_BlackboardBase.generated.h"

class UBehaviorTree;
class UBlackboardComponent;

UCLASS(Abstract, MinimalAPI)
class UBTDecorator_BlackboardBase : public UBTDecorator
{
	GENERATED_UCLASS_BODY()

	/** initialize any asset related data */
	AIMODULE_API virtual void InitializeFromAsset(UBehaviorTree& Asset) override;

	/** notify about change in blackboard keys */
	AIMODULE_API virtual EBlackboardNotificationResult OnBlackboardKeyValueChange(const UBlackboardComponent& Blackboard, FBlackboard::FKey ChangedKeyID);

#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
	AIMODULE_API virtual FString GetErrorMessage() const override;
#endif

public:
	/** get name of selected blackboard key */
	FName GetSelectedBlackboardKey() const;

protected:

	/** blackboard key selector */
	UPROPERTY(EditAnywhere, Category=Blackboard)
	FBlackboardKeySelector BlackboardKey;

	/** called when execution flow controller becomes active */
	AIMODULE_API virtual void OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;

	/** called when execution flow controller becomes inactive */
	AIMODULE_API virtual void OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
};

//////////////////////////////////////////////////////////////////////////
// Inlines

inline FName UBTDecorator_BlackboardBase::GetSelectedBlackboardKey() const
{
	return BlackboardKey.SelectedKeyName;
}
