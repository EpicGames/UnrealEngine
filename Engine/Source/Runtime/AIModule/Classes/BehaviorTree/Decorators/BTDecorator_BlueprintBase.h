// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "InputCoreTypes.h"
#include "BehaviorTree/BTDecorator.h"
#include "BTDecorator_BlueprintBase.generated.h"

class AActor;
class AAIController;
class APawn;
class UBehaviorTree;
class UBlackboardComponent;

/**
 *  Base class for blueprint based decorator nodes. Do NOT use it for creating native c++ classes!
 *
 *  Unlike task and services, decorator have two execution chains: 
 *   ExecutionStart-ExecutionFinish and ObserverActivated-ObserverDeactivated
 *  which makes automatic latent action cleanup impossible. Keep in mind, that
 *  you HAVE TO verify is given chain is still active after resuming from any
 *  latent action (like Delay, Timelines, etc).
 *
 *  Helper functions:
 *  - IsDecoratorExecutionActive (true after ExecutionStart, until ExecutionFinish)
 *  - IsDecoratorObserverActive (true after ObserverActivated, until ObserverDeactivated)
 */

UCLASS(Abstract, Blueprintable, MinimalAPI)
class UBTDecorator_BlueprintBase : public UBTDecorator
{
	GENERATED_BODY()

public:
	AIMODULE_API UBTDecorator_BlueprintBase(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** initialize data about blueprint defined properties */
	AIMODULE_API void InitializeProperties();

	/** setup node name */
	AIMODULE_API virtual void PostInitProperties() override;
	AIMODULE_API virtual void PostLoad() override;

	/** notify about changes in blackboard */
	AIMODULE_API EBlackboardNotificationResult OnBlackboardKeyValueChange(const UBlackboardComponent& Blackboard, FBlackboard::FKey ChangedKeyID);

	AIMODULE_API virtual FString GetStaticDescription() const override;
	AIMODULE_API virtual void DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const override;
	AIMODULE_API virtual bool CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const override;
	AIMODULE_API virtual void OnInstanceDestroyed(UBehaviorTreeComponent& OwnerComp) override;
	AIMODULE_API virtual void InitializeFromAsset(UBehaviorTree& Asset) override;

	enum class EAbortType : uint8
	{
		NoAbort,
		ActivateBranch,
		DeactivateBranch,
		Unknown,
	};

	/** return this decorator abort type in current circumstances */
	AIMODULE_API EAbortType EvaluateAbortType(UBehaviorTreeComponent& OwnerComp) const;

	UE_DEPRECATED(5.2, "Use GetShouldAbortType instead")
	bool GetShouldAbort(UBehaviorTreeComponent& OwnerComp) const
	{
		return  EvaluateAbortType(OwnerComp) != EAbortType::NoAbort;
	}

	AIMODULE_API virtual void SetOwner(AActor* ActorOwner) override;

#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
	AIMODULE_API virtual bool UsesBlueprint() const override;
#endif

protected:
	/** Cached AIController owner of BehaviorTreeComponent. */
	UPROPERTY(Transient)
	TObjectPtr<AAIController> AIOwner;

	/** Cached AIController owner of BehaviorTreeComponent. */
	UPROPERTY(Transient)
	TObjectPtr<AActor> ActorOwner;

	/** blackboard key names that should be observed */
	UPROPERTY()
	TArray<FName> ObservedKeyNames;

	/** properties with runtime values, stored only in class default object */
	TArray<FProperty*> PropertyData;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = Description)
	FString CustomDescription;
#endif // WITH_EDITORONLY_DATA

	/** show detailed information about properties */
	UPROPERTY(EditInstanceOnly, Category=Description)
	uint32 bShowPropertyDetails : 1;

	/** Applies only if Decorator has any FBlackboardKeySelector property and if decorator is 
	 *	set to abort BT flow. Is set to true ReceiveConditionCheck will be called only on changes 
	  *	to observed BB keys. If false or no BB keys observed ReceiveConditionCheck will be called every tick */
	UPROPERTY(EditDefaultsOnly, Category = "FlowControl", AdvancedDisplay)
	uint32 bCheckConditionOnlyBlackBoardChanges : 1;

	/** gets set to true if decorator declared BB keys it can potentially observe */
	uint32 bIsObservingBB : 1;
	
	/** set if ReceiveTick is implemented by blueprint */
	uint32 ReceiveTickImplementations : 2;

	/** set if ReceiveExecutionStart is implemented by blueprint */
	uint32 ReceiveExecutionStartImplementations : 2;

	/** set if ReceiveExecutionFinish is implemented by blueprint */
	uint32 ReceiveExecutionFinishImplementations : 2;

	/** set if ReceiveObserverActivated is implemented by blueprint */
	uint32 ReceiveObserverActivatedImplementations : 2;

	/** set if ReceiveObserverDeactivated is implemented by blueprint */
	uint32 ReceiveObserverDeactivatedImplementations : 2;

	/** set if ReceiveConditionCheck is implemented by blueprint */
	uint32 PerformConditionCheckImplementations : 2;

	AIMODULE_API virtual void OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	AIMODULE_API virtual void OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	AIMODULE_API virtual void OnNodeActivation(FBehaviorTreeSearchData& SearchData) override;
	AIMODULE_API virtual void OnNodeDeactivation(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type NodeResult) override;
	AIMODULE_API virtual void TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds) override;

	/** tick function
	 *	@Note that if both generic and AI event versions are implemented only the more
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent)
	AIMODULE_API void ReceiveTick(AActor* OwnerActor, float DeltaSeconds);

	/** called on execution of underlying node 
	 *	@Note that if both generic and AI event versions are implemented only the more
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent)
	AIMODULE_API void ReceiveExecutionStart(AActor* OwnerActor);

	/** called when execution of underlying node is finished 
	 *	@Note that if both generic and AI event versions are implemented only the more
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent)
	AIMODULE_API void ReceiveExecutionFinish(AActor* OwnerActor, enum EBTNodeResult::Type NodeResult);

	/** called when observer is activated (flow controller) 
	 *	@Note that if both generic and AI event versions are implemented only the more
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent)
	AIMODULE_API void ReceiveObserverActivated(AActor* OwnerActor);

	/** called when observer is deactivated (flow controller) 
	 *	@Note that if both generic and AI event versions are implemented only the more
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent)
	AIMODULE_API void ReceiveObserverDeactivated(AActor* OwnerActor);

	/** called when testing if underlying node can be executed, must call FinishConditionCheck
	 *	@Note that if both generic and AI event versions are implemented only the more
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent)
	AIMODULE_API bool PerformConditionCheck(AActor* OwnerActor);

	/** Alternative AI version of ReceiveTick
	 *	@see ReceiveTick for more details
	 *	@Note that if both generic and AI event versions are implemented only the more
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent, Category = AI)
	AIMODULE_API void ReceiveTickAI(AAIController* OwnerController, APawn* ControlledPawn, float DeltaSeconds);

	/** Alternative AI version of ReceiveExecutionStart
	 *	@see ReceiveExecutionStart for more details
	 *	@Note that if both generic and AI event versions are implemented only the more
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent, Category = AI)
	AIMODULE_API void ReceiveExecutionStartAI(AAIController* OwnerController, APawn* ControlledPawn);

	/** Alternative AI version of ReceiveExecutionFinish
	 *	@see ReceiveExecutionFinish for more details
	 *	@Note that if both generic and AI event versions are implemented only the more
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent, Category = AI)
	AIMODULE_API void ReceiveExecutionFinishAI(AAIController* OwnerController, APawn* ControlledPawn, enum EBTNodeResult::Type NodeResult);

	/** Alternative AI version of ReceiveObserverActivated
	 *	@see ReceiveObserverActivated for more details
	 *	@Note that if both generic and AI event versions are implemented only the more
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent, Category = AI)
	AIMODULE_API void ReceiveObserverActivatedAI(AAIController* OwnerController, APawn* ControlledPawn);

	/** Alternative AI version of ReceiveObserverDeactivated
	 *	@see ReceiveObserverDeactivated for more details
	 *	@Note that if both generic and AI event versions are implemented only the more
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent, Category = AI)
	AIMODULE_API void ReceiveObserverDeactivatedAI(AAIController* OwnerController, APawn* ControlledPawn);

	/** Alternative AI version of ReceiveConditionCheck
	 *	@see ReceiveConditionCheck for more details
	 *	@Note that if both generic and AI event versions are implemented only the more
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent, Category = AI)
	AIMODULE_API bool PerformConditionCheckAI(AAIController* OwnerController, APawn* ControlledPawn);
		
	/** check if decorator is part of currently active branch */
	UFUNCTION(BlueprintCallable, Category="AI|BehaviorTree")
	AIMODULE_API bool IsDecoratorExecutionActive() const;

	/** check if decorator's observer is currently active */
	UFUNCTION(BlueprintCallable, Category="AI|BehaviorTree")
	AIMODULE_API bool IsDecoratorObserverActive() const;

	inline bool GetNeedsTickForConditionChecking() const { return PerformConditionCheckImplementations != 0 && (bIsObservingBB == false || bCheckConditionOnlyBlackBoardChanges == false); }

	/** Handle abort request calls from the specified abort type */
	AIMODULE_API void RequestAbort(UBehaviorTreeComponent& OwnerComp, const EAbortType Type);
};
