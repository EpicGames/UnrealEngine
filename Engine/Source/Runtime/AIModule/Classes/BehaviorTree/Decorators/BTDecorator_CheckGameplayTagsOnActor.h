// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameplayTagContainer.h"
#include "BehaviorTree/BTDecorator.h"
#include "BTDecorator_CheckGameplayTagsOnActor.generated.h"

class UBehaviorTree;

/**
 * GameplayTag decorator node.
 * A decorator node that bases its condition on whether the specified Actor (in the blackboard) has a Gameplay Tag or
 * Tags specified.
 */
UCLASS(MinimalAPI)
class UBTDecorator_CheckGameplayTagsOnActor : public UBTDecorator
{
	GENERATED_UCLASS_BODY()

	AIMODULE_API virtual bool CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const override;
	AIMODULE_API virtual void DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const override;
	AIMODULE_API virtual FString GetStaticDescription() const override;

protected:

	UPROPERTY(EditAnywhere, Category=GameplayTagCheck,
		Meta=(ToolTips="Which Actor (from the blackboard) should be checked for these gameplay tags?"))
	struct FBlackboardKeySelector ActorToCheck;

	UPROPERTY(EditAnywhere, Category=GameplayTagCheck)
	EGameplayContainerMatchType TagsToMatch;

	UPROPERTY(EditAnywhere, Category=GameplayTagCheck)
	FGameplayTagContainer GameplayTags;

	/** cached description */
	UPROPERTY()
	FString CachedDescription;

#if WITH_EDITOR
	/** describe decorator and cache it */
	AIMODULE_API virtual void BuildDescription();
	AIMODULE_API virtual FString GetErrorMessage() const override;
	AIMODULE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	AIMODULE_API virtual void InitializeFromAsset(UBehaviorTree& Asset) override;
};
