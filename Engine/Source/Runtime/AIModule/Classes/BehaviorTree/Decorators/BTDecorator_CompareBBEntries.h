// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "InputCoreTypes.h"
#include "BehaviorTree/BehaviorTreeTypes.h"
#include "BehaviorTree/BTDecorator.h"
#include "BTDecorator_CompareBBEntries.generated.h"

class UBehaviorTree;
class UBlackboardComponent;

UENUM()
namespace EBlackBoardEntryComparison
{
	enum Type : int
	{
		Equal			UMETA(DisplayName="Is Equal To"),
		NotEqual		UMETA(DisplayName="Is Not Equal To")
	};
}

/**
 * Blackboard comparison decorator node.
 * A decorator node that bases its condition on a comparison between two Blackboard keys.
 */
UCLASS(HideCategories=(Condition), MinimalAPI)
class UBTDecorator_CompareBBEntries : public UBTDecorator
{
	GENERATED_UCLASS_BODY()

protected:

	/** operation type */
	UPROPERTY(EditAnywhere, Category=Blackboard)
	TEnumAsByte<EBlackBoardEntryComparison::Type> Operator;

	/** blackboard key selector */
	UPROPERTY(EditAnywhere, Category=Blackboard)
	FBlackboardKeySelector BlackboardKeyA;

	/** blackboard key selector */
	UPROPERTY(EditAnywhere, Category=Blackboard)
	FBlackboardKeySelector BlackboardKeyB;

public:

	AIMODULE_API virtual bool CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const override;
	AIMODULE_API virtual FString GetStaticDescription() const override;

	AIMODULE_API virtual void InitializeFromAsset(UBehaviorTree& Asset) override;

	AIMODULE_API virtual EBlackboardNotificationResult OnBlackboardKeyValueChange(const UBlackboardComponent& Blackboard, FBlackboard::FKey ChangedKeyID);
	AIMODULE_API virtual void OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	AIMODULE_API virtual void OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;

#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
	AIMODULE_API virtual FString GetErrorMessage() const override;
#endif // WITH_EDITOR
};
