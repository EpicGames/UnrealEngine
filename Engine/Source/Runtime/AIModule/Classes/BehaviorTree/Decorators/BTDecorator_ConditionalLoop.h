// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "InputCoreTypes.h"
#include "BehaviorTree/Decorators/BTDecorator_Blackboard.h"
#include "BTDecorator_ConditionalLoop.generated.h"

class UBlackboardComponent;

struct FBTConditionalLoopDecoratorMemory
{
	int32 SearchId;
};

/**
 * Conditional loop decorator node.
 * A decorator node that loops execution as long as condition is satisfied.
 */
UCLASS(HideCategories=(FlowControl), MinimalAPI)
class UBTDecorator_ConditionalLoop : public UBTDecorator_Blackboard
{
	GENERATED_UCLASS_BODY()

	AIMODULE_API virtual bool CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const override;
	AIMODULE_API virtual EBlackboardNotificationResult OnBlackboardKeyValueChange(const UBlackboardComponent& Blackboard, FBlackboard::FKey ChangedKeyID) override;
	AIMODULE_API virtual void OnNodeDeactivation(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type NodeResult) override;

	AIMODULE_API virtual uint16 GetInstanceMemorySize() const override;
	AIMODULE_API virtual void InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const override;
	AIMODULE_API virtual void CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const override;

#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
#endif // WITH_EDITOR
};
