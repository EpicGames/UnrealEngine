// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "InputCoreTypes.h"
#include "BehaviorTree/BehaviorTreeTypes.h"
#include "BehaviorTree/BTDecorator.h"
#include "BehaviorTree/ValueOrBBKey.h"
#include "BTDecorator_ConeCheck.generated.h"

class UBehaviorTree;
class UBlackboardComponent;

struct FBTConeCheckDecoratorMemory
{
	bool bLastRawResult;
};

/**
 * Cone check decorator node.
 * A decorator node that bases its condition on a cone check, using Blackboard entries to form the parameters of the check.
 */
UCLASS(MinimalAPI)
class UBTDecorator_ConeCheck : public UBTDecorator
{
	GENERATED_UCLASS_BODY()

	typedef FBTConeCheckDecoratorMemory TNodeInstanceMemory;

	/** Angle between cone direction and code cone edge, or a half of the total cone angle */
	UPROPERTY(Category=Decorator, EditAnywhere)
	FValueOrBBKey_Float ConeHalfAngle;
	
	/** blackboard key selector */
	UPROPERTY(EditAnywhere, Category=Blackboard)
	FBlackboardKeySelector ConeOrigin;

	/** "None" means "use ConeOrigin's direction" */
	UPROPERTY(EditAnywhere, Category=Blackboard)
	FBlackboardKeySelector ConeDirection;

	/** blackboard key selector */
	UPROPERTY(EditAnywhere, Category=Blackboard)
	FBlackboardKeySelector Observed;

	AIMODULE_API virtual void InitializeFromAsset(UBehaviorTree& Asset) override;
	AIMODULE_API virtual uint16 GetInstanceMemorySize() const override;
	AIMODULE_API virtual void InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const override;
	AIMODULE_API virtual void CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const override;
	AIMODULE_API virtual void DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const override;
	AIMODULE_API virtual FString GetStaticDescription() const override;

#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
	AIMODULE_API virtual FString GetErrorMessage() const override;
#endif // WITH_EDITOR

protected:
	float GetConeHalfAngleDot(const UBehaviorTreeComponent& OwnerComp) const;
	AIMODULE_API virtual bool CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const override;
	AIMODULE_API virtual void OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	AIMODULE_API void OnBlackboardChange(const UBlackboardComponent& Blackboard, FBlackboard::FKey ChangedKeyID);

	AIMODULE_API virtual void TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds) override;
	
	AIMODULE_API bool CalculateDirection(const UBlackboardComponent* BlackboardComp, const FBlackboardKeySelector& Origin, const FBlackboardKeySelector& End, FVector& Direction) const;

private:
	bool CalcConditionImpl(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const;
};
