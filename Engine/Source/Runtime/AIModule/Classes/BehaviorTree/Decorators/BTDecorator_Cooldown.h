// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/BTDecorator.h"
#include "BehaviorTree/ValueOrBBKey.h"
#include "BTDecorator_Cooldown.generated.h"

struct FBTCooldownDecoratorMemory
{
	double LastUseTimestamp;
	uint8 bRequestedRestart : 1;
};

/**
 * Cooldown decorator node.
 * A decorator node that bases its condition on whether a cooldown timer has expired.
 */
UCLASS(HideCategories=(Condition), MinimalAPI)
class UBTDecorator_Cooldown : public UBTDecorator
{
	GENERATED_UCLASS_BODY()

	/** max allowed time for execution of underlying node */
	UPROPERTY(Category=Decorator, EditAnywhere)
	FValueOrBBKey_Float CoolDownTime;

	//~ Begin UObject Interface
	AIMODULE_API virtual void PostLoad() override;
	//~ End UObject Interface

	AIMODULE_API virtual bool CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const override;
	AIMODULE_API virtual void InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const override;
	AIMODULE_API virtual void CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const override;
	AIMODULE_API virtual uint16 GetInstanceMemorySize() const override;
	AIMODULE_API virtual void DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const override;
	AIMODULE_API virtual FString GetStaticDescription() const override;

#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
#endif // WITH_EDITOR

protected:

	AIMODULE_API virtual void OnNodeDeactivation(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type NodeResult) override;
	AIMODULE_API virtual void TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds) override;
};
