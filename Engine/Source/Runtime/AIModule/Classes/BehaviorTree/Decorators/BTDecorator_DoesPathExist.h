// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "BehaviorTree/BehaviorTreeTypes.h"
#include "BehaviorTree/BTDecorator.h"
#include "BehaviorTree/ValueOrBBKey.h"
#include "NavFilters/NavigationQueryFilter.h"
#include "BTDecorator_DoesPathExist.generated.h"

class UBehaviorTree;

UENUM()
namespace EPathExistanceQueryType
{
	enum Type : int
	{
		NavmeshRaycast2D UMETA(ToolTip = "Really Fast"),
		HierarchicalQuery UMETA(ToolTip = "Fast"),
		RegularPathFinding UMETA(ToolTip = "Slow"),
	};
}

/**
 * Does Path Exist decorator node.
 * A decorator node that bases its condition on whether a path exists between two points from the Blackboard.
 */
UCLASS(MinimalAPI)
class UBTDecorator_DoesPathExist : public UBTDecorator
{
	GENERATED_UCLASS_BODY()

protected:

	/** blackboard key selector */
	UPROPERTY(EditAnywhere, Category=Condition)
	FBlackboardKeySelector BlackboardKeyA;

	/** blackboard key selector */
	UPROPERTY(EditAnywhere, Category=Condition)
	FBlackboardKeySelector BlackboardKeyB;

public:

	// deprecated, set value of blackboard key A on initialization
	UPROPERTY()
	uint32 bUseSelf:1;

	UPROPERTY(EditAnywhere, Category=Condition)
	FValueOrBBKey_Enum PathQueryType;

	/** "None" will result in default filter being used */
	UPROPERTY(Category=Node, EditAnywhere)
	FValueOrBBKey_Class FilterClass = TSubclassOf<UNavigationQueryFilter>();

	AIMODULE_API virtual bool CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const override;
	AIMODULE_API virtual FString GetStaticDescription() const override;
	AIMODULE_API virtual void InitializeFromAsset(UBehaviorTree& Asset) override;

#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
	AIMODULE_API virtual FString GetErrorMessage() const override;
#endif // WITH_EDITOR
};
