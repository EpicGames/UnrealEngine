// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/BTDecorator.h"
#include "BTDecorator_ForceSuccess.generated.h"

/** 
 * Change node result to Success
 * useful for creating optional branches in sequence
 *
 * Forcing failed result was not implemented, because it doesn't make sense in both basic composites:
 * - sequence = child nodes behind it will be never run
 * - selector = would allow executing multiple nodes, turning it into a sequence...
 */

UCLASS(HideCategories=(Condition), MinimalAPI)
class UBTDecorator_ForceSuccess : public UBTDecorator
{
	GENERATED_UCLASS_BODY()

protected:

	AIMODULE_API virtual void OnNodeProcessed(struct FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type& NodeResult) override;

#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
#endif // WITH_EDITOR
};
