// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AITypes.h"

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "DataProviders/AIDataProvider.h"
#include "BehaviorTree/Decorators/BTDecorator_BlackboardBase.h"
#include "BehaviorTree/ValueOrBBKey.h"
#include "BTDecorator_IsAtLocation.generated.h"

/**
 * Is At Location decorator node.
 * A decorator node that checks if AI controlled pawn is at given location.
 */
UCLASS(MinimalAPI)
class UBTDecorator_IsAtLocation : public UBTDecorator_BlackboardBase
{
	GENERATED_UCLASS_BODY()

	/** distance threshold to accept as being at location */
	UPROPERTY(EditAnywhere, Category = Condition, meta = (ClampMin = "0.0", EditCondition = "!bUseParametrizedRadius"))
	float AcceptableRadius;

	UPROPERTY(EditAnywhere, Category = Condition, meta = (EditCondition = "bUseParametrizedRadius"))
	FAIDataProviderFloatValue ParametrizedAcceptableRadius;

	UPROPERTY(EditAnywhere, Category = Condition, meta = (EditCondition = "!bPathFindingBasedTest"))
	FAIDistanceType GeometricDistanceType;

	UPROPERTY()
	uint32 bUseParametrizedRadius : 1;

	/** if moving to an actor and this actor is a nav agent, then we will move to their nav agent location */
	UPROPERTY(EditAnywhere, Category = Condition, DisplayName = UseNavAgentGoalLocation, meta = (EditCondition = "bPathFindingBasedTest"))
	FValueOrBBKey_Bool bUseNavAgentGoalLocation;

	/** If true the result will be consistent with tests done while following paths.
	 *	Set to false to use geometric distance as configured with DistanceType */
	UPROPERTY(EditAnywhere, Category = Condition, DisplayName = PathFindingBasedTest)
	FValueOrBBKey_Bool bPathFindingBasedTest;

	AIMODULE_API virtual bool CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const override;
	
	AIMODULE_API virtual FString GetStaticDescription() const override;

#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
#endif // WITH_EDITOR

protected:
	AIMODULE_API FVector::FReal GetGeometricDistanceSquared(const FVector& A, const FVector& B) const;
};
