// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "InputCoreTypes.h"
#include "Templates/SubclassOf.h"
#include "BehaviorTree/Decorators/BTDecorator_BlackboardBase.h"
#include "BehaviorTree/ValueOrBBKey.h"
#include "BTDecorator_IsBBEntryOfClass.generated.h"

class UBlackboardComponent;

UCLASS(HideCategories=(Condition), MinimalAPI)
class UBTDecorator_IsBBEntryOfClass : public UBTDecorator_BlackboardBase
{
	GENERATED_BODY()
		
public:
	AIMODULE_API UBTDecorator_IsBBEntryOfClass(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

protected:

	UPROPERTY(Category = Blackboard, EditAnywhere)
	FValueOrBBKey_Class TestClass = TSubclassOf<UObject>();


	AIMODULE_API virtual bool CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const override;
	AIMODULE_API virtual EBlackboardNotificationResult OnBlackboardKeyValueChange(const UBlackboardComponent& Blackboard, FBlackboard::FKey ChangedKeyID) override;
	AIMODULE_API virtual void DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const override;
	AIMODULE_API virtual FString GetStaticDescription() const override;
};
