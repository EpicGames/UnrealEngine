// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/BehaviorTreeTypes.h"
#include "BehaviorTree/BTDecorator.h"
#include "BehaviorTree/ValueOrBBKey.h"
#include "BTDecorator_KeepInCone.generated.h"

class UBehaviorTree;

struct FBTKeepInConeDecoratorMemory
{
	FVector InitialDirection;
};

/**
 * Keep In Cone decorator node.
 * A decorator node that bases its condition on whether the observed position is still inside a cone. The cone's direction is calculated when the node first becomes relevant.
 */
UCLASS(HideCategories=(Condition), MinimalAPI)
class UBTDecorator_KeepInCone : public UBTDecorator
{
	GENERATED_UCLASS_BODY()

	typedef FBTKeepInConeDecoratorMemory TNodeInstanceMemory;

	/** max allowed time for execution of underlying node */
	UPROPERTY(Category=Decorator, EditAnywhere)
	FValueOrBBKey_Float ConeHalfAngle;
	
	/** blackboard key selector */
	UPROPERTY(EditAnywhere, Category=Blackboard)
	FBlackboardKeySelector ConeOrigin;

	/** blackboard key selector */
	UPROPERTY(EditAnywhere, Category=Blackboard)
	FBlackboardKeySelector Observed;

	// deprecated, set value of ConeOrigin on initialization
	UPROPERTY()
	uint32 bUseSelfAsOrigin:1;

	// deprecated, set value of Observed on initialization
	UPROPERTY()
	uint32 bUseSelfAsObserved:1;

	float GetConeHalfAngleDot(const UBehaviorTreeComponent& OwnerComp) const;

	AIMODULE_API virtual void InitializeFromAsset(UBehaviorTree& Asset) override;
	AIMODULE_API virtual uint16 GetInstanceMemorySize() const override;
	AIMODULE_API virtual void InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const override;
	AIMODULE_API virtual void CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const override;
	AIMODULE_API virtual void DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const override;
	AIMODULE_API virtual FString GetStaticDescription() const override;

#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
	AIMODULE_API virtual FString GetErrorMessage() const override;
#endif // WITH_EDITOR

protected:

	AIMODULE_API virtual void OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	AIMODULE_API virtual void TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds) override;

	AIMODULE_API bool CalculateCurrentDirection(const UBehaviorTreeComponent& OwnerComp, FVector& Direction) const;
};
