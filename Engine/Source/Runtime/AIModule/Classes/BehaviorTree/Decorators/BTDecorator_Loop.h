// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/BTDecorator.h"
#include "BehaviorTree/ValueOrBBKey.h"
#include "BTDecorator_Loop.generated.h"

struct FBTLoopDecoratorMemory
{
	int32 SearchId;
	uint8 RemainingExecutions;
	double TimeStarted;
};

/**
 * Loop decorator node.
 * A decorator node that bases its condition on whether its loop counter has been exceeded.
 */
UCLASS(HideCategories=(Condition), MinimalAPI)
class UBTDecorator_Loop : public UBTDecorator
{
	GENERATED_UCLASS_BODY()

	/** number of executions */
	UPROPERTY(Category=Decorator, EditAnywhere, meta=(EditCondition="!bInfiniteLoop", ClampMin="1", ClampMax="255"))
	FValueOrBBKey_Int32 NumLoops;

	/** infinite loop */
	UPROPERTY(Category = Decorator, EditAnywhere)
	bool bInfiniteLoop;

	/** timeout (when looping infinitely, when we finish a loop we will check whether we have spent this time looping, if we have we will stop looping). A negative value means loop forever. */
	UPROPERTY(Category = Decorator, EditAnywhere, meta = (EditCondition = "bInfiniteLoop"))
	FValueOrBBKey_Float InfiniteLoopTimeoutTime;

	AIMODULE_API virtual uint16 GetInstanceMemorySize() const override;
	AIMODULE_API virtual void InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const override;
	AIMODULE_API virtual void CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const override;
	AIMODULE_API virtual void DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const override;
	AIMODULE_API virtual FString GetStaticDescription() const override;

#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
#endif // WITH_EDITOR

protected:

	AIMODULE_API virtual void OnNodeActivation(FBehaviorTreeSearchData& SearchData) override;
};
