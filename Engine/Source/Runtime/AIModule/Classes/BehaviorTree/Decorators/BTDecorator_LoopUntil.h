// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BehaviorTree/ValueOrBBKey.h"
#include "BehaviorTree/BTDecorator.h"
#include "BTDecorator_LoopUntil.generated.h"

class UBlackboardComponent;

struct FBTLoopUntilDecoratorMemory
{
	int32 SearchId = 0;
};

/**
 * Loop until decorator node.
 * A decorator node that loops execution until the child execution returns the required result
 */
UCLASS(HideCategories=(FlowControl, Condition), MinimalAPI)
class UBTDecorator_LoopUntil : public UBTDecorator
{
	GENERATED_BODY()

public:
	AIMODULE_API UBTDecorator_LoopUntil(const FObjectInitializer& ObjectInitializer);

private:
	AIMODULE_API virtual void OnNodeDeactivation(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type NodeResult) override;

	AIMODULE_API virtual uint16 GetInstanceMemorySize() const override;
	AIMODULE_API virtual void InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const override;
	AIMODULE_API virtual void CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const override;

#if WITH_EDITOR
	AIMODULE_API virtual FString GetErrorMessage() const;
	AIMODULE_API virtual FName GetNodeIconName() const override;
	AIMODULE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	void SetNodeName();

protected:
	UPROPERTY(EditAnywhere, Category=Decorator)
	FValueOrBBKey_Enum RequiredResult = EBTNodeResult::Succeeded;
};
