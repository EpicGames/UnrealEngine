// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/BTDecorator.h"
#include "BTDecorator_ReachedMoveGoal.generated.h"

/**
 * Reached Move Goal decorator node.
 * A decorator node that bases its condition on whether the AI controller's path following component returns that it has reached its goal.
 */
UCLASS(meta = (DeprecatedNode, DeprecationMessage = "Please use IsAtLocation decorator instead."), MinimalAPI)
class UBTDecorator_ReachedMoveGoal : public UBTDecorator
{
	GENERATED_UCLASS_BODY()

	AIMODULE_API virtual bool CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const override;

#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
#endif // WITH_EDITOR
};
