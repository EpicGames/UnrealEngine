// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameplayTagContainer.h"
#include "BehaviorTree/BTDecorator.h"
#include "BehaviorTree/ValueOrBBKey.h"
#include "BTDecorator_SetTagCooldown.generated.h"

/**
 * Set tag cooldown decorator node.
 * A decorator node that sets a gameplay tag cooldown.
 */
UCLASS(HideCategories=(Condition), MinimalAPI)
class UBTDecorator_SetTagCooldown : public UBTDecorator
{
	GENERATED_UCLASS_BODY()

	/** Gameplay tag that will be used for the cooldown. */
	UPROPERTY(Category = Decorator, EditAnywhere)
	FGameplayTag CooldownTag;

	/** Value we will add or set to the Cooldown tag when this task runs. */
	UPROPERTY(Category = Decorator, EditAnywhere)
	FValueOrBBKey_Float CooldownDuration;

	/** True if we are adding to any existing duration, false if we are setting the duration (potentially invalidating an existing end time). */
	UPROPERTY(Category = Decorator, EditAnywhere, DisplayName = AddToExistingDuration)
	FValueOrBBKey_Bool bAddToExistingDuration;

	AIMODULE_API virtual FString GetStaticDescription() const override;

#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
#endif // WITH_EDITOR

protected:
	AIMODULE_API virtual void OnNodeDeactivation(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type NodeResult) override;
};
