// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameplayTagContainer.h"
#include "BehaviorTree/BTDecorator.h"
#include "BehaviorTree/ValueOrBBKey.h"
#include "BTDecorator_TagCooldown.generated.h"

struct FBTTagCooldownDecoratorMemory
{	
	uint8 bRequestedRestart : 1;
};

/**
 * Cooldown decorator node.
 * A decorator node that bases its condition on whether a cooldown timer based on a gameplay tag has expired.
 */
UCLASS(HideCategories=(Condition), MinimalAPI)
class UBTDecorator_TagCooldown : public UBTDecorator
{
	GENERATED_UCLASS_BODY()

	/** Gameplay tag that will be used for the cooldown. */
	UPROPERTY(Category = Decorator, EditAnywhere)
	FGameplayTag CooldownTag;

	/** Value we will add or set to the Cooldown tag when this node is deactivated. */
	UPROPERTY(Category = Decorator, EditAnywhere, meta = (EditCondition = "bActivatesCooldown"))
	FValueOrBBKey_Float CooldownDuration;

	/** True if we are adding to any existing duration, false if we are setting the duration (potentially invalidating an existing end time). */
	UPROPERTY(Category = Decorator, EditAnywhere, DisplayName = AddToExistingDuration, meta = (EditCondition = "bActivatesCooldown"))
	FValueOrBBKey_Bool bAddToExistingDuration;

	/** Whether or not we are adding/setting to the cooldown tag's value when the decorator deactivates. */
	UPROPERTY(Category = Decorator, EditAnywhere, meta = (DisplayName = "Adds/Sets Cooldown on Deactivation"))
	FValueOrBBKey_Bool bActivatesCooldown;

	//~ Begin UObject Interface
	AIMODULE_API virtual void PostLoad() override;
	//~ End UObject Interface

	AIMODULE_API virtual bool CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const override;
	AIMODULE_API virtual void InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const override;
	AIMODULE_API virtual void CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const override;
	AIMODULE_API virtual uint16 GetInstanceMemorySize() const override;
	AIMODULE_API virtual void DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const override;
	AIMODULE_API virtual FString GetStaticDescription() const override;

#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
#endif // WITH_EDITOR

protected:

	AIMODULE_API virtual void OnNodeDeactivation(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type NodeResult) override;
	AIMODULE_API virtual void TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds) override;

private:
	bool HasCooldownFinished(const UBehaviorTreeComponent& OwnerComp) const;
};
