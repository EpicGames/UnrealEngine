// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/BTService.h"
#include "BTService_BlackboardBase.generated.h"

class UBehaviorTree;

UCLASS(Abstract, MinimalAPI)
class UBTService_BlackboardBase : public UBTService
{
	GENERATED_UCLASS_BODY()

	/** initialize any asset related data */
	AIMODULE_API virtual void InitializeFromAsset(UBehaviorTree& Asset) override;

	/** get name of selected blackboard key */
	FName GetSelectedBlackboardKey() const;

protected:

#if WITH_EDITOR
	AIMODULE_API virtual FString GetErrorMessage() const override;
#endif

	/** blackboard key selector */
	UPROPERTY(EditAnywhere, Category=Blackboard)
	struct FBlackboardKeySelector BlackboardKey;
};

//////////////////////////////////////////////////////////////////////////
// Inlines

inline FName UBTService_BlackboardBase::GetSelectedBlackboardKey() const
{
	return BlackboardKey.SelectedKeyName;
}
