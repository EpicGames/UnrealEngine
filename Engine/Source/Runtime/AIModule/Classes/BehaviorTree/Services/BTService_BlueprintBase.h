// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/BTService.h"
#include "BTService_BlueprintBase.generated.h"

class AActor;
class AAIController;
class APawn;
class UBehaviorTree;

/**
 *  Base class for blueprint based service nodes. Do NOT use it for creating native c++ classes!
 *
 *  When service receives Deactivation event, all latent actions associated this instance are being removed.
 *  This prevents from resuming activity started by Activation, but does not handle external events.
 *  Please use them safely (unregister at abort) and call IsServiceActive() when in doubt.
 */

UCLASS(Abstract, Blueprintable, MinimalAPI)
class UBTService_BlueprintBase : public UBTService
{
	GENERATED_UCLASS_BODY()

	AIMODULE_API virtual void DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const override;
	AIMODULE_API virtual void OnInstanceDestroyed(UBehaviorTreeComponent& OwnerComp) override;
	AIMODULE_API virtual void InitializeFromAsset(UBehaviorTree& Asset) override;

	AIMODULE_API virtual void SetOwner(AActor* ActorOwner) override;

#if WITH_EDITOR
	AIMODULE_API virtual bool UsesBlueprint() const override;
#endif

protected:
	/** Cached AIController owner of BehaviorTreeComponent. */
	UPROPERTY(Transient)
	TObjectPtr<AAIController> AIOwner;

	/** Cached actor owner of BehaviorTreeComponent. */
	UPROPERTY(Transient)
	TObjectPtr<AActor> ActorOwner;

	// Gets the description for our service
	AIMODULE_API virtual FString GetStaticServiceDescription() const override;

	/** properties with runtime values, stored only in class default object */
	TArray<FProperty*> PropertyData;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = Description)
	FString CustomDescription;
#endif // WITH_EDITORONLY_DATA

	/** show detailed information about properties */
	UPROPERTY(EditInstanceOnly, Category=Description)
	uint32 bShowPropertyDetails : 1;

	/** show detailed information about implemented events */
	UPROPERTY(EditInstanceOnly, Category = Description)
	uint32 bShowEventDetails : 1;

	/** set if ReceiveTick is implemented by blueprint */
	uint32 ReceiveTickImplementations : 2;
	
	/** set if ReceiveActivation is implemented by blueprint */
	uint32 ReceiveActivationImplementations : 2;

	/** set if ReceiveDeactivation is implemented by blueprint */
	uint32 ReceiveDeactivationImplementations : 2;

	/** set if ReceiveSearchStart is implemented by blueprint */
	uint32 ReceiveSearchStartImplementations : 2;

	AIMODULE_API virtual void OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	AIMODULE_API virtual void OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	AIMODULE_API virtual void TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds) override;
	AIMODULE_API virtual void OnSearchStart(FBehaviorTreeSearchData& SearchData) override;

	/** tick function
	 *	@Note that if both generic and AI event versions are implemented only the more
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent)
	AIMODULE_API void ReceiveTick(AActor* OwnerActor, float DeltaSeconds);

	/** task search enters branch of tree
	 *	@Note that if both generic and AI event versions are implemented only the more
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent)
	AIMODULE_API void ReceiveSearchStart(AActor* OwnerActor);

	/** service became active
	 *	@Note that if both generic and AI event versions are implemented only the more
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent)
	AIMODULE_API void ReceiveActivation(AActor* OwnerActor);

	/** service became inactive
	 *	@Note that if both generic and AI event versions are implemented only the more
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent)
	AIMODULE_API void ReceiveDeactivation(AActor* OwnerActor);

	/** Alternative AI version of ReceiveTick function.
	 *	@see ReceiveTick for more details
	 *	@Note that if both generic and AI event versions are implemented only the more
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent, Category = AI)
	AIMODULE_API void ReceiveTickAI(AAIController* OwnerController, APawn* ControlledPawn, float DeltaSeconds);

	/** Alternative AI version of ReceiveSearchStart function.
	 *	@see ReceiveSearchStart for more details
	 *	@Note that if both generic and AI event versions are implemented only the more
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent, Category = AI)
	AIMODULE_API void ReceiveSearchStartAI(AAIController* OwnerController, APawn* ControlledPawn);

	/** Alternative AI version of ReceiveActivation function.
	 *	@see ReceiveActivation for more details
	 *	@Note that if both generic and AI event versions are implemented only the more
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent, Category = AI)
	AIMODULE_API void ReceiveActivationAI(AAIController* OwnerController, APawn* ControlledPawn);

	/** Alternative AI version of ReceiveDeactivation function.
	 *	@see ReceiveDeactivation for more details
	 *	@Note that if both generic and AI event versions are implemented only the more
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent, Category = AI)
	AIMODULE_API void ReceiveDeactivationAI(AAIController* OwnerController, APawn* ControlledPawn);

	/** check if service is currently being active */
	UFUNCTION(BlueprintCallable, Category="AI|BehaviorTree")
	AIMODULE_API bool IsServiceActive() const;
};
