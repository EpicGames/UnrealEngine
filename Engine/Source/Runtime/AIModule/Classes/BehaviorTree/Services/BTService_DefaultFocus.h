// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "InputCoreTypes.h"
#include "BehaviorTree/Services/BTService_BlackboardBase.h"
#include "BTService_DefaultFocus.generated.h"

class AActor;
class UBlackboardComponent;

struct FBTFocusMemory
{
	AActor* FocusActorSet;
	FVector FocusLocationSet;
	bool bActorSet;

	void Reset()
	{
		FocusActorSet = nullptr;
		FocusLocationSet = FAISystem::InvalidLocation;
		bActorSet = false;
	}
};

/**
 * Default Focus service node.
 * A service node that automatically sets the AI controller's focus when it becomes active.
 */
UCLASS(hidecategories=(Service), MinimalAPI)
class UBTService_DefaultFocus : public UBTService_BlackboardBase
{
	GENERATED_BODY()

protected:
	// not exposed to users on purpose. Here to make reusing focus-setting mechanics by derived classes possible
	UPROPERTY()
	uint8 FocusPriority;

	AIMODULE_API UBTService_DefaultFocus(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual uint16 GetInstanceMemorySize() const override { return sizeof(FBTFocusMemory); }
	AIMODULE_API virtual void InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const override;
	AIMODULE_API virtual void CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const override;
	AIMODULE_API virtual void OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	AIMODULE_API virtual void OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;

	AIMODULE_API virtual FString GetStaticDescription() const override;

	AIMODULE_API EBlackboardNotificationResult OnBlackboardKeyValueChange(const UBlackboardComponent& Blackboard, FBlackboard::FKey ChangedKeyID);

#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
#endif // WITH_EDITOR
};
