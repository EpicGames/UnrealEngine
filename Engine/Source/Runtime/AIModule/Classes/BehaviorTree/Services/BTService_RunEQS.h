// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "BehaviorTree/Services/BTService_BlackboardBase.h"
#include "BTService_RunEQS.generated.h"

class UBehaviorTree;

struct FBTEQSServiceMemory
{
	/** Query request ID */
	int32 RequestID;
};

UCLASS(MinimalAPI)
class UBTService_RunEQS : public UBTService_BlackboardBase
{
	GENERATED_BODY()

protected:
	UPROPERTY(Category = EQS, EditAnywhere)
	FEQSParametrizedQueryExecutionRequest EQSRequest;

	UPROPERTY(Category = EQS, EditAnywhere)
	bool bUpdateBBOnFail = false;

	FQueryFinishedSignature QueryFinishedDelegate;

public:
	AIMODULE_API UBTService_RunEQS(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	AIMODULE_API virtual void InitializeFromAsset(UBehaviorTree& Asset) override;
	AIMODULE_API virtual void TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds) override;
	AIMODULE_API virtual void OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	AIMODULE_API virtual void InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const override;
	AIMODULE_API virtual void CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const override;
	virtual uint16 GetInstanceMemorySize() const override { return sizeof(FBTEQSServiceMemory); }

	AIMODULE_API virtual FString GetStaticDescription() const override;

#if WITH_EDITOR
	/** prepare query params */
	AIMODULE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

protected:
	AIMODULE_API void OnQueryFinished(TSharedPtr<FEnvQueryResult> Result);
};
