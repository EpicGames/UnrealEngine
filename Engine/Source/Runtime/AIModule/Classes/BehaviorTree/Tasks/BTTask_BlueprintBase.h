// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/BTTaskNode.h"
#include "BTTask_BlueprintBase.generated.h"

class AActor;
class AAIController;
class APawn;
class UBehaviorTree;

/**
 *  Base class for blueprint based task nodes. Do NOT use it for creating native c++ classes!
 *
 *  When task receives Abort event, all latent actions associated this instance are being removed.
 *  This prevents from resuming activity started by Execute, but does not handle external events.
 *  Please use them safely (unregister at abort) and call IsTaskExecuting() when in doubt.
 */

UCLASS(Abstract, Blueprintable, MinimalAPI)
class UBTTask_BlueprintBase : public UBTTaskNode
{
	GENERATED_UCLASS_BODY()
	
	AIMODULE_API virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	AIMODULE_API virtual EBTNodeResult::Type AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;

	AIMODULE_API virtual FString GetStaticDescription() const override;
	AIMODULE_API virtual void DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const override;
	AIMODULE_API virtual void OnInstanceDestroyed(UBehaviorTreeComponent& OwnerComp) override;
	AIMODULE_API virtual void OnTaskFinished(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTNodeResult::Type TaskResult) override;
	AIMODULE_API virtual void InitializeFromAsset(UBehaviorTree& Asset) override;

	AIMODULE_API virtual void SetOwner(AActor* ActorOwner) override;

#if WITH_EDITOR
	AIMODULE_API virtual bool UsesBlueprint() const override;
#endif

protected:
	/** Cached AIController owner of BehaviorTreeComponent. */
	UPROPERTY(Transient)
	TObjectPtr<AAIController> AIOwner;

	/** Cached actor owner of BehaviorTreeComponent. */
	UPROPERTY(Transient)
	TObjectPtr<AActor> ActorOwner;

	/** If any of the Tick functions is implemented, how often should they be ticked.
	 *	Values < 0 mean 'every tick'. */
	UPROPERTY(EditAnywhere, Category = Task)
	FIntervalCountdown TickInterval;
	
	/** temporary variable for ReceiveExecute(Abort)-FinishExecute(Abort) chain */
	mutable TEnumAsByte<EBTNodeResult::Type> CurrentCallResult;

	/** properties that should be copied */
	TArray<FProperty*> PropertyData;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = Description)
	FString CustomDescription;
#endif // WITH_EDITORONLY_DATA

	/** show detailed information about properties */
	UPROPERTY(EditInstanceOnly, Category=Description)
	uint32 bShowPropertyDetails : 1;

	/** set if ReceiveTick is implemented by blueprint */
	uint32 ReceiveTickImplementations : 2;

	/** set if ReceiveExecute is implemented by blueprint */
	uint32 ReceiveExecuteImplementations : 2;

	/** set if ReceiveAbort is implemented by blueprint */
	uint32 ReceiveAbortImplementations : 2;

	/** set when task enters Aborting state */
	uint32 bIsAborting : 1;

	/** if set, execution is inside blueprint's ReceiveExecute(Abort) event
	  * FinishExecute(Abort) function should store their result in CurrentCallResult variable */
	mutable uint32 bStoreFinishResult : 1;

	/** entry point, task will stay active until FinishExecute is called.
	 *	@Note that if both generic and AI event versions are implemented only the more 
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent)
	AIMODULE_API void ReceiveExecute(AActor* OwnerActor);

	/** if blueprint graph contains this event, task will stay active until FinishAbort is called
	 *	@Note that if both generic and AI event versions are implemented only the more
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent)
	AIMODULE_API void ReceiveAbort(AActor* OwnerActor);

	/** tick function
	 *	@Note that if both generic and AI event versions are implemented only the more
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent)
	AIMODULE_API void ReceiveTick(AActor* OwnerActor, float DeltaSeconds);

	/** Alternative AI version of ReceiveExecute
	*	@see ReceiveExecute for more details
	 *	@Note that if both generic and AI event versions are implemented only the more
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent, Category = AI)
	AIMODULE_API void ReceiveExecuteAI(AAIController* OwnerController, APawn* ControlledPawn);

	/** Alternative AI version of ReceiveAbort
	 *	@see ReceiveAbort for more details
	 *	@Note that if both generic and AI event versions are implemented only the more
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent, Category = AI)
	AIMODULE_API void ReceiveAbortAI(AAIController* OwnerController, APawn* ControlledPawn);

	/** Alternative AI version of tick function.
	 *	@see ReceiveTick for more details
	 *	@Note that if both generic and AI event versions are implemented only the more
	 *	suitable one will be called, meaning the AI version if called for AI, generic one otherwise */
	UFUNCTION(BlueprintImplementableEvent, Category = AI)
	AIMODULE_API void ReceiveTickAI(AAIController* OwnerController, APawn* ControlledPawn, float DeltaSeconds);

	/** finishes task execution with Success or Fail result */
	UFUNCTION(BlueprintCallable, Category="AI|BehaviorTree")
	AIMODULE_API void FinishExecute(bool bSuccess);

	/** aborts task execution */
	UFUNCTION(BlueprintCallable, Category="AI|BehaviorTree")
	AIMODULE_API void FinishAbort();

	/** task execution will be finished (with result 'Success') after receiving specified message */
	UFUNCTION(BlueprintCallable, Category="AI|BehaviorTree")
	AIMODULE_API void SetFinishOnMessage(FName MessageName);

	/** task execution will be finished (with result 'Success') after receiving specified message with indicated ID */
	UFUNCTION(BlueprintCallable, Category="AI|BehaviorTree")
	AIMODULE_API void SetFinishOnMessageWithId(FName MessageName, int32 RequestID = -1);

	/** check if task is currently being executed */
	UFUNCTION(BlueprintCallable, Category="AI|BehaviorTree")
	AIMODULE_API bool IsTaskExecuting() const;
	
	/** check if task is currently being aborted */
	UFUNCTION(BlueprintCallable, Category = "AI|BehaviorTree")
	AIMODULE_API bool IsTaskAborting() const;

	/** ticks this task */
	AIMODULE_API virtual void TickTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds) override;
};
