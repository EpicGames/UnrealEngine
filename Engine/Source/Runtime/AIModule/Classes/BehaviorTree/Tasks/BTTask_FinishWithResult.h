// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "BehaviorTree/BTTaskNode.h"
#include "BehaviorTree/ValueOrBBKey.h"
#include "BTTask_FinishWithResult.generated.h"

/**
 * Instantly finishes with given result
 */
UCLASS(MinimalAPI)
class UBTTask_FinishWithResult : public UBTTaskNode
{
	GENERATED_BODY()

public:
	AIMODULE_API UBTTask_FinishWithResult(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	AIMODULE_API virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	AIMODULE_API virtual FString GetStaticDescription() const override;
	
protected:
	UPROPERTY(Category = Result, EditAnywhere)
	FValueOrBBKey_Enum Result = EBTNodeResult::Type::Succeeded;
};
