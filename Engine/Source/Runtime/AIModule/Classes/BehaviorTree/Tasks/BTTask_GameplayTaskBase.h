// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/BTTaskNode.h"
#include "BehaviorTree/ValueOrBBKey.h"
#include "BTTask_GameplayTaskBase.generated.h"

struct FBTGameplayTaskMemory
{
	TWeakObjectPtr<UAITask> Task;
	uint8 bObserverCanFinishTask : 1;
};

/**
 * Base class for managing gameplay tasks
 * Since AITask doesn't have any kind of success/failed results, default implemenation will only return EBTNode::Succeeded
 *
 * In your ExecuteTask:
 * - use NewBTAITask() helper to create task
 * - initialize task with values if needed
 * - use StartGameplayTask() helper to execute and get node result
 */
UCLASS(Abstract, MinimalAPI)
class UBTTask_GameplayTaskBase : public UBTTaskNode
{
	GENERATED_UCLASS_BODY()

	AIMODULE_API virtual EBTNodeResult::Type AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	AIMODULE_API virtual void OnTaskFinished(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTNodeResult::Type TaskResult) override;
	AIMODULE_API virtual void OnGameplayTaskDeactivated(UGameplayTask& Task) override;
	AIMODULE_API virtual uint16 GetInstanceMemorySize() const override;
	AIMODULE_API virtual void InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const override;
	AIMODULE_API virtual void CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const override;

protected:

	/** if set, behavior tree task will wait until gameplay tasks finishes */
	UPROPERTY(EditAnywhere, Category = Task, AdvancedDisplay, DisplayName = WaitForGameplayTask)
	FValueOrBBKey_Bool bWaitForGameplayTask;

	/** start task and initialize FBTGameplayTaskMemory memory block */
	AIMODULE_API EBTNodeResult::Type StartGameplayTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, UAITask& Task);

	/** get finish result from task */
	AIMODULE_API virtual EBTNodeResult::Type DetermineGameplayTaskResult(UAITask& Task) const;
};
