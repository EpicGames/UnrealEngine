// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/BTTaskNode.h"
#include "BehaviorTree/ValueOrBBKey.h"
#include "BTTask_MakeNoise.generated.h"

/**
 * Make Noise task node.
 * A task node that calls MakeNoise() on this Pawn when executed.
 */
UCLASS(MinimalAPI)
class UBTTask_MakeNoise : public UBTTaskNode
{
	GENERATED_UCLASS_BODY()

	/** Loudness of generated noise */
	UPROPERTY(Category=Node, EditAnywhere, meta=(ClampMin = "0", ClampMax = "1", UIMin = "0", UIMax = "1"), DisplayName = Loudness)
	FValueOrBBKey_Float Loudnes;

	AIMODULE_API virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;

#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
#endif // WITH_EDITOR
};
