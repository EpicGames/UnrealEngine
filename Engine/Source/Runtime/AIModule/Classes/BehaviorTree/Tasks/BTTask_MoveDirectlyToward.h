// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/Tasks/BTTask_MoveTo.h"
#include "BTTask_MoveDirectlyToward.generated.h"

class UBehaviorTree;

/**
 * Move Directly Toward task node.
 * Moves the AI pawn toward the specified Actor or Location (Vector) blackboard entry in a straight line, without regard to any navigation system. If you need the AI to navigate, use the "Move To" node instead.
 */
UCLASS(config=Game, MinimalAPI)
class UBTTask_MoveDirectlyToward : public UBTTask_MoveTo
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
#endif // WITH_EDITOR
};
