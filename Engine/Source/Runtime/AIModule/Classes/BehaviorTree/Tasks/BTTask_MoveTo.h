// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "InputCoreTypes.h"
#include "Templates/SubclassOf.h"
#include "AITypes.h"
#include "BehaviorTree/Tasks/BTTask_BlackboardBase.h"
#include "BehaviorTree/ValueOrBBKey.h"
#include "NavFilters/NavigationQueryFilter.h"
#include "BTTask_MoveTo.generated.h"

class UAITask_MoveTo;
class UBlackboardComponent;

struct FBTMoveToTaskMemory
{
	/** Move request ID */
	FAIRequestID MoveRequestID;

	FDelegateHandle BBObserverDelegateHandle;
	FVector PreviousGoalLocation;

	TWeakObjectPtr<UAITask_MoveTo> Task;

	uint8 bObserverCanFinishTask : 1;
};

/**
 * Move To task node.
 * Moves the AI pawn toward the specified Actor or Location blackboard entry using the navigation system.
 */
UCLASS(config=Game, MinimalAPI)
class UBTTask_MoveTo : public UBTTask_BlackboardBase
{
	GENERATED_UCLASS_BODY()

	/** fixed distance added to threshold between AI and goal location in destination reach test */
	UPROPERTY(config, Category = Node, EditAnywhere, meta=(ClampMin = "0.0", UIMin="0.0"))
	FValueOrBBKey_Float AcceptableRadius;

	/** "None" will result in default filter being used */
	UPROPERTY(Category = Node, EditAnywhere)
	FValueOrBBKey_Class FilterClass = TSubclassOf<UNavigationQueryFilter>();

	/** if task is expected to react to changes to location represented by BB key 
	 *	this property can be used to tweak sensitivity of the mechanism. Value is 
	 *	recommended to be less than AcceptableRadius */
	UPROPERTY(Category = Blackboard, EditAnywhere, meta = (EditCondition = "bObserveBlackboardValue", ClampMin = "1", UIMin = "1"))
	FValueOrBBKey_Float ObservedBlackboardValueTolerance;

	UPROPERTY(Category = Node, EditAnywhere, DisplayName = AllowStrafe)
	FValueOrBBKey_Bool bAllowStrafe;

	/** if set, use incomplete path when goal can't be reached */
	UPROPERTY(Category = Node, EditAnywhere, AdvancedDisplay,  DisplayName = AllowPartialPath)
	FValueOrBBKey_Bool bAllowPartialPath;

	/** if set, path to goal actor will update itself when actor moves */
	UPROPERTY(Category = Node, EditAnywhere, AdvancedDisplay,  DisplayName = TrackMovingGoal)
	FValueOrBBKey_Bool bTrackMovingGoal;

	/** if set, the goal location will need to be navigable */
	UPROPERTY(Category = Node, EditAnywhere, AdvancedDisplay,  DisplayName = RequireNavigableEndLocation)
	FValueOrBBKey_Bool bRequireNavigableEndLocation;

	/** if set, goal location will be projected on navigation data (navmesh) before using */
	UPROPERTY(Category = Node, EditAnywhere, AdvancedDisplay,  DisplayName = ProjectGoalLocation)
	FValueOrBBKey_Bool bProjectGoalLocation;

	/** if set, radius of AI's capsule will be added to threshold between AI and goal location in destination reach test  */
	UPROPERTY(Category = Node, EditAnywhere,  DisplayName = ReachTestIncludesAgentRadius)
	FValueOrBBKey_Bool bReachTestIncludesAgentRadius;
	
	/** if set, radius of goal's capsule will be added to threshold between AI and goal location in destination reach test  */
	UPROPERTY(Category = Node, EditAnywhere,  DisplayName = ReachTestIncludesGoalRadius)
	FValueOrBBKey_Bool bReachTestIncludesGoalRadius;

	/** if set, the path request will start from the end of the previous path (if any), and the generated path will be merged with the remaining points of the previous path */
	UPROPERTY(Category = Node, EditAnywhere,  DisplayName = StartFromPreviousPath)
	FValueOrBBKey_Bool bStartFromPreviousPath;

	/** if set, move will use pathfinding. Not exposed on purpose, please use BTTask_MoveDirectlyToward */
	uint32 bUsePathfinding : 1;

	UPROPERTY()
	uint32 bObserveBlackboardValue : 1;

	AIMODULE_API virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	AIMODULE_API virtual EBTNodeResult::Type AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	AIMODULE_API virtual void OnTaskFinished(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTNodeResult::Type TaskResult) override;
	AIMODULE_API virtual uint16 GetInstanceMemorySize() const override;
	AIMODULE_API virtual void InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const override;
	AIMODULE_API virtual void CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const override;

	AIMODULE_API virtual void OnGameplayTaskDeactivated(UGameplayTask& Task) override;
	AIMODULE_API virtual void OnMessage(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, FName Message, int32 RequestID, bool bSuccess) override;
	AIMODULE_API EBlackboardNotificationResult OnBlackboardValueChange(const UBlackboardComponent& Blackboard, FBlackboard::FKey ChangedKeyID);

	AIMODULE_API virtual void DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const override;
	AIMODULE_API virtual FString GetStaticDescription() const override;

#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
#endif // WITH_EDITOR

protected:

	AIMODULE_API virtual EBTNodeResult::Type PerformMoveTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory);
	
	/** prepares move task for activation */
	AIMODULE_API virtual UAITask_MoveTo* PrepareMoveTask(UBehaviorTreeComponent& OwnerComp, UAITask_MoveTo* ExistingTask, FAIMoveRequest& MoveRequest);
};
