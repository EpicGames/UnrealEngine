// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "TimerManager.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/AnimationAsset.h"
#include "BehaviorTree/BTTaskNode.h"
#include "BehaviorTree/ValueOrBBKey.h"
#include "BTTask_PlayAnimation.generated.h"

/**
 *	Play indicated AnimationAsset on Pawn controlled by BT 
 *	Note that this node is generic and is handing multiple special cases,
 *	If you want a more efficient solution you'll need to implement it yourself (or wait for our BTTask_PlayCharacterAnimation)
 */
UCLASS(MinimalAPI)
class UBTTask_PlayAnimation : public UBTTaskNode
{
	GENERATED_UCLASS_BODY()

	/** Animation asset to play. Note that it needs to match the skeleton of pawn this BT is controlling */
	UPROPERTY(Category = Node, EditAnywhere)
	FValueOrBBKey_Object AnimationToPlay = TObjectPtr<UAnimationAsset>();

	UPROPERTY(Category = Node, EditAnywhere, DisplayName = Looping)
	FValueOrBBKey_Bool bLooping = false;

	/** if true the task will just trigger the animation and instantly finish. Fire and Forget. */
	UPROPERTY(Category = Node, EditAnywhere, DisplayName = NonBlocking)
	FValueOrBBKey_Bool bNonBlocking = false;

	UPROPERTY()
	TObjectPtr<UBehaviorTreeComponent> MyOwnerComp;

	UPROPERTY()
	TObjectPtr<USkeletalMeshComponent> CachedSkelMesh;

	EAnimationMode::Type PreviousAnimationMode;

	FTimerDelegate TimerDelegate;
	FTimerHandle TimerHandle;

	AIMODULE_API virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	AIMODULE_API virtual EBTNodeResult::Type AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	AIMODULE_API virtual FString GetStaticDescription() const override;

	AIMODULE_API void OnAnimationTimerDone();
	
#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
#endif // WITH_EDITOR

protected:
	AIMODULE_API void CleanUp(UBehaviorTreeComponent& OwnerComp);
};
