// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/BTTaskNode.h"
#include "BehaviorTree/ValueOrBBKey.h"
#include "Sound/SoundCue.h"
#include "BTTask_PlaySound.generated.h"

class USoundCue;

/**
 * Play Sound task node.
 * Plays the specified sound when executed.
 */
UCLASS(MinimalAPI)
class UBTTask_PlaySound : public UBTTaskNode
{
	GENERATED_UCLASS_BODY()

	/** CUE to play */
	UPROPERTY(Category=Node, EditAnywhere)
	FValueOrBBKey_Object SoundToPlay = TObjectPtr<USoundCue>();

	AIMODULE_API virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	AIMODULE_API virtual FString GetStaticDescription() const override;

#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
#endif // WITH_EDITOR
};
