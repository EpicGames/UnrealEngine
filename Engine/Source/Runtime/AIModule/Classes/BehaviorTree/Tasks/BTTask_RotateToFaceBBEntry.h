// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/Tasks/BTTask_BlackboardBase.h"
#include "BehaviorTree/Services/BTService_DefaultFocus.h"
#include "BehaviorTree/ValueOrBBKey.h"
#include "BTTask_RotateToFaceBBEntry.generated.h"

class AAIController;

/**
 * 
 */
UCLASS(config = Game, MinimalAPI)
class UBTTask_RotateToFaceBBEntry : public UBTTask_BlackboardBase
{
	GENERATED_UCLASS_BODY()

protected:
	/** Success condition precision in degrees */
	UPROPERTY(config, Category = Node, EditAnywhere, meta = (ClampMin = "0.0"))
	FValueOrBBKey_Float Precision;

public:

	AIMODULE_API virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	AIMODULE_API virtual void TickTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds) override;
	AIMODULE_API virtual EBTNodeResult::Type AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	AIMODULE_API virtual void DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const override;
	AIMODULE_API virtual FString GetStaticDescription() const override;
	
	virtual uint16 GetInstanceMemorySize() const override { return sizeof(FBTFocusMemory); }
	AIMODULE_API virtual void InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const override;
	AIMODULE_API virtual void CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const override;

protected:

	AIMODULE_API float GetPrecisionDot(const UBehaviorTreeComponent& OwnerComp) const;
	AIMODULE_API void CleanUp(AAIController& AIController, uint8* NodeMemory);
};
