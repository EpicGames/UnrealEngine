// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/BTTaskNode.h"
#include "BehaviorTree/BehaviorTree.h"
#include "BTTask_RunBehavior.generated.h"

/**
 * RunBehavior task allows pushing subtrees on execution stack.
 * Subtree asset can't be changed in runtime! 
 *
 * This limitation is caused by support for subtree's root level decorators,
 * which are injected into parent tree, and structure of running tree
 * cannot be modified in runtime (see: BTNode: ExecutionIndex, MemoryOffset)
 *
 * Use RunBehaviorDynamic task for subtrees that need to be changed in runtime.
 */

UCLASS(MinimalAPI)
class UBTTask_RunBehavior : public UBTTaskNode
{
	GENERATED_UCLASS_BODY()

	AIMODULE_API virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	AIMODULE_API virtual FString GetStaticDescription() const override;

#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
	AIMODULE_API virtual const UObject* GetAssociatedAsset(TOptional<FBehaviorTreeNodeDebugContext> DebugContext) const override;
#endif // WITH_EDITOR

	/** @returns number of injected nodes */
	int32 GetInjectedNodesCount() const;

	/** @returns subtree asset */
	UBehaviorTree* GetSubtreeAsset() const;

protected:

	/** behavior to run */
	UPROPERTY(Category = Node, EditAnywhere, BlueprintReadOnly)
	TObjectPtr<UBehaviorTree> BehaviorAsset;

	/** called when subtree is removed from active stack */
	AIMODULE_API virtual void OnSubtreeDeactivated(UBehaviorTreeComponent& OwnerComp, EBTNodeResult::Type NodeResult);
};

//////////////////////////////////////////////////////////////////////////
// Inlines

inline UBehaviorTree* UBTTask_RunBehavior::GetSubtreeAsset() const
{
	return BehaviorAsset;
}

inline int32 UBTTask_RunBehavior::GetInjectedNodesCount() const
{
	return BehaviorAsset ? BehaviorAsset->RootDecorators.Num() : 0;
}
