// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameplayTagContainer.h"
#include "BehaviorTree/BTTaskNode.h"
#include "BTTask_RunBehaviorDynamic.generated.h"

class UBehaviorTree;

/**
 * RunBehaviorDynamic task allows pushing subtrees on execution stack.
 * Subtree asset can be assigned at runtime with SetDynamicSubtree function of BehaviorTreeComponent.
 *
 * Does NOT support subtree's root level decorators!
 */

UCLASS(MinimalAPI)
class UBTTask_RunBehaviorDynamic : public UBTTaskNode
{
	GENERATED_UCLASS_BODY()

	AIMODULE_API virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	AIMODULE_API virtual void OnInstanceCreated(UBehaviorTreeComponent& OwnerComp) override;
	AIMODULE_API virtual FString GetStaticDescription() const override;
	AIMODULE_API virtual void DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const override;

#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
	AIMODULE_API virtual const UObject* GetAssociatedAsset(TOptional<FBehaviorTreeNodeDebugContext> DebugContext) const override;

	UE_DEPRECATED(5.6, "GetBehaviorAssetFromRuntimeValue has been replaced with the more generic GetAssociatedAsset.")
	AIMODULE_API UBehaviorTree* GetBehaviorAssetFromRuntimeValue(const FString& RuntimeValue) const;

#endif // WITH_EDITOR

	bool HasMatchingTag(const FGameplayTag& Tag) const;
	const FGameplayTag& GetInjectionTag() const;
	AIMODULE_API bool SetBehaviorAsset(UBehaviorTree* NewBehaviorAsset);
	
	/** @returns default subtree asset */
	UBehaviorTree* GetDefaultBehaviorAsset() const;

protected:

	/** Gameplay tag that will identify this task for subtree injection */
	UPROPERTY(Category=Node, EditAnywhere)
	FGameplayTag InjectionTag;

	/** default behavior to run */
	UPROPERTY(Category=Node, EditAnywhere, BlueprintReadOnly)
	TObjectPtr<UBehaviorTree> DefaultBehaviorAsset;

	/** current subtree */
	UPROPERTY(BlueprintReadOnly, Category = Node)
	TObjectPtr<UBehaviorTree> BehaviorAsset;

	/** called when subtree is removed from active stack */
	AIMODULE_API virtual void OnSubtreeDeactivated(UBehaviorTreeComponent& OwnerComp, EBTNodeResult::Type NodeResult);
};

//////////////////////////////////////////////////////////////////////////
// Inlines

inline bool UBTTask_RunBehaviorDynamic::HasMatchingTag(const FGameplayTag& Tag) const
{
	return InjectionTag == Tag;
}

inline UBehaviorTree* UBTTask_RunBehaviorDynamic::GetDefaultBehaviorAsset() const
{
	return DefaultBehaviorAsset;
}

inline const FGameplayTag& UBTTask_RunBehaviorDynamic::GetInjectionTag() const
{
	return InjectionTag;
}
