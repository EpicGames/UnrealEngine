// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "BehaviorTree/Tasks/BTTask_BlackboardBase.h"
#include "BTTask_RunEQSQuery.generated.h"

class UBehaviorTree;
class UEnvQuery;

struct FBTEnvQueryTaskMemory
{
	/** Query request ID */
	int32 RequestID;
};

/**
 * Run Environment Query System Query task node.
 * Runs the specified environment query when executed.
 */
UCLASS(MinimalAPI)
class UBTTask_RunEQSQuery : public UBTTask_BlackboardBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Node, meta=(InlineEditConditionToggle))
	bool bUseBBKey;

	UPROPERTY(Category = EQS, EditAnywhere)
	FEQSParametrizedQueryExecutionRequest EQSRequest;

	UPROPERTY(Category = EQS, EditAnywhere)
	bool bUpdateBBOnFail = false;

	FQueryFinishedSignature QueryFinishedDelegate;

	AIMODULE_API virtual void InitializeFromAsset(UBehaviorTree& Asset) override;
	AIMODULE_API virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	AIMODULE_API virtual EBTNodeResult::Type AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;

	AIMODULE_API virtual void DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const override;
	AIMODULE_API virtual FString GetStaticDescription() const override;
	AIMODULE_API virtual uint16 GetInstanceMemorySize() const override;
	AIMODULE_API virtual void InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const override;
	AIMODULE_API virtual void CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const override;

	/** finish task */
	AIMODULE_API void OnQueryFinished(TSharedPtr<FEnvQueryResult> Result);

#if WITH_EDITOR
	/** prepare query params */
	AIMODULE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	AIMODULE_API virtual FName GetNodeIconName() const override;
#endif

protected:

	/** gather all filters from existing EnvQueryItemTypes */
	AIMODULE_API void CollectKeyFilters();
};
