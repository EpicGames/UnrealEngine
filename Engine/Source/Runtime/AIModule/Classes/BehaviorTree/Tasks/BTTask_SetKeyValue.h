// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "BehaviorTree/Tasks/BTTask_BlackboardBase.h"
#include "BehaviorTree/ValueOrBBKey.h"

#include "BTTask_SetKeyValue.generated.h"

UCLASS(DisplayName = "Set Bool Key")
class UBTTask_SetKeyValueBool : public UBTTask_BlackboardBase
{
	GENERATED_BODY()

public:
	UBTTask_SetKeyValueBool(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual FString GetStaticDescription() const override;

private:
	UPROPERTY(EditAnywhere, Category = Blackboard)
	FValueOrBBKey_Bool Value;
};

UCLASS(DisplayName = "Set Class Key")
class UBTTask_SetKeyValueClass : public UBTTask_BlackboardBase
{
	GENERATED_BODY()

public:
	UBTTask_SetKeyValueClass(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual FString GetStaticDescription() const override;

	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
private:
	UPROPERTY(EditAnywhere, Category = Blackboard, meta = (AllowAbstract = "true", NoClear, ForceRebuildProperty="Value"))
	TSubclassOf<UObject> BaseClass = UObject::StaticClass();

	UPROPERTY(EditAnywhere, Category = Blackboard)
	FValueOrBBKey_Class Value;
};

UCLASS(DisplayName = "Set Enum Key")
class UBTTask_SetKeyValueEnum : public UBTTask_BlackboardBase
{
	GENERATED_BODY()

public:
	UBTTask_SetKeyValueEnum(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual FString GetStaticDescription() const override;

	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
private:
	UPROPERTY(EditAnywhere, Category = Blackboard, meta = (AllowAbstract = "true", NoClear, ForceRebuildProperty = "Value"))
	TObjectPtr<UEnum> EnumType;

	UPROPERTY(EditAnywhere, Category = Blackboard)
	FValueOrBBKey_Enum Value;
};

UCLASS(DisplayName = "Set Int Key")
class UBTTask_SetKeyValueInt32 : public UBTTask_BlackboardBase
{
	GENERATED_BODY()

public:
	UBTTask_SetKeyValueInt32(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual FString GetStaticDescription() const override;

private:
	UPROPERTY(EditAnywhere, Category = Blackboard)
	FValueOrBBKey_Int32 Value;
};

UCLASS(DisplayName = "Set Float Key")
class UBTTask_SetKeyValueFloat : public UBTTask_BlackboardBase
{
	GENERATED_BODY()

public:
	UBTTask_SetKeyValueFloat(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual FString GetStaticDescription() const override;

private:
	UPROPERTY(EditAnywhere, Category = Blackboard)
	FValueOrBBKey_Float Value;
};

UCLASS(DisplayName = "Set Name Key")
class UBTTask_SetKeyValueName : public UBTTask_BlackboardBase
{
	GENERATED_BODY()

public:
	UBTTask_SetKeyValueName(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual FString GetStaticDescription() const override;

private:
	UPROPERTY(EditAnywhere, Category = Blackboard)
	FValueOrBBKey_Name Value;
};

UCLASS(DisplayName = "Set String Key")
class UBTTask_SetKeyValueString : public UBTTask_BlackboardBase
{
	GENERATED_BODY()

public:
	UBTTask_SetKeyValueString(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual FString GetStaticDescription() const override;

private:
	UPROPERTY(EditAnywhere, Category = Blackboard)
	FValueOrBBKey_String Value;
};

UCLASS(DisplayName = "Set Object Key")
class UBTTask_SetKeyValueObject : public UBTTask_BlackboardBase
{
	GENERATED_BODY()

public:
	UBTTask_SetKeyValueObject(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual FString GetStaticDescription() const override;

	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
private:
	UPROPERTY(EditAnywhere, Category = Blackboard, meta = (AllowAbstract = "true", NoClear, ForceRebuildProperty = "Value"))
	TSubclassOf<UObject> BaseClass = UObject::StaticClass();

	UPROPERTY(EditAnywhere, Category = Blackboard)
	FValueOrBBKey_Object Value;
};

UCLASS(DisplayName = "Set Rotator Key")
class UBTTask_SetKeyValueRotator : public UBTTask_BlackboardBase
{
	GENERATED_BODY()

public:
	UBTTask_SetKeyValueRotator(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual FString GetStaticDescription() const override;

private:
	UPROPERTY(EditAnywhere, Category = Blackboard)
	FValueOrBBKey_Rotator Value;
};

UCLASS(DisplayName = "Set Struct Key")
class UBTTask_SetKeyValueStruct : public UBTTask_BlackboardBase
{
	GENERATED_BODY()

public:
	UBTTask_SetKeyValueStruct(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual FString GetStaticDescription() const override;

	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
private:
	UPROPERTY(EditAnywhere, Category = Blackboard, meta = (ForceRebuildProperty = "Value"))
	TObjectPtr<UScriptStruct> StructType;

	UPROPERTY(EditAnywhere, Category = Blackboard)
	FValueOrBBKey_Struct Value;
};

UCLASS(DisplayName = "Set Vector Key")
class UBTTask_SetKeyValueVector : public UBTTask_BlackboardBase
{
	GENERATED_BODY()

public:
	UBTTask_SetKeyValueVector(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual FString GetStaticDescription() const override;

private:
	UPROPERTY(EditAnywhere, Category = Blackboard)
	FValueOrBBKey_Vector Value;
};
