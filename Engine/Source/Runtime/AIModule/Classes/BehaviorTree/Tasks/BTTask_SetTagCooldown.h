// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameplayTagContainer.h"
#include "BehaviorTree/BTTaskNode.h"
#include "BehaviorTree/ValueOrBBKey.h"
#include "BTTask_SetTagCooldown.generated.h"

/**
 * Cooldown task node.
 * Sets a cooldown tag value.  Use with cooldown tag decorators to prevent behavior tree execution.
 */
UCLASS(MinimalAPI)
class UBTTask_SetTagCooldown : public UBTTaskNode
{
	GENERATED_UCLASS_BODY()

	/** Gameplay tag that will be used for the cooldown. */
	UPROPERTY(Category = Cooldown, EditAnywhere)
	FGameplayTag CooldownTag;

	/** True if we are adding to any existing duration, false if we are setting the duration (potentially invalidating an existing end time). */
	UPROPERTY(Category = Cooldown, EditAnywhere, DisplayName = AddToExistingDuration)
	FValueOrBBKey_Bool bAddToExistingDuration;

	/** Value we will add or set to the Cooldown tag when this task runs. */
	UPROPERTY(Category = Cooldown, EditAnywhere)
	FValueOrBBKey_Float CooldownDuration;

	AIMODULE_API virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	AIMODULE_API virtual FString GetStaticDescription() const override;

#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
#endif // WITH_EDITOR
};
