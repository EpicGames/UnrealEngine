// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/BTTaskNode.h"
#include "BehaviorTree/ValueOrBBKey.h"
#include "BTTask_Wait.generated.h"

struct FBTWaitTaskMemory
{
};

/**
 * Wait task node.
 * Wait for the specified time when executed.
 */
UCLASS(MinimalAPI)
class UBTTask_Wait : public UBTTaskNode
{
	GENERATED_UCLASS_BODY()

	/** wait time in seconds */
	UPROPERTY(Category = Wait, EditAnywhere, meta = (ClampMin = "0.0", UIMin = "0.0"))
	FValueOrBBKey_Float WaitTime;

	/** allows adding random time to wait time */
	UPROPERTY(Category = Wait, EditAnywhere, meta = (UIMin = 0, ClampMin = 0))
	FValueOrBBKey_Float RandomDeviation;

	AIMODULE_API virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	AIMODULE_API virtual void DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const override;
	AIMODULE_API virtual FString GetStaticDescription() const override;

#if WITH_EDITOR
	AIMODULE_API virtual FName GetNodeIconName() const override;
#endif // WITH_EDITOR

protected:

	AIMODULE_API virtual void TickTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds) override;
};
