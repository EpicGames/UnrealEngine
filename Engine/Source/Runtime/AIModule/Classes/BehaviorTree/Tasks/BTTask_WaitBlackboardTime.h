// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BehaviorTree/Tasks/BTTask_Wait.h"
#include "BTTask_WaitBlackboardTime.generated.h"

class UBehaviorTree;

/**
* DEPRECATED Replace with UBTTask_Wait that now accepts blackboard keys
 * Wait task node.
 * Wait for the time specified by a Blackboard key when executed.
 */
UCLASS(hidecategories=Wait, hidden, MinimalAPI)
class UBTTask_WaitBlackboardTime : public UBTTask_Wait
{
	GENERATED_UCLASS_BODY()

	AIMODULE_API virtual void InitializeFromAsset(UBehaviorTree& Asset) override;

	/** get name of selected blackboard key */
	FName GetSelectedBlackboardKey() const;


protected:

#if WITH_EDITOR
	AIMODULE_API virtual FString GetErrorMessage() const override;
#endif

	/** blackboard key selector */
	UPROPERTY(EditAnywhere, Category=Blackboard)
	struct FBlackboardKeySelector BlackboardKey;
	
};

//////////////////////////////////////////////////////////////////////////
// Inlines

inline FName UBTTask_WaitBlackboardTime::GetSelectedBlackboardKey() const
{
	return BlackboardKey.SelectedKeyName;
}
