// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BehaviorTree/BehaviorTreeComponent.h"
#include "BehaviorTree/Blackboard/BlackboardKey.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "GameplayTagContainer.h"
#include "Misc/Guid.h"
#include "StructUtils/InstancedStruct.h"
#include "StructUtils/StructView.h"

#include "ValueOrBBKey.generated.h"

class UBlackboardKeyType;
class FValueOrBBKeyDetails;

struct FPropertyTag;
struct FConstStructView;

// Drop in replacement for Property in bt nodes that allows easy binding to blackboard key. Replace the old property with the corresponding type and call the GetValue() function to retrieve the value.
struct FValueOrBBKey_Bool;
struct FValueOrBBKey_Class;
struct FValueOrBBKey_Enum;
struct FValueOrBBKey_Float;
struct FValueOrBBKey_Int32;
struct FValueOrBBKey_Name;
struct FValueOrBBKey_String;
struct FValueOrBBKey_Object;
struct FValueOrBBKey_Rotator;
struct FValueOrBBKey_Struct;
struct FValueOrBBKey_Vector;

namespace FBlackboard
{
	template <typename T>
	TOptional<typename T::FDataType> TryGetBlackboardKeyValue(const UBlackboardComponent& Blackboard, const FName& Name, FBlackboard::FKey& InOutCachedKey)
	{
		if (InOutCachedKey == FBlackboard::InvalidKey)
		{
			InOutCachedKey = Blackboard.GetKeyID(Name);
		}
		if (Blackboard.GetKeyType(InOutCachedKey) == T::StaticClass())
		{
			return { Blackboard.GetValue<T>(InOutCachedKey) };
		}
		return {};
	}

	template <typename T>
	typename T::FDataType GetValue(const UBlackboardComponent& Blackboard, const FName& Name, FBlackboard::FKey& InOutCachedKey, const typename T::FDataType& DefaultValue)
	{
		if (!Name.IsNone())
		{
			if (const TOptional<typename T::FDataType> KeyValue = FBlackboard::TryGetBlackboardKeyValue<T>(Blackboard, Name, InOutCachedKey))
			{
				return KeyValue.GetValue();
			}
		}
		return DefaultValue;
	}

	template <typename T>
	typename T::FDataType GetValue(const UBehaviorTreeComponent& BehaviorComp, const FName& Name, FBlackboard::FKey& InOutCachedKey, const typename T::FDataType& DefaultValue)
	{
		if (const UBlackboardComponent* Blackboard = BehaviorComp.GetBlackboardComponent())
		{
			return GetValue<T>(*Blackboard, Name, InOutCachedKey, DefaultValue);
		}
		return DefaultValue;
	}

	FConstStructView TryGetBlackboardKeyStruct(const UBlackboardComponent& Blackboard, const FName& Name, FBlackboard::FKey& InOutCachedKey, const UScriptStruct* TargetStruct);

	FConstStructView GetStructValue(const UBlackboardComponent& Blackboard, const FName& Name, FBlackboard::FKey& InOutCachedKey, const FConstStructView& DefaultValue);

	FConstStructView GetStructValue(const UBehaviorTreeComponent& BehaviorComp, const FName& Name, FBlackboard::FKey& InOutCachedKey, const FConstStructView& DefaultValue);
} // namespace FBlackboard

// Base struct to simplify edition in the editor, shouldn't be used elsewhere
USTRUCT(BlueprintType)
struct FValueOrBlackboardKeyBase
{
	GENERATED_BODY()

	friend FValueOrBBKeyDetails;
	virtual ~FValueOrBlackboardKeyBase() = default;

#if WITH_EDITOR
	virtual bool IsCompatibleType(const UBlackboardKeyType* KeyType) const { return false; };
	AIMODULE_API void PreSave(const UObject* Outer, const UBlackboardData& Blackboard, FName PropertyName);
#endif // WITH_EDITOR

	virtual FString ToString() const { return FString(); }

	const FName& GetKey() const { return Key; }
	void SetKey(FName NewKey) { Key = NewKey; }

	AIMODULE_API FBlackboard::FKey GetKeyId(const UBehaviorTreeComponent& OwnerComp) const;

protected:
	template <typename T>
	FString ToStringInternal(const T& DefaultValue) const
	{
		if (!Key.IsNone())
		{
			return ToStringKeyName();
		}
		else
		{
			return FString::Format(TEXT("{0}"), { DefaultValue });
		}
	}

	AIMODULE_API FString ToStringKeyName() const;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Value")
	FName Key;

	mutable FBlackboard::FKey KeyId = FBlackboard::InvalidKey;
};

USTRUCT(BlueprintType)
struct FValueOrBBKey_Bool : public FValueOrBlackboardKeyBase
{
	GENERATED_BODY()

	FValueOrBBKey_Bool(bool Default = false)
		: DefaultValue(Default) {}
	AIMODULE_API bool GetValue(const UBehaviorTreeComponent& BehaviorComp) const;
	AIMODULE_API bool GetValue(const UBehaviorTreeComponent* BehaviorComp) const;
	AIMODULE_API bool GetValue(const UBlackboardComponent& Blackboard) const;
	AIMODULE_API bool GetValue(const UBlackboardComponent* Blackboard) const;

#if WITH_EDITOR
	AIMODULE_API virtual bool IsCompatibleType(const UBlackboardKeyType* KeyType) const override;
#endif // WITH_EDITOR

	bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	FString ToString() const override { return ToStringInternal(DefaultValue); }

	UE_DEPRECATED_FORGAME(5.5, "Implicit conversion will be removed next version. Call GetValue instead")
	operator bool() const { return DefaultValue; }

protected:
	UPROPERTY(EditAnywhere, Category = "Value")
	bool DefaultValue = false;
};

USTRUCT(BlueprintType)
struct FValueOrBBKey_Class : public FValueOrBlackboardKeyBase
{
	GENERATED_BODY()

	friend class FValueOrBBKeyDetails_Class;

	FValueOrBBKey_Class() = default;

	template <typename T>
	FValueOrBBKey_Class(TSubclassOf<T> Default)
		: DefaultValue(Default), BaseClass(T::StaticClass()) {}

	template <typename T>
	TSubclassOf<T> GetValue(const UBehaviorTreeComponent& BehaviorComp) const
	{
		return TSubclassOf<T>(GetValue(BehaviorComp));
	}

	template <typename T>
	TSubclassOf<T> GetValue(const UBehaviorTreeComponent* BehaviorComp) const
	{
		return TSubclassOf<T>(GetValue(BehaviorComp));
	}

	template <typename T>
	TSubclassOf<T> GetValue(const UBlackboardComponent& BlackboardComp) const
	{
		return TSubclassOf<T>(GetValue(BlackboardComp));
	}

	template <typename T>
	TSubclassOf<T> GetValue(const UBlackboardComponent* BlackboardComp) const
	{
		return TSubclassOf<T>(GetValue(BlackboardComp));
	}

	AIMODULE_API UClass* GetValue(const UBehaviorTreeComponent& BehaviorComp) const;
	AIMODULE_API UClass* GetValue(const UBehaviorTreeComponent* BehaviorComp) const;
	AIMODULE_API UClass* GetValue(const UBlackboardComponent& Blackboard) const;
	AIMODULE_API UClass* GetValue(const UBlackboardComponent* Blackboard) const;

	bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

#if WITH_EDITOR
	AIMODULE_API virtual bool IsCompatibleType(const UBlackboardKeyType* KeyType) const override;
#endif // WITH_EDITOR

	AIMODULE_API FString ToString() const override;

	UE_DEPRECATED_FORGAME(5.5, "Implicit conversion will be removed next version. Call GetValue instead")
	operator UClass*() const { return DefaultValue; }

	AIMODULE_API void SetBaseClass(UClass* NewBaseClass);

protected:
	UPROPERTY(EditAnywhere, Category = "Value")
	TObjectPtr<UClass> DefaultValue = nullptr;

	UPROPERTY(EditDefaultsOnly, Category = "Value", meta = (AllowAbstract = "1"))
	TObjectPtr<UClass> BaseClass = UObject::StaticClass();
};

USTRUCT(BlueprintType)
struct FValueOrBBKey_Enum : public FValueOrBlackboardKeyBase
{
	GENERATED_BODY()

	friend class FValueOrBBKeyDetails_Enum;

	FValueOrBBKey_Enum() = default;

	template <typename T>
	FValueOrBBKey_Enum(T Default)
		: DefaultValue(static_cast<uint8>(Default)), EnumType(StaticEnum<T>()), NativeEnumTypeName(GetNameSafe(EnumType)) {}

	template <typename T>
	T GetValue(const UBehaviorTreeComponent& BehaviorComp) const
	{
		return static_cast<T>(GetValue(BehaviorComp));
	}

	template <typename T>
	T GetValue(const UBehaviorTreeComponent* BehaviorComp) const
	{
		return static_cast<T>(GetValue(BehaviorComp));
	}

	template <typename T>
	T GetValue(const UBlackboardComponent& BlackboardComp) const
	{
		return static_cast<T>(GetValue(BlackboardComp));
	}

	template <typename T>
	T GetValue(const UBlackboardComponent* BlackboardComp) const
	{
		return static_cast<T>(GetValue(BlackboardComp));
	}

	AIMODULE_API uint8 GetValue(const UBehaviorTreeComponent& BehaviorComp) const;
	AIMODULE_API uint8 GetValue(const UBehaviorTreeComponent* BehaviorComp) const;
	AIMODULE_API uint8 GetValue(const UBlackboardComponent& Blackboard) const;
	AIMODULE_API uint8 GetValue(const UBlackboardComponent* Blackboard) const;

	bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

#if WITH_EDITOR
	AIMODULE_API virtual bool IsCompatibleType(const UBlackboardKeyType* KeyType) const override;
#endif // WITH_EDITOR

	AIMODULE_API FString ToString() const override;

	UE_DEPRECATED_FORGAME(5.5, "Implicit conversion will be removed next version. Call GetValue instead")
	operator uint8() const { return DefaultValue; }

	AIMODULE_API void SetEnumType(UEnum* NewEnumType);

protected:
	UPROPERTY(EditAnywhere, Category = "Value")
	uint8 DefaultValue = 0;

	UPROPERTY(EditDefaultsOnly, Category = "Value")
	TObjectPtr<UEnum> EnumType = nullptr;

	/** Name of enum type defined in C++ code, will take priority over asset from EnumType property */
	UPROPERTY(EditDefaultsOnly, Category = "Value")
	FString NativeEnumTypeName;
};

USTRUCT(BlueprintType)
struct FValueOrBBKey_Float : public FValueOrBlackboardKeyBase
{
	GENERATED_BODY()

	FValueOrBBKey_Float(float Default = 0.f)
		: DefaultValue(Default) {}
	AIMODULE_API float GetValue(const UBehaviorTreeComponent& BehaviorComp) const;
	AIMODULE_API float GetValue(const UBehaviorTreeComponent* BehaviorComp) const;
	AIMODULE_API float GetValue(const UBlackboardComponent& Blackboard) const;
	AIMODULE_API float GetValue(const UBlackboardComponent* Blackboard) const;

	bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

#if WITH_EDITOR
	AIMODULE_API virtual bool IsCompatibleType(const UBlackboardKeyType* KeyType) const override;
#endif // WITH_EDITOR

	AIMODULE_API FString ToString() const override;
	AIMODULE_API bool IsBoundOrNonZero() const;

	UE_DEPRECATED_FORGAME(5.5, "Implicit conversion will be removed next version. Call GetValue instead")
	operator float() const { return DefaultValue; }

protected:
	UPROPERTY(EditAnywhere, Category = "Value")
	float DefaultValue = 0.f;
};

USTRUCT(BlueprintType)
struct FValueOrBBKey_Int32 : public FValueOrBlackboardKeyBase
{
	GENERATED_BODY()
	FValueOrBBKey_Int32(int32 Default = 0)
		: DefaultValue(Default) {}
	AIMODULE_API int32 GetValue(const UBehaviorTreeComponent& BehaviorComp) const;
	AIMODULE_API int32 GetValue(const UBehaviorTreeComponent* BehaviorComp) const;
	AIMODULE_API int32 GetValue(const UBlackboardComponent& Blackboard) const;
	AIMODULE_API int32 GetValue(const UBlackboardComponent* Blackboard) const;

	bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

#if WITH_EDITOR
	AIMODULE_API virtual bool IsCompatibleType(const UBlackboardKeyType* KeyType) const override;
#endif // WITH_EDITOR

	FString ToString() const override { return ToStringInternal(DefaultValue); }

	UE_DEPRECATED_FORGAME(5.5, "Implicit conversion will be removed next version. Call GetValue instead")
	operator int32() const { return DefaultValue; }

protected:
	UPROPERTY(EditAnywhere, Category = "Value")
	int32 DefaultValue = 0;
};

USTRUCT(BlueprintType)
struct FValueOrBBKey_Name : public FValueOrBlackboardKeyBase
{
	GENERATED_BODY()

	FValueOrBBKey_Name(FName Default = FName())
		: DefaultValue(Default) {}
	AIMODULE_API FName GetValue(const UBehaviorTreeComponent& BehaviorComp) const;
	AIMODULE_API FName GetValue(const UBehaviorTreeComponent* BehaviorComp) const;
	AIMODULE_API FName GetValue(const UBlackboardComponent& Blackboard) const;
	AIMODULE_API FName GetValue(const UBlackboardComponent* Blackboard) const;

	bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

#if WITH_EDITOR
	AIMODULE_API virtual bool IsCompatibleType(const UBlackboardKeyType* KeyType) const override;
#endif // WITH_EDITOR

	AIMODULE_API FString ToString() const override;

	UE_DEPRECATED_FORGAME(5.5, "Implicit conversion will be removed next version. Call GetValue instead")
	operator FName() const { return DefaultValue; }

protected:
	UPROPERTY(EditAnywhere, Category = "Value")
	FName DefaultValue;
};

USTRUCT(BlueprintType)
struct FValueOrBBKey_String : public FValueOrBlackboardKeyBase
{
	GENERATED_BODY()

	FValueOrBBKey_String(FString Default = FString())
		: DefaultValue(MoveTemp(Default)) {}
	AIMODULE_API FString GetValue(const UBehaviorTreeComponent& BehaviorComp) const;
	AIMODULE_API FString GetValue(const UBehaviorTreeComponent* BehaviorComp) const;
	AIMODULE_API FString GetValue(const UBlackboardComponent& Blackboard) const;
	AIMODULE_API FString GetValue(const UBlackboardComponent* Blackboard) const;

	bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

#if WITH_EDITOR
	AIMODULE_API virtual bool IsCompatibleType(const UBlackboardKeyType* KeyType) const override;
#endif // WITH_EDITOR

	UE_DEPRECATED_FORGAME(5.5, "Implicit conversion will be removed next version. Call GetValue instead")
	operator FString() const { return DefaultValue; }

	FString ToString() const override { return ToStringInternal(DefaultValue); }

protected:
	UPROPERTY(EditAnywhere, Category = "Value")
	FString DefaultValue;
};

USTRUCT(BlueprintType)
struct FValueOrBBKey_Object : public FValueOrBlackboardKeyBase
{
	GENERATED_BODY()
	friend class FValueOrBBKeyDetails_Object;

	FValueOrBBKey_Object() = default;

	template <typename T>
	FValueOrBBKey_Object(TObjectPtr<T> Default)
		: DefaultValue(Default), BaseClass(T::StaticClass()) {}

	template <typename T>
	T* GetValue(const UBehaviorTreeComponent& BehaviorComp) const
	{
		return Cast<T>(GetValue(BehaviorComp));
	}

	template <typename T>
	T* GetValue(const UBehaviorTreeComponent* BehaviorComp) const
	{
		return Cast<T>(GetValue(BehaviorComp));
	}

	template <typename T>
	T* GetValue(const UBlackboardComponent& BlackboardComp) const
	{
		return Cast<T>(GetValue(BlackboardComp));
	}

	template <typename T>
	T* GetValue(const UBlackboardComponent* BlackboardComp) const
	{
		return Cast<T>(GetValue(BlackboardComp));
	}

	AIMODULE_API UObject* GetValue(const UBehaviorTreeComponent& BehaviorComp) const;
	AIMODULE_API UObject* GetValue(const UBehaviorTreeComponent* BehaviorComp) const;
	AIMODULE_API UObject* GetValue(const UBlackboardComponent& Blackboard) const;
	AIMODULE_API UObject* GetValue(const UBlackboardComponent* Blackboard) const;

	bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

#if WITH_EDITOR
	AIMODULE_API virtual bool IsCompatibleType(const UBlackboardKeyType* KeyType) const override;
#endif // WITH_EDITOR

	AIMODULE_API FString ToString() const override;

	UE_DEPRECATED_FORGAME(5.5, "Implicit conversion will be removed next version. Call GetValue instead")
	operator UObject*() const { return DefaultValue; }

	AIMODULE_API void SetBaseClass(UClass* NewBaseClass);

protected:
	UPROPERTY(EditAnywhere, Category = "Value")
	TObjectPtr<UObject> DefaultValue = nullptr;

	UPROPERTY(EditDefaultsOnly, Category = "Value", meta = (AllowAbstract = "1"))
	TObjectPtr<UClass> BaseClass = UObject::StaticClass();
};

USTRUCT(BlueprintType)
struct FValueOrBBKey_Rotator : public FValueOrBlackboardKeyBase
{
	GENERATED_BODY()

	FValueOrBBKey_Rotator(FRotator Default = FRotator::ZeroRotator)
		: DefaultValue(MoveTemp(Default)) {}
	AIMODULE_API FRotator GetValue(const UBehaviorTreeComponent& BehaviorComp) const;
	AIMODULE_API FRotator GetValue(const UBehaviorTreeComponent* BehaviorComp) const;
	AIMODULE_API FRotator GetValue(const UBlackboardComponent& Blackboard) const;
	AIMODULE_API FRotator GetValue(const UBlackboardComponent* Blackboard) const;

	bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

#if WITH_EDITOR
	AIMODULE_API virtual bool IsCompatibleType(const UBlackboardKeyType* KeyType) const override;
#endif // WITH_EDITOR

	AIMODULE_API FString ToString() const override;

	UE_DEPRECATED_FORGAME(5.5, "Implicit conversion will be removed next version. Call GetValue instead")
	operator FRotator() const { return DefaultValue; }

protected:
	UPROPERTY(EditAnywhere, Category = "Value")
	FRotator DefaultValue;
};

USTRUCT(BlueprintType)
struct FValueOrBBKey_Vector : public FValueOrBlackboardKeyBase
{
	GENERATED_BODY()

	FValueOrBBKey_Vector(FVector Default = FVector::ZeroVector)
		: DefaultValue(MoveTemp(Default)) {}
	AIMODULE_API FVector GetValue(const UBehaviorTreeComponent& BehaviorComp) const;
	AIMODULE_API FVector GetValue(const UBehaviorTreeComponent* BehaviorComp) const;
	AIMODULE_API FVector GetValue(const UBlackboardComponent& Blackboard) const;
	AIMODULE_API FVector GetValue(const UBlackboardComponent* Blackboard) const;

	bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

#if WITH_EDITOR
	AIMODULE_API virtual bool IsCompatibleType(const UBlackboardKeyType* KeyType) const override;
#endif // WITH_EDITOR

	AIMODULE_API FString ToString() const override;

	UE_DEPRECATED_FORGAME(5.5, "Implicit conversion will be removed next version. Call GetValue instead")
	operator FVector() const { return DefaultValue; }

protected:
	UPROPERTY(EditAnywhere, Category = "Value")
	FVector DefaultValue;
};

USTRUCT(BlueprintType)
struct FValueOrBBKey_Struct : public FValueOrBlackboardKeyBase
{
	GENERATED_BODY()

public:
	template <typename T>
	FValueOrBBKey_Struct(const T& Value)
	{
		DefaultValue.InitializeAs<T>(Value);
	}
	FValueOrBBKey_Struct() = default;

#if WITH_EDITOR
	AIMODULE_API virtual bool IsCompatibleType(const UBlackboardKeyType* KeyType) const override;
#endif // WITH_EDITOR

	AIMODULE_API FString ToString() const override;

	template <typename T>
	const T& GetValue(const UBehaviorTreeComponent& BehaviorComp) const
	{
		check(DefaultValue.GetScriptStruct() == TBaseStructure<T>::Get());
		return GetValue(BehaviorComp).Get<const T>();
	}

	template <typename T>
	const T& GetValue(const UBehaviorTreeComponent* BehaviorComp) const
	{
		check(DefaultValue.GetScriptStruct() == TBaseStructure<T>::Get());
		return GetValue(BehaviorComp).Get<const T>();
	}

	template <typename T>
	const T& GetValue(const UBlackboardComponent& BlackboardComp) const
	{
		check(DefaultValue.GetScriptStruct() == TBaseStructure<T>::Get());
		return GetValue(BlackboardComp).Get<const T>();
	}

	template <typename T>
	const T& GetValue(const UBlackboardComponent* BlackboardComp) const
	{
		check(DefaultValue.GetScriptStruct() == TBaseStructure<T>::Get());
		return GetValue(BlackboardComp).Get<const T>();
	}

	bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	AIMODULE_API FConstStructView GetValue(const UBehaviorTreeComponent& BehaviorComp) const;
	AIMODULE_API FConstStructView GetValue(const UBehaviorTreeComponent* BehaviorComp) const;
	AIMODULE_API FConstStructView GetValue(const UBlackboardComponent& Blackboard) const;
	AIMODULE_API FConstStructView GetValue(const UBlackboardComponent* Blackboard) const;

	AIMODULE_API void SetStructType(UScriptStruct* NewStructType);

protected:
	friend class FValueOrBBKeyDetails_Struct;

	UPROPERTY(EditAnywhere, Category = Value)
	FInstancedStruct DefaultValue;

#if WITH_EDITORONLY_DATA
	/** Exist to give a way for the details to know if the DefaultValue type can be edited. */
	UPROPERTY(EditDefaultsOnly, Category = Value)
	bool bCanEditDefaultValueType = false;
#endif // WITH_EDITORONLY_DATA
};

template <>
struct TStructOpsTypeTraits<FValueOrBBKey_Bool> : public TStructOpsTypeTraitsBase2<FValueOrBBKey_Bool>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

template <>
struct TStructOpsTypeTraits<FValueOrBBKey_Class> : public TStructOpsTypeTraitsBase2<FValueOrBBKey_Class>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

template <>
struct TStructOpsTypeTraits<FValueOrBBKey_Enum> : public TStructOpsTypeTraitsBase2<FValueOrBBKey_Enum>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

template <>
struct TStructOpsTypeTraits<FValueOrBBKey_Float> : public TStructOpsTypeTraitsBase2<FValueOrBBKey_Float>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

template <>
struct TStructOpsTypeTraits<FValueOrBBKey_Int32> : public TStructOpsTypeTraitsBase2<FValueOrBBKey_Int32>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

template <>
struct TStructOpsTypeTraits<FValueOrBBKey_Name> : public TStructOpsTypeTraitsBase2<FValueOrBBKey_Name>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

template <>
struct TStructOpsTypeTraits<FValueOrBBKey_String> : public TStructOpsTypeTraitsBase2<FValueOrBBKey_String>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

template <>
struct TStructOpsTypeTraits<FValueOrBBKey_Object> : public TStructOpsTypeTraitsBase2<FValueOrBBKey_Object>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

template <>
struct TStructOpsTypeTraits<FValueOrBBKey_Rotator> : public TStructOpsTypeTraitsBase2<FValueOrBBKey_Rotator>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

template <>
struct TStructOpsTypeTraits<FValueOrBBKey_Struct> : public TStructOpsTypeTraitsBase2<FValueOrBBKey_Struct>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

template <>
struct TStructOpsTypeTraits<FValueOrBBKey_Vector> : public TStructOpsTypeTraitsBase2<FValueOrBBKey_Vector>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};
