// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/EngineTypes.h"
#include "AITypes.h"
#include "Navigation/PathFollowingComponent.h"
#include "AIAsyncTaskBlueprintProxy.generated.h"

class AAIController;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOAISimpleDelegate, EPathFollowingResult::Type, MovementResult);

UCLASS(MinimalAPI)
class UAIAsyncTaskBlueprintProxy : public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FOAISimpleDelegate	OnSuccess;

	UPROPERTY(BlueprintAssignable)
	FOAISimpleDelegate	OnFail;

public:
	UFUNCTION()
	AIMODULE_API void OnMoveCompleted(FAIRequestID RequestID, EPathFollowingResult::Type MovementResult);

	AIMODULE_API void OnNoPath();
	AIMODULE_API void OnAtGoal();

	//~ Begin UObject Interface
	virtual void BeginDestroy() override;
	//~ End UObject Interface

	TWeakObjectPtr<AAIController> AIController;
	FAIRequestID MoveRequestId;
	TWeakObjectPtr<UWorld> MyWorld;

	FTimerHandle TimerHandle_OnInstantFinish;
};
