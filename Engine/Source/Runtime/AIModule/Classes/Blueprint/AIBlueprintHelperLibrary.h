// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * This kismet library is used for helper functions primarily used in the kismet compiler for AI related nodes
 * NOTE: Do not change the signatures for any of these functions as it can break the kismet compiler and/or the nodes referencing them
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "GameFramework/Pawn.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "AIBlueprintHelperLibrary.generated.h"

class AAIController;
class UAIAsyncTaskBlueprintProxy;
class UAnimInstance;
class UBehaviorTree;
class UBlackboardComponent;
class UNavigationPath;
class UPathFollowingComponent;

UCLASS(meta=(ScriptName="AIHelperLibrary"), MinimalAPI)
class UAIBlueprintHelperLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	UFUNCTION(BlueprintCallable, meta=(WorldContext="WorldContextObject", BlueprintInternalUseOnly = "TRUE"))
	static AIMODULE_API UAIAsyncTaskBlueprintProxy* CreateMoveToProxyObject(UObject* WorldContextObject, APawn* Pawn, FVector Destination, AActor* TargetActor = NULL, float AcceptanceRadius = 5.f, bool bStopOnOverlap = false);

	UFUNCTION(BlueprintCallable, Category="AI", meta=(DefaultToSelf="MessageSource"))
	static AIMODULE_API void SendAIMessage(APawn* Target, FName Message, UObject* MessageSource, bool bSuccess = true);

	/**	Spawns AI agent of a given class. The PawnClass needs to have AIController 
	 *	set for the function to spawn a controller as well.
	 *	@param BehaviorTree if set, and the function has successfully spawned 
	 *		and AI controller, this BehaviorTree asset will be assigned to the AI 
	 *		controller, and run.
	 *	@param Owner lets you spawn the AI in a sublevel rather than in the 
	 *		persistent level (which is the default behavior).
	 */
	UFUNCTION(BlueprintCallable, Category="AI", meta=(WorldContext="WorldContextObject", UnsafeDuringActorConstruction="true", AdvancedDisplay = "Owner"))
	static AIMODULE_API APawn* SpawnAIFromClass(UObject* WorldContextObject, TSubclassOf<APawn> PawnClass, UBehaviorTree* BehaviorTree, FVector Location, FRotator Rotation = FRotator::ZeroRotator, bool bNoCollisionFail = false, AActor* Owner = nullptr);

	/** The way it works exactly is if the actor passed in is a pawn, then the function retrieves 
	 *	pawn's controller cast to AIController. Otherwise the function returns actor cast to AIController. */
	UFUNCTION(BlueprintPure, Category = "AI", meta = (DefaultToSelf = "ControlledObject"))
	static AIMODULE_API AAIController* GetAIController(AActor* ControlledActor);

	UFUNCTION(BlueprintPure, Category="AI", meta=(DefaultToSelf="Target"))
	static AIMODULE_API UBlackboardComponent* GetBlackboard(AActor* Target);

	/** locks indicated AI resources of animated pawn */
	UFUNCTION(BlueprintCallable, Category = "Animation", BlueprintAuthorityOnly, meta = (DefaultToSelf = "AnimInstance"))
	static AIMODULE_API void LockAIResourcesWithAnimation(UAnimInstance* AnimInstance, bool bLockMovement, bool LockAILogic);

	/** unlocks indicated AI resources of animated pawn. Will unlock only animation-locked resources */
	UFUNCTION(BlueprintCallable, Category = "Animation", BlueprintAuthorityOnly, meta = (DefaultToSelf = "AnimInstance"))
	static AIMODULE_API void UnlockAIResourcesWithAnimation(UAnimInstance* AnimInstance, bool bUnlockMovement, bool UnlockAILogic);

	UFUNCTION(BlueprintPure, Category = "AI")
	static AIMODULE_API bool IsValidAILocation(FVector Location);

	UFUNCTION(BlueprintPure, Category = "AI")
	static AIMODULE_API bool IsValidAIDirection(FVector DirectionVector);

	UFUNCTION(BlueprintPure, Category = "AI")
	static AIMODULE_API bool IsValidAIRotation(FRotator Rotation);

	/** Returns a NEW UOBJECT that is a COPY of navigation path given controller is currently using. 
	 *	The result being a copy means you won't be able to influence agent's pathfollowing 
	 *	by manipulating received path.
	 *	Please use GetCurrentPathPoints if you only need the array of path points. */
	UFUNCTION(BlueprintPure, Category = "AI", meta = (UnsafeDuringActorConstruction = "true"))
	static AIMODULE_API UNavigationPath* GetCurrentPath(AController* Controller);

	/** Returns an array of navigation path points given controller is currently using. */
	UFUNCTION(BlueprintPure, Category = "AI", meta = (UnsafeDuringActorConstruction = "true"))
	static AIMODULE_API const TArray<FVector> GetCurrentPathPoints(AController* Controller);

	/** Return the path index the given controller is currently at. Returns INDEX_NONE if no path. */
	UFUNCTION(BlueprintPure, Category = "AI", meta = (UnsafeDuringActorConstruction = "true"))
	static AIMODULE_API int32 GetCurrentPathIndex(const AController* Controller);

	/** Return the path index of the next nav link for the current path of the given controller. Returns INDEX_NONE if no path or no incoming nav link. */
	UFUNCTION(BlueprintPure, Category = "AI", meta = (UnsafeDuringActorConstruction = "true"))
	static AIMODULE_API int32 GetNextNavLinkIndex(const AController* Controller);

	UFUNCTION(BlueprintCallable, Category = "AI|Navigation")
	static AIMODULE_API void SimpleMoveToActor(AController* Controller, const AActor* Goal);

	UFUNCTION(BlueprintCallable, Category = "AI|Navigation")
	static AIMODULE_API void SimpleMoveToLocation(AController* Controller, const FVector& Goal);

private:
	static UPathFollowingComponent* GetPathComp(const AController* Controller);
};
