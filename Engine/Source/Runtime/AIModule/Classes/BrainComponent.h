// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtr.h"
#include "Components/ActorComponent.h"
#include "EngineDefines.h"
#include "AITypes.h"
#include "AIResourceInterface.h"
#include "BrainComponent.generated.h"

class AAIController;
class AController;
class APawn;
class UBlackboardComponent;
class UBrainComponent;
struct FAIMessage;
struct FAIMessageObserver;

DECLARE_DELEGATE_TwoParams(FOnAIMessage, UBrainComponent*, const FAIMessage&);

DECLARE_LOG_CATEGORY_EXTERN(LogBrain, Warning, All);

struct FAIMessage
{
	enum EStatus
	{
		Failure,
		Success,
	};

	/** type of message */
	FName MessageName;

	/** message source */
	FWeakObjectPtr Sender;

	/** message param: ID */
	FAIRequestID RequestID;

	/** message param: status */
	TEnumAsByte<EStatus> Status;

	/** message param: custom flags */
	uint8 MessageFlags;

	FAIMessage() : MessageName(NAME_None), Sender(nullptr), RequestID(0), Status(FAIMessage::Success), MessageFlags(0) {}
	FAIMessage(FName InMessage, UObject* InSender) : MessageName(InMessage), Sender(InSender), RequestID(0), Status(FAIMessage::Success), MessageFlags(0) {}
	FAIMessage(FName InMessage, UObject* InSender, FAIRequestID InID, EStatus InStatus) : MessageName(InMessage), Sender(InSender), RequestID(InID), Status(InStatus), MessageFlags(0) {}
	FAIMessage(FName InMessage, UObject* InSender, FAIRequestID InID, bool bSuccess) : MessageName(InMessage), Sender(InSender), RequestID(InID), Status(bSuccess ? Success : Failure), MessageFlags(0) {}
	FAIMessage(FName InMessage, UObject* InSender, EStatus InStatus) : MessageName(InMessage), Sender(InSender), RequestID(0), Status(InStatus), MessageFlags(0) {}
	FAIMessage(FName InMessage, UObject* InSender, bool bSuccess) : MessageName(InMessage), Sender(InSender), RequestID(0), Status(bSuccess ? Success : Failure), MessageFlags(0) {}

	void SetFlags(uint8 Flags) { MessageFlags = Flags; }
	void SetFlag(uint8 Flag) { MessageFlags |= Flag; }
	void ClearFlag(uint8 Flag) { MessageFlags &= ~Flag; }
	bool HasFlag(uint8 Flag) const { return (MessageFlags & Flag) != 0; }

	static AIMODULE_API void Send(AController* Controller, const FAIMessage& Message);
	static AIMODULE_API void Send(APawn* Pawn, const FAIMessage& Message);
	static AIMODULE_API void Send(UBrainComponent* BrainComp, const FAIMessage& Message);

	static AIMODULE_API void Broadcast(UObject* WorldContextObject, const FAIMessage& Message);
};

typedef TSharedPtr<struct FAIMessageObserver> FAIMessageObserverHandle;

struct FAIMessageObserver : public TSharedFromThis<FAIMessageObserver>
{
public:
	AIMODULE_API FAIMessageObserver();

	static AIMODULE_API FAIMessageObserverHandle Create(AController* Controller, FName MessageType, FOnAIMessage const& Delegate);
	static AIMODULE_API FAIMessageObserverHandle Create(AController* Controller, FName MessageType, FAIRequestID MessageID, FOnAIMessage const& Delegate);

	static AIMODULE_API FAIMessageObserverHandle Create(APawn* Pawn, FName MessageType, FOnAIMessage const& Delegate);
	static AIMODULE_API FAIMessageObserverHandle Create(APawn* Pawn, FName MessageType, FAIRequestID MessageID, FOnAIMessage const& Delegate);

	static AIMODULE_API FAIMessageObserverHandle Create(UBrainComponent* BrainComp, FName MessageType, FOnAIMessage const& Delegate);
	static AIMODULE_API FAIMessageObserverHandle Create(UBrainComponent* BrainComp, FName MessageType, FAIRequestID MessageID, FOnAIMessage const& Delegate);

	AIMODULE_API ~FAIMessageObserver();

	AIMODULE_API void OnMessage(const FAIMessage& Message);
	AIMODULE_API FString DescribeObservedMessage() const;
	
	inline FName GetObservedMessageType() const { return MessageType; }
	inline FAIRequestID GetObservedMessageID() const { return MessageID; }
	inline bool IsObservingMessageID() const { return bFilterByID; }

private:

	AIMODULE_API void Register(UBrainComponent* OwnerComp);
	AIMODULE_API void Unregister();

	/** observed message type */
	FName MessageType;

	/** filter: message ID */
	FAIRequestID MessageID;
	bool bFilterByID;

	/** delegate to call */
	FOnAIMessage ObserverDelegate;

	/** brain component owning this observer */
	TWeakObjectPtr<UBrainComponent> Owner;

	// Non-copyable
	AIMODULE_API FAIMessageObserver(const FAIMessageObserver&);
	AIMODULE_API FAIMessageObserver& operator=(const FAIMessageObserver&);
};

UCLASS(ClassGroup = AI, BlueprintType, hidecategories = (Sockets, Collision), MinimalAPI)
class UBrainComponent : public UActorComponent, public IAIResourceInterface
{
	GENERATED_UCLASS_BODY()

protected:
	/** blackboard component */
	UPROPERTY(transient)
	TObjectPtr<UBlackboardComponent> BlackboardComp;

	UPROPERTY(transient)
	TObjectPtr<AAIController> AIOwner;

	// @TODO this is a temp contraption to implement delayed messages delivering
	// until proper AI messaging is implemented
	TArray<FAIMessage> MessagesToProcess;

public:
	virtual FString GetDebugInfoString() const { return TEXT(""); }

	/** To be called in case we want to restart AI logic while it's still being locked.
	 *	On subsequent ResumeLogic instead RestartLogic will be called. 
	 *	@note this call does nothing if logic is not locked at the moment of call */
	AIMODULE_API void RequestLogicRestartOnUnlock();

	/** Starts brain logic. If brain is already running, will not do anything. */
	UFUNCTION(BlueprintCallable, Category = "AI|Logic")
	AIMODULE_API virtual void StartLogic();

	/** Restarts currently running or previously ran brain logic. */
	UFUNCTION(BlueprintCallable, Category = "AI|Logic")
	AIMODULE_API virtual void RestartLogic();

	/** Stops currently running brain logic. */
	UFUNCTION(BlueprintCallable, Category = "AI|Logic")
	AIMODULE_API virtual void StopLogic(const FString& Reason);

	/** AI logic won't be needed anymore, stop all activity and run cleanup */
	virtual void Cleanup() {}

	/** Pause logic and blackboard updates. */
	virtual void PauseLogic(const FString& Reason) {}
	
	/** Resumes paused brain logic.
	 *  MUST be called by child implementations!
	 *	@return indicates whether child class' ResumeLogic should be called (true) or has it been 
	 *	handled in a different way and no other actions are required (false)*/
	AIMODULE_API virtual EAILogicResuming::Type ResumeLogic(const FString& Reason);
public:

	UFUNCTION(BlueprintPure, Category = "AI|Logic")
	AIMODULE_API virtual bool IsRunning() const;

	UFUNCTION(BlueprintPure, Category = "AI|Logic")
	AIMODULE_API virtual bool IsPaused() const;

#if ENABLE_VISUAL_LOG
	AIMODULE_API virtual void DescribeSelfToVisLog(struct FVisualLogEntry* Snapshot) const;
#endif // ENABLE_VISUAL_LOG
	
	// IAIResourceInterface begin
	AIMODULE_API virtual void LockResource(EAIRequestPriority::Type LockSource) override;
	AIMODULE_API virtual void ClearResourceLock(EAIRequestPriority::Type LockSource) override;
	AIMODULE_API virtual void ForceUnlockResource() override;
	AIMODULE_API virtual bool IsResourceLocked() const override;
	// IAIResourceInterface end
	
	AIMODULE_API virtual void HandleMessage(const FAIMessage& Message);
	
	/** BEGIN UActorComponent overrides */
	AIMODULE_API virtual void InitializeComponent() override;
	AIMODULE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	AIMODULE_API virtual void OnRegister() override;
	/** END UActorComponent overrides */

	/** caches BlackboardComponent's pointer to be used with this brain component */
	AIMODULE_API void CacheBlackboardComponent(UBlackboardComponent* BBComp);

	/** @return blackboard used with this component */
	UBlackboardComponent* GetBlackboardComponent();

	/** @return blackboard used with this component */
	const UBlackboardComponent* GetBlackboardComponent() const;

	inline AAIController* GetAIOwner() const { return AIOwner; }

protected:

	/** active message observers */
	TArray<FAIMessageObserver*> MessageObservers;

	friend struct FAIMessageObserver;
	friend struct FAIMessage;

	/** used to keep track of which subsystem requested this AI resource be locked */
	FAIResourceLock ResourceLock;

private:
	uint32 bDoLogicRestartOnUnlock : 1;

public:
	// static names to be used with SendMessage. Fell free to define game-specific
	// messages anywhere you want
	static AIMODULE_API const FName AIMessage_MoveFinished;
	static AIMODULE_API const FName AIMessage_RepathFailed;
	static AIMODULE_API const FName AIMessage_QueryFinished;
};

//////////////////////////////////////////////////////////////////////////
// Inlines

inline UBlackboardComponent* UBrainComponent::GetBlackboardComponent()
{
	return BlackboardComp;
}

inline const UBlackboardComponent* UBrainComponent::GetBlackboardComponent() const
{
	return BlackboardComp;
}
