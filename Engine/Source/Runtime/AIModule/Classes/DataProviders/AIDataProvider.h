// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "UObject/UnrealType.h"
#include "AIDataProvider.generated.h"

class UAIDataProvider;

/**
 * AIDataProvider is an object that can provide collection of properties
 * associated with bound pawn owner or request Id.
 *
 * Editable properties are used to set up provider instance,
 * creating additional filters or ways of accessing data (e.g. gameplay tag of ability)
 *
 * Non editable properties are holding data
 */

USTRUCT()
struct FAIDataProviderValue
{
	GENERATED_USTRUCT_BODY()

private:
	/** cached uproperty of provider */
	mutable FProperty* CachedProperty;

public:
	/** (optional) provider for dynamic data binding */
	UPROPERTY(EditAnywhere, Instanced, Category = Value)
	TObjectPtr<UAIDataProvider> DataBinding;

	/** name of provider's value property */
	UPROPERTY(EditAnywhere, Category = Value)
	FName DataField;

	/** describe default data */
	AIMODULE_API virtual FString ValueToString() const;
	AIMODULE_API FString ToString() const;

	/** filter for provider's properties */
	AIMODULE_API virtual bool IsMatchingType(FProperty* PropType) const;

	/** find all properties of provider that are matching filter */
	AIMODULE_API void GetMatchingProperties(TArray<FName>& MatchingProperties) const;

	/** return raw data from provider's property */
	template<typename T>
	T* GetRawValuePtr() const
	{
		return CachedProperty ? CachedProperty->ContainerPtrToValuePtr<T>(DataBinding) : nullptr;
	}

	/** bind data in provider and cache property for faster access */
	AIMODULE_API void BindData(const UObject* Owner, int32 RequestId) const;

	inline bool IsDynamic() const { return DataBinding != nullptr; }

	FAIDataProviderValue() :
		CachedProperty(nullptr),
		DataBinding(nullptr)
	{
	}

	virtual ~FAIDataProviderValue() = default;
};

USTRUCT()
struct FAIDataProviderTypedValue : public FAIDataProviderValue
{
	GENERATED_USTRUCT_BODY()

	FAIDataProviderTypedValue()
		: PropertyType_DEPRECATED(nullptr)
		, PropertyType(nullptr)
	{}

	/** type of value */
	UPROPERTY()
	TObjectPtr<UClass> PropertyType_DEPRECATED;
	FFieldClass* PropertyType;

	/** filter for provider's properties */
	AIMODULE_API virtual bool IsMatchingType(FProperty* PropType) const override;

	/** Implementing Serialize to convert UClass to FFieldClass */
	AIMODULE_API bool Serialize(FArchive& Ar);
};

template<>
struct TStructOpsTypeTraits<FAIDataProviderTypedValue> : public TStructOpsTypeTraitsBase2<FAIDataProviderTypedValue>
{
	enum
	{
		WithSerializer = true,
	};
};

USTRUCT()
struct FAIDataProviderStructValue : public FAIDataProviderValue
{
	GENERATED_USTRUCT_BODY()

	/** name of UStruct type */
	FString StructName;

	AIMODULE_API virtual bool IsMatchingType(FProperty* PropType) const override;
};

USTRUCT()
struct FAIDataProviderIntValue : public FAIDataProviderTypedValue
{
	GENERATED_USTRUCT_BODY()
	AIMODULE_API FAIDataProviderIntValue();

	UPROPERTY(EditAnywhere, Category = Value)
	int32 DefaultValue;

	AIMODULE_API int32 GetValue() const;
	AIMODULE_API virtual FString ValueToString() const override;
};

USTRUCT()
struct FAIDataProviderFloatValue : public FAIDataProviderTypedValue
{
	GENERATED_USTRUCT_BODY()
	AIMODULE_API FAIDataProviderFloatValue();

	UPROPERTY(EditAnywhere, Category = Value)
	float DefaultValue;

	AIMODULE_API float GetValue() const;
	AIMODULE_API virtual FString ValueToString() const override;
};

USTRUCT()
struct FAIDataProviderBoolValue : public FAIDataProviderTypedValue
{
	GENERATED_USTRUCT_BODY()
	AIMODULE_API FAIDataProviderBoolValue();

	UPROPERTY(EditAnywhere, Category = Value)
	bool DefaultValue;

	AIMODULE_API bool GetValue() const;
	AIMODULE_API virtual FString ValueToString() const override;
};

UCLASS(EditInlineNew, Abstract, CollapseCategories, AutoExpandCategories=(Provider), MinimalAPI)
class UAIDataProvider : public UObject
{
	GENERATED_UCLASS_BODY()

	AIMODULE_API virtual void BindData(const UObject& Owner, int32 RequestId);
	AIMODULE_API virtual FString ToString(FName PropName) const;
};
