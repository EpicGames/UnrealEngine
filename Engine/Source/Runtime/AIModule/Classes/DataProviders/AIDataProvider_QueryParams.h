// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "DataProviders/AIDataProvider.h"
#include "AIDataProvider_QueryParams.generated.h"

/**
 * AIDataProvider_QueryParams is used with environment queries
 *
 * It allows defining simple parameters for running query,
 * which are not tied to any specific pawn, but defined
 * for every query execution.
 */

UCLASS(EditInlineNew, meta=(DisplayName="Query Params"), MinimalAPI)
class UAIDataProvider_QueryParams : public UAIDataProvider
{
	GENERATED_BODY()

public:
	AIMODULE_API virtual void BindData(const UObject& Owner, int32 RequestId) override;
	AIMODULE_API virtual FString ToString(FName PropName) const override;

	/** Arbitrary name this query parameter will be exposed as to outside world (like BT nodes) */
	UPROPERTY(EditAnywhere, Category = Provider)
	FName ParamName;

	UPROPERTY()
	float FloatValue;

	UPROPERTY()
	int32 IntValue;

	UPROPERTY()
	bool BoolValue;
};
