// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataProviders/AIDataProvider_QueryParams.h"
#include "AIDataProvider_Random.generated.h"

UCLASS(EditInlineNew, meta = (DisplayName = "Random number"))
class UAIDataProvider_Random : public UAIDataProvider_QueryParams
{
	GENERATED_BODY()

protected:
	UPROPERTY(EditAnywhere, Category = AI)
	float Min;

	UPROPERTY(EditAnywhere, Category = AI)
	float Max;

	UPROPERTY(EditAnywhere, Category = AI)
	uint8 bInteger : 1;

public:
	UAIDataProvider_Random(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual void BindData(const UObject& Owner, int32 RequestId) override;
	virtual FString ToString(FName PropName) const override;
};
