// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "AIController.h"
#include "DetourCrowdAIController.generated.h"

#define UE_API AIMODULE_API

UCLASS(MinimalAPI)
class ADetourCrowdAIController : public AAIController
{
	GENERATED_BODY()
public:
	UE_API ADetourCrowdAIController(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
};

#undef UE_API
