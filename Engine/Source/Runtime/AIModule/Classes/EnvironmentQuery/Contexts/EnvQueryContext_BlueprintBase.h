// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EnvironmentQuery/EnvQueryContext.h"
#include "EnvQueryContext_BlueprintBase.generated.h"

class AActor;
struct FEnvQueryContextData;
struct FEnvQueryInstance;

UCLASS(MinimalAPI, Abstract, Blueprintable)
class UEnvQueryContext_BlueprintBase : public UEnvQueryContext
{
	GENERATED_UCLASS_BODY()

	enum ECallMode
	{
		InvalidCallMode,
		SingleActor,
		SingleLocation,
		ActorSet,
		LocationSet
	};

	ECallMode CallMode;

	// We need to implement GetWorld() so that blueprint functions which use a hidden WorldContextObject* will work properly.
	virtual UWorld* GetWorld() const override;

	virtual void ProvideContext(FEnvQueryInstance& QueryInstance, FEnvQueryContextData& ContextData) const override;

	UFUNCTION(BlueprintImplementableEvent)
	void ProvideSingleActor(UObject* QuerierObject, AActor* QuerierActor, AActor*& ResultingActor) const;

	UFUNCTION(BlueprintImplementableEvent)
	void ProvideSingleLocation(UObject* QuerierObject, AActor* QuerierActor, FVector& ResultingLocation) const;

	UFUNCTION(BlueprintImplementableEvent)
	void ProvideActorsSet(UObject* QuerierObject, AActor* QuerierActor, TArray<AActor*>& ResultingActorsSet) const;

	UFUNCTION(BlueprintImplementableEvent)
	void ProvideLocationsSet(UObject* QuerierObject, AActor* QuerierActor, TArray<FVector>& ResultingLocationSet) const;
};
