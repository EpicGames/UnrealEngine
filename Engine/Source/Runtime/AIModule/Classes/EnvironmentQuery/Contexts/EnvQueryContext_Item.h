// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EnvironmentQuery/EnvQueryContext.h"
#include "EnvQueryContext_Item.generated.h"

UCLASS(MinimalAPI)
class UEnvQueryContext_Item : public UEnvQueryContext
{
	GENERATED_UCLASS_BODY()
};
