// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AI/Navigation/NavigationTypes.h"
#include "EnvironmentQuery/EnvQueryContext.h"
#include "UObject/ObjectMacros.h"

#include "EnvQueryContext_NavigationData.generated.h"

struct FEnvQueryContextData;
struct FEnvQueryInstance;

UCLASS(MinimalAPI, Abstract, Blueprintable)
class UEnvQueryContext_NavigationData : public UEnvQueryContext
{
	GENERATED_UCLASS_BODY()

	/** Return the NavigationData setup in NavAgentProperties */
	AIMODULE_API virtual void ProvideContext(FEnvQueryInstance& QueryInstance, FEnvQueryContextData& ContextData) const override;

protected:

	/** NavAgentProperties used to find a NavigationData Override */
	UPROPERTY(EditDefaultsOnly, Category = "NavigationData")
	FNavAgentProperties NavAgentProperties;
};
