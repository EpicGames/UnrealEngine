// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EnvironmentQuery/EnvQueryContext.h"
#include "EnvQueryContext_Querier.generated.h"

struct FEnvQueryContextData;
struct FEnvQueryInstance;

UCLASS(MinimalAPI)
class UEnvQueryContext_Querier : public UEnvQueryContext
{
	GENERATED_UCLASS_BODY()
 
	virtual void ProvideContext(FEnvQueryInstance& QueryInstance, FEnvQueryContextData& ContextData) const override;
};
