// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "EQSQueryResultSourceInterface.generated.h"

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UEQSQueryResultSourceInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IEQSQueryResultSourceInterface
{
	GENERATED_IINTERFACE_BODY()

	virtual const struct FEnvQueryResult* GetQueryResult() const { return NULL; }
	virtual const struct FEnvQueryInstance* GetQueryInstance() const { return NULL; }

	// debugging
	virtual bool GetShouldDebugDrawLabels() const { return true; }
	virtual bool GetShouldDrawFailedItems() const { return true; }
	virtual float GetHighlightRangePct() const { return 1.0f; }
};
