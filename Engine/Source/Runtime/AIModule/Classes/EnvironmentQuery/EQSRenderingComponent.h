// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EngineDefines.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "PrimitiveViewRelevance.h"
#include "DebugRenderSceneProxy.h"
#include "EnvironmentQuery/EnvQueryDebugHelpers.h"
#include "Debug/DebugDrawComponent.h"
#include "EQSRenderingComponent.generated.h"

class APlayerController;
class IEQSQueryResultSourceInterface;
class UCanvas;

class FEQSSceneProxy final : public FDebugRenderSceneProxy
{
	friend class FEQSRenderingDebugDrawDelegateHelper;
public:
	AIMODULE_API virtual SIZE_T GetTypeHash() const override;

	AIMODULE_API explicit FEQSSceneProxy(const UPrimitiveComponent& InComponent, const FString& ViewFlagName = TEXT("DebugAI"), const TArray<FSphere>& Spheres = TArray<FSphere>(), const TArray<FText3d>& Texts = TArray<FText3d>());
	
	AIMODULE_API virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;

#if  USE_EQS_DEBUGGER 
	static AIMODULE_API void CollectEQSData(const UPrimitiveComponent* InComponent, const IEQSQueryResultSourceInterface* QueryDataSource, TArray<FSphere>& Spheres, TArray<FText3d>& Texts, TArray<EQSDebug::FDebugHelper>& DebugItems);
	static AIMODULE_API void CollectEQSData(const FEnvQueryResult* ResultItems, const FEnvQueryInstance* QueryInstance, float HighlightRangePct, bool ShouldDrawFailedItems, TArray<FSphere>& Spheres, TArray<FText3d>& Texts, TArray<EQSDebug::FDebugHelper>& DebugItems);
#endif
private:
	FEnvQueryResult QueryResult;	
	// can be 0
	AActor* ActorOwner;
	const IEQSQueryResultSourceInterface* QueryDataSource;
	uint32 bDrawOnlyWhenSelected : 1;

	static AIMODULE_API const FVector3f ItemDrawRadius;

	AIMODULE_API bool SafeIsActorSelected() const;
};

#if  USE_EQS_DEBUGGER
class FEQSRenderingDebugDrawDelegateHelper : public FDebugDrawDelegateHelper
{
	typedef FDebugDrawDelegateHelper Super;

public:
	FEQSRenderingDebugDrawDelegateHelper()
		: ActorOwner(nullptr)
		, QueryDataSource(nullptr)
		, bDrawOnlyWhenSelected(false)
	{
	}

	void SetupFromProxy(const FEQSSceneProxy* InSceneProxy)
	{
		ActorOwner = InSceneProxy->ActorOwner;
		QueryDataSource = InSceneProxy->QueryDataSource;
		bDrawOnlyWhenSelected = InSceneProxy->bDrawOnlyWhenSelected;
	}

	void Reset()
	{
		ResetTexts();
	}

protected:
	AIMODULE_API virtual void DrawDebugLabels(UCanvas* Canvas, APlayerController*) override;

private:
	// can be 0
	AActor* ActorOwner;
	const IEQSQueryResultSourceInterface* QueryDataSource;
	uint32 bDrawOnlyWhenSelected : 1;
};
#endif

UCLASS(ClassGroup = Debug, MinimalAPI)
class UEQSRenderingComponent : public UDebugDrawComponent
{
	GENERATED_UCLASS_BODY()

	FString DrawFlagName;
	uint32 bDrawOnlyWhenSelected : 1;

	AIMODULE_API void ClearStoredDebugData();
#if  USE_EQS_DEBUGGER || ENABLE_VISUAL_LOG
	AIMODULE_API void StoreDebugData(const EQSDebug::FQueryData& DebugData);
#endif

protected:
	AIMODULE_API virtual FBoxSphereBounds CalcBounds(const FTransform &LocalToWorld) const override;

#if UE_ENABLE_DEBUG_DRAWING && USE_EQS_DEBUGGER
	AIMODULE_API virtual FDebugRenderSceneProxy* CreateDebugSceneProxy() override;
	virtual FDebugDrawDelegateHelper& GetDebugDrawDelegateHelper() override { return EQSRenderingDebugDrawDelegateHelper; }
	FEQSRenderingDebugDrawDelegateHelper EQSRenderingDebugDrawDelegateHelper;
#endif

	//EQSDebug::FQueryData DebugData;
	TArray<FDebugRenderSceneProxy::FSphere> DebugDataSolidSpheres;
	TArray<FDebugRenderSceneProxy::FText3d> DebugDataTexts;
};
