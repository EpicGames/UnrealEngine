// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "EnvironmentQuery/EQSQueryResultSourceInterface.h"
#include "GameFramework/Character.h"
#include "AI/Navigation/NavigationTypes.h"
#include "EQSTestingPawn.generated.h"

class UEnvQuery;
class UEQSRenderingComponent;

UENUM()
enum class EEnvQueryHightlightMode : uint8
{
	All,
	Best5Pct UMETA(DisplayName = "Best 5%"),
	Best25Pct UMETA(DisplayName = "Best 25%"),
};

UCLASS(hidedropdown, hidecategories=(Advanced, Attachment, Mesh, Animation, Clothing, Physics, Rendering, Lighting, Activation, CharacterMovement, AgentPhysics, Avoidance, MovementComponent, Velocity, Shape, Camera, Input, Layers, SkeletalMesh, Optimization, Pawn, Replication, Actor), MinimalAPI)
class AEQSTestingPawn : public ACharacter, public IEQSQueryResultSourceInterface
{
	GENERATED_UCLASS_BODY()
	
	UPROPERTY(Category=EQS, EditAnywhere)
	TObjectPtr<UEnvQuery> QueryTemplate;

	/** optional parameters for query */
	UE_DEPRECATED_FORGAME(5.0, "QueryParams has been deprecated for a long while now. Will be removed in the next engine version.")
	UPROPERTY()
	TArray<FEnvNamedValue> QueryParams;

	UPROPERTY(Category=EQS, EditAnywhere)
	TArray<FAIDynamicParam> QueryConfig;

	UPROPERTY(Category=EQS, EditAnywhere)
	float TimeLimitPerStep;

	UPROPERTY(Category=EQS, EditAnywhere)
	int32 StepToDebugDraw;

	UPROPERTY(Category = EQS, EditAnywhere)
	EEnvQueryHightlightMode HighlightMode;

	UPROPERTY(Category = EQS, EditAnywhere)
	uint32 bDrawLabels:1;

	UPROPERTY(Category=EQS, EditAnywhere)
	uint32 bDrawFailedItems:1;

	UPROPERTY(Category=EQS, EditAnywhere)
	uint32 bReRunQueryOnlyOnFinishedMove:1;

	UPROPERTY(Category=EQS, EditAnywhere)
	uint32 bShouldBeVisibleInGame:1;

	UPROPERTY(Category = EQS, EditAnywhere)
	uint32 bTickDuringGame : 1;

	UPROPERTY(Category = EQS, EditAnywhere)
	uint32 bRunQueryOnSelectionChanged : 1;

	UPROPERTY(Category=EQS, EditAnywhere)
	TEnumAsByte<EEnvQueryRunMode::Type> QueryingMode;

	UPROPERTY(Category = EQS, EditAnywhere)
	FNavAgentProperties NavAgentProperties;

#if WITH_EDITORONLY_DATA
private:
	/** Editor Preview */
	UPROPERTY(Transient)
	TObjectPtr<UEQSRenderingComponent> EdRenderComp;
#endif // WITH_EDITORONLY_DATA

protected:
	TSharedPtr<FEnvQueryInstance> QueryInstance;

	UPROPERTY(Transient)
	TArray<FEnvQueryInstance> StepResults;

public:
	/** This pawn class spawns its controller in PostInitProperties to have it available in editor mode*/
	AIMODULE_API virtual void TickActor( float DeltaTime, enum ELevelTick TickType, FActorTickFunction& ThisTickFunction ) override;
	AIMODULE_API virtual void PostLoad() override;

#if WITH_EDITOR
	AIMODULE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	AIMODULE_API virtual void PostEditMove(bool bFinished) override;
	AIMODULE_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
	AIMODULE_API void OnPropertyChanged(const FName PropName);
#endif // WITH_EDITOR

	//~ Begin UObject Interface
	static AIMODULE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	//~ End UObject Interface

	// IEQSQueryResultSourceInterface start
	AIMODULE_API virtual const FEnvQueryResult* GetQueryResult() const override;
	AIMODULE_API virtual const FEnvQueryInstance* GetQueryInstance() const  override;

	virtual bool GetShouldDebugDrawLabels() const override { return bDrawLabels; }
	virtual bool GetShouldDrawFailedItems() const override{ return bDrawFailedItems; }
	AIMODULE_API virtual float GetHighlightRangePct() const override;
	// IEQSQueryResultSourceInterface end

	// INavAgentInterface begin
	AIMODULE_API virtual const FNavAgentProperties& GetNavAgentPropertiesRef() const override;
	// INavAgentInterface end

	AIMODULE_API void RunEQSQuery();

protected:	
	AIMODULE_API void Reset() override;
	AIMODULE_API void MakeOneStep();

	AIMODULE_API void UpdateDrawing();

#if WITH_EDITOR
	static AIMODULE_API void OnEditorSelectionChanged(UObject* NewSelection);
#endif // WITH_EDITOR

public:
#if WITH_EDITORONLY_DATA
	/** Returns EdRenderComp subobject **/
	UEQSRenderingComponent* GetEdRenderComp() { return EdRenderComp; }
#endif
};
