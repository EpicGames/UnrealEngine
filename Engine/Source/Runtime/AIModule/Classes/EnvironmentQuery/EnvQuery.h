// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "Engine/DataAsset.h"
#include "EnvQuery.generated.h"

class UEdGraph;
class UEnvQueryOption;

#if WITH_EDITORONLY_DATA
class UEdGraph;
#endif // WITH_EDITORONLY_DATA

UCLASS(BlueprintType, MinimalAPI)
class UEnvQuery : public UDataAsset
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	/** Graph for query */
	UPROPERTY()
	TObjectPtr<UEdGraph>	EdGraph;
#endif

protected:
	friend class UEnvQueryManager;

	UPROPERTY()
	FName QueryName;

	UPROPERTY()
	TArray<TObjectPtr<UEnvQueryOption>> Options;

public:
	/** Set to true to not cook this query into client builds. Useful for server-only AI queries. */
	UPROPERTY(EditAnywhere, Category=Query)
	uint8 bStripFromClientBuilds : 1 = false;

	/** Gather all required named params */
	AIMODULE_API void CollectQueryParams(UObject& QueryOwner, TArray<FAIDynamicParam>& NamedValues) const;

	AIMODULE_API virtual  void PostInitProperties() override;

	/** QueryName patching up */
	AIMODULE_API virtual void PostLoad() override;
#if WITH_EDITOR
	AIMODULE_API virtual void PostRename(UObject* OldOuter, const FName OldName) override;
	AIMODULE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
#endif // WITH_EDITOR

	virtual bool NeedsLoadForClient() const override
	{
		return !bStripFromClientBuilds && Super::NeedsLoadForClient();
	}
	virtual bool NeedsLoadForEditorGame() const override
	{
		return !bStripFromClientBuilds && Super::NeedsLoadForEditorGame();
	}

	FName GetQueryName() const { return QueryName; }

	TArray<TObjectPtr<UEnvQueryOption>>& GetOptionsMutable() { return Options; }
	const TArray<UEnvQueryOption*>& GetOptions() const { return Options; }
};
