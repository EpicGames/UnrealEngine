// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "EnvQueryContext.generated.h"

struct FEnvQueryContextData;
struct FEnvQueryInstance;

UCLASS(Abstract, EditInlineNew, MinimalAPI)
class UEnvQueryContext : public UObject
{
	GENERATED_UCLASS_BODY()

	AIMODULE_API virtual void ProvideContext(FEnvQueryInstance& QueryInstance, FEnvQueryContextData& ContextData) const;
};
