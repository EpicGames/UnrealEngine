// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "EngineDefines.h"
#include "Engine/World.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "VisualLogger/VisualLogger.h"
#include "DebugRenderSceneProxy.h"
#include "EnvQueryDebugHelpers.generated.h"

class FTestData;
class UEnvQueryDebugHelpers;

#if  USE_EQS_DEBUGGER || ENABLE_VISUAL_LOG

struct FLogCategoryBase;

namespace EQSDebug
{
	struct FItemData
	{
		FString Desc;
		int32 ItemIdx;
		float TotalScore;

		TArray<float> TestValues;
		TArray<float> TestScores;
	};

	struct FTestData
	{
		FString ShortName;
		FString Detailed;
	};

	// struct filled while collecting data (to store additional debug data needed to display per rendered item)
	struct FDebugHelper
	{
		FDebugHelper() : Location(FVector::ZeroVector), Radius(0), FailedTestIndex(INDEX_NONE){}
		FDebugHelper(FVector Loc, float R) : Location(Loc), Radius(R), FailedTestIndex(INDEX_NONE) {}
		FDebugHelper(FVector Loc, float R, const FString& Desc) : Location(Loc), Radius(R), FailedTestIndex(INDEX_NONE), AdditionalInformation(Desc) {}

		FVector Location;
		float Radius;
		int32 FailedTestIndex;
		float FailedScore;
		FString AdditionalInformation;
	};

	struct FQueryData
	{
		TArray<FItemData> Items;
		TArray<FTestData> Tests;
		TArray<FDebugRenderSceneProxy::FSphere> SolidSpheres;
		TArray<FDebugRenderSceneProxy::FText3d> Texts;
		TArray<FDebugHelper> RenderDebugHelpers;
		TArray<FString>	Options;
		int32 UsedOption;
		int32 NumValidItems;
		int32 Id;
		FString Name;
		double Timestamp;

		void Reset()
		{
			UsedOption = 0;
			Options.Reset();
			NumValidItems = 0;
			Id = INDEX_NONE;
			Name.Empty();
			Items.Reset();
			Tests.Reset();
			SolidSpheres.Reset();
			Texts.Reset();
			Timestamp = 0;
			RenderDebugHelpers.Reset();
		}
	};
}

inline
FArchive& operator<<(FArchive& Ar, FDebugRenderSceneProxy::FSphere& Data)
{
	Ar << Data.Radius;
	Ar << Data.Location;
	Ar << Data.Color;
	return Ar;
}

inline
FArchive& operator<<(FArchive& Ar, FDebugRenderSceneProxy::FText3d& Data)
{
	Ar << Data.Text;
	Ar << Data.Location;
	Ar << Data.Color;
	return Ar;
}

inline
FArchive& operator<<(FArchive& Ar, EQSDebug::FItemData& Data)
{
	Ar << Data.Desc;
	Ar << Data.ItemIdx;
	Ar << Data.TotalScore;
	Ar << Data.TestValues;
	Ar << Data.TestScores;
	return Ar;
}

inline
FArchive& operator<<(FArchive& Ar, EQSDebug::FTestData& Data)
{
	Ar << Data.ShortName;
	Ar << Data.Detailed;
	return Ar;
}

inline
FArchive& operator<<(FArchive& Ar, EQSDebug::FDebugHelper& Data)
{
	Ar << Data.Location;
	Ar << Data.Radius;
	Ar << Data.AdditionalInformation;
	Ar << Data.FailedTestIndex;
	return Ar;
}

inline
FArchive& operator<<(FArchive& Ar, EQSDebug::FQueryData& Data)
{
	Ar << Data.Items;
	Ar << Data.Tests;
	Ar << Data.SolidSpheres;
	Ar << Data.Texts;
	Ar << Data.NumValidItems;
	Ar << Data.Id;
	Ar << Data.Name;
	Ar << Data.Timestamp;
	Ar << Data.RenderDebugHelpers;
	Ar << Data.Options;
	Ar << Data.UsedOption;
	return Ar;
}

#endif //USE_EQS_DEBUGGER || ENABLE_VISUAL_LOG

#if ENABLE_VISUAL_LOG && USE_EQS_DEBUGGER
#	define UE_VLOG_EQS(Query, Category, Verbosity)  UEnvQueryDebugHelpers::LogQuery(Query, Category, ELogVerbosity::Verbosity);
#else
#	define UE_VLOG_EQS(Query, CategoryName, Verbosity)
#endif //ENABLE_VISUAL_LOG && USE_EQS_DEBUGGER

UCLASS(Abstract, MinimalAPI)
class UEnvQueryDebugHelpers : public UObject
{
	GENERATED_BODY()
public:
#if USE_EQS_DEBUGGER
	static AIMODULE_API void QueryToDebugData(FEnvQueryInstance& Query, EQSDebug::FQueryData& EQSLocalData, int32 MaxItemsToStore = 10);
	static AIMODULE_API void QueryToBlobArray(FEnvQueryInstance& Query, TArray<uint8>& BlobArray, bool bUseCompression = false);
	static AIMODULE_API void DebugDataToBlobArray(EQSDebug::FQueryData& QueryData, TArray<uint8>& BlobArray, bool bUseCompression = false);
	static AIMODULE_API void BlobArrayToDebugData(const TArray<uint8>& BlobArray, EQSDebug::FQueryData& EQSLocalData, bool bUseCompression = false);
#endif

#if ENABLE_VISUAL_LOG && USE_EQS_DEBUGGER
	static void LogQuery(FEnvQueryInstance& Query, const FLogCategoryBase& Category, ELogVerbosity::Type Verbosity);
	static void LogQuery(FEnvQueryInstance& Query, const FName& CategoryName, ELogVerbosity::Type Verbosity);

private:
	static void LogQueryInternal(FEnvQueryInstance& Query, const FName& CategoryName, ELogVerbosity::Type Verbosity, double TimeSeconds, FVisualLogEntry *CurrentEntry);
#endif
};

#if ENABLE_VISUAL_LOG && USE_EQS_DEBUGGER
inline void UEnvQueryDebugHelpers::LogQuery(FEnvQueryInstance& Query, const FLogCategoryBase& Category, ELogVerbosity::Type Verbosity)
{
	const FName CategoryName = Category.GetCategoryName(); 
	LogQuery(Query, CategoryName, Verbosity);
}

inline void UEnvQueryDebugHelpers::LogQuery(FEnvQueryInstance& Query, const FName& CategoryName, ELogVerbosity::Type Verbosity)
{
	UWorld *World = nullptr;
	FVisualLogEntry *CurrentEntry = nullptr;
	if (FVisualLogger::CheckVisualLogInputInternal(Query.Owner.Get(), CategoryName, Verbosity, &World, &CurrentEntry) == false)
	{
		return;
	}

	LogQueryInternal(Query, CategoryName, Verbosity, World->TimeSeconds, CurrentEntry);
}
#endif

