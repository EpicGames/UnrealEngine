// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "EnvironmentQuery/EnvQueryNode.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "EnvironmentQuery/Items/EnvQueryItemType.h"
#include "EnvQueryGenerator.generated.h"

namespace EnvQueryGeneratorVersion
{
	inline const int32 Initial = 0;
	inline const int32 DataProviders = 1;

	inline const int32 Latest = DataProviders;
}

UCLASS(EditInlineNew, Abstract, meta = (Category = "Generators"), MinimalAPI)
class UEnvQueryGenerator : public UEnvQueryNode
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditDefaultsOnly, Category=Option)
	FString OptionName;

	/** type of generated items */
	UPROPERTY()
	TSubclassOf<UEnvQueryItemType> ItemType;

	/** if set, tests will be automatically sorted for best performance before running query */
	UPROPERTY(EditDefaultsOnly, Category = Option, AdvancedDisplay)
	uint32 bAutoSortTests : 1;

	virtual void GenerateItems(FEnvQueryInstance& QueryInstance) const { checkNoEntry(); }
	virtual bool IsValidGenerator() const { return ItemType != nullptr; }
	bool CanRunAsync() const { return bCanRunAsync; }
	EEnvQueryResultNormalizationOption GetNormalizationOption() const;

	AIMODULE_API virtual void PostLoad() override;
	AIMODULE_API void UpdateNodeVersion() override;

protected:
	/** To be overwritten by MassEnvQueryGenerators to indicate that they will run asynchronously. */
	UPROPERTY(EditDefaultsOnly, Category = Option, AdvancedDisplay)
	uint32 bCanRunAsync : 1;

	/** Should the scores be normalized or not before returning the result. 
	 *  If set to UEnvQueryResultNormalizationOption::Default, only runmode EEnvQueryRunMode::AllMatching is normalized and the other modes are not
	 *  If set to UEnvQueryResultNormalizationOption::Normalized, all run modes will be normalized.
	 *  If set to UEnvQueryResultNormalizationOption::Unaltered, all run modes will NOT be normalized.
	 */
	UPROPERTY(EditDefaultsOnly, Category = Option, AdvancedDisplay)
	EEnvQueryResultNormalizationOption EnvQueryResultNormalizationOption = EEnvQueryResultNormalizationOption::Default;
};
