// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "UObject/ScriptMacros.h"
#include "EnvironmentQuery/Items/EnvQueryItemType.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "EnvironmentQuery/EQSQueryResultSourceInterface.h"
#include "EnvQueryInstanceBlueprintWrapper.generated.h"

class AActor;
struct FEnvQueryRequest;

UCLASS(Blueprintable, BlueprintType, meta = (DisplayName = "EQS Query Instance"), MinimalAPI)
class UEnvQueryInstanceBlueprintWrapper : public UObject, public IEQSQueryResultSourceInterface
{
	GENERATED_BODY()

public:
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FEQSQueryDoneSignature, UEnvQueryInstanceBlueprintWrapper*, QueryInstance, EEnvQueryStatus::Type, QueryStatus);

	/** IEQSQueryResultSourceInterface function. Returns the Query Result. */
	virtual const FEnvQueryResult* GetQueryResult() const { return QueryResult.Get(); }

	/** IEQSQueryResultSourceInterface function. Returns the Query Instance. */
	virtual const FEnvQueryInstance* GetQueryInstance() const { return QueryInstance.Get(); }

protected:
	UPROPERTY(BlueprintReadOnly, Category = "EQS")
	int32 QueryID;

	EEnvQueryRunMode::Type RunMode;

	TSharedPtr<FEnvQueryResult> QueryResult;
	TSharedPtr<FEnvQueryInstance> QueryInstance;

	UPROPERTY(BlueprintReadOnly, Category = "EQS")
	TSubclassOf<UEnvQueryItemType> ItemType;

	/** index of query option, that generated items */
	UPROPERTY(BlueprintReadOnly, Category = "EQS")
	int32 OptionIndex;

	UPROPERTY(BlueprintAssignable, Category = "AI|EQS", meta = (DisplayName = "OnQueryFinished"))
	FEQSQueryDoneSignature OnQueryFinishedEvent;

#if !UE_BUILD_SHIPPING
	FWeakObjectPtr Instigator;
#endif // !UE_BUILD_SHIPPING

public:
	AIMODULE_API UEnvQueryInstanceBlueprintWrapper(const FObjectInitializer& ObjectInitializer);
	
	AIMODULE_API void RunQuery(const EEnvQueryRunMode::Type InRunMode, FEnvQueryRequest& QueryRequest);

	UFUNCTION(BlueprintPure, Category = "AI|EQS")
	AIMODULE_API float GetItemScore(int32 ItemIndex) const;

	/** Outputs an array filled with resulting actors.  Note that it makes sense only if ItemType is a EnvQueryItemType_ActorBase-derived type.  Returns false if there is no valid result. */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "AI|EQS")
	AIMODULE_API bool GetQueryResultsAsActors(TArray<AActor*>& ResultActors) const;

	/** Outputs an array of locations generated by the query. If the query generated Actors the the array is filled with their locations.  Returns false if there is no valid result. */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "AI|EQS")
	AIMODULE_API bool GetQueryResultsAsLocations(TArray<FVector>& ResultLocations) const;

	/** DEPRECATED!  Use GetQueryResultsAsActors instead.  Returns an array filled with resulting actors. Note that it makes sense only if ItemType is a EnvQueryItemType_ActorBase-derived type. */
	UFUNCTION(BlueprintPure, Category = "AI|EQS", Meta=(DeprecatedFunction,
		DeprecationMessage="Use GetQueryResultsAsActors instead, which is more efficient itself and protects against very bad perf issues in some blueprints."))
	AIMODULE_API TArray<AActor*> GetResultsAsActors() const;

	/** DEPRECATED!  Use GetQueryResultsAsLocations instead.  Returns an array of locations generated by the query. If the query generated Actors the the array is filled with their locations. */
	UFUNCTION(BlueprintPure, Category = "AI|EQS", Meta=(DeprecatedFunction,
		DeprecationMessage="GetQueryResultsAsLocations instead, which is more efficient itself and protects against very bad perf issues in some blueprints."))
	AIMODULE_API TArray<FVector> GetResultsAsLocations() const;

	UFUNCTION(BlueprintCallable, Category = "AI|EQS")
	AIMODULE_API void SetNamedParam(FName ParamName, float Value);

	AIMODULE_API void SetInstigator(const UObject* Object);

	/** Used in UMassEQSBlueprintLibrary::GetEnviromentQueryResultAsEntityInfo */
	inline EEnvQueryRunMode::Type GetRunMode() const { return RunMode; };

	FEQSQueryDoneSignature& GetOnQueryFinishedEvent() { return OnQueryFinishedEvent; }

protected:
	AIMODULE_API void OnQueryFinished(TSharedPtr<FEnvQueryResult> Result);

	AIMODULE_API virtual bool IsSupportedForNetworking() const override;
};
