// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "EnvironmentQuery/Items/EnvQueryItemType.h"
#include "EnvironmentQuery/EnvQueryContext.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "AISubsystem.h"
#include "EnvironmentQuery/EnvQueryInstanceBlueprintWrapper.h"
#include "EnvQueryManager.generated.h"

class UEnvQuery;
class UEnvQueryManager;
class UEnvQueryOption;
class UEnvQueryTest;

//----------------------------------------------------------------------//
// FEnvQueryManagerConfig
//----------------------------------------------------------------------//

/** Wrapper to hold config variables */
USTRUCT()
struct FEnvQueryManagerConfig
{
	GENERATED_BODY()

public:

	/** how long are we allowed to test per update, in seconds. */
	UPROPERTY(config)
	float MaxAllowedTestingTime = 0.003f;

	/** whether we update EQS queries based on:
	running a test on one query and move to the next (breadth) - default behavior,
	or test an entire query before moving to the next one (depth). */
	UPROPERTY(config)
	bool bTestQueriesUsingBreadth = false;

	/** if greater than zero, we will warn once when the number of queries is greater than or equal to this number, and log the queries out */
	UPROPERTY(config)
	int32 QueryCountWarningThreshold = 200;

	/** how often (in seconds) we will warn about the number of queries (allows us to catch multiple occurrences in a session) */
	UPROPERTY(config)
	double QueryCountWarningInterval = 60.0f;

	/** Maximum EQS execution duration (in seconds) before a warning is reported. */
	UPROPERTY(config)
	double ExecutionTimeWarningSeconds = 0.025f;

	/** Maximum EQS Query FinishDelegate duration (in seconds) before a warning is reported. */
	UPROPERTY(config)
	double HandlingResultTimeWarningSeconds = 0.025f;

	/** Maximum EQS Generator duration (in seconds) before a warning is reported. */
	UPROPERTY(config)
	double GenerationTimeWarningSeconds = 0.01f;

public:
	FString ToString() const;
};

/** wrapper for easy query execution */
USTRUCT()
struct FEnvQueryRequest
{
	GENERATED_USTRUCT_BODY()

	FEnvQueryRequest() : QueryTemplate(NULL), Owner(NULL), World(NULL) {}

	// basic constructor: owner will be taken from finish delegate bindings
	FEnvQueryRequest(const UEnvQuery* Query) : QueryTemplate(Query), Owner(NULL), World(NULL) {}

	// use when owner is different from finish delegate binding
	FEnvQueryRequest(const UEnvQuery* Query, UObject* RequestOwner) : QueryTemplate(Query), Owner(RequestOwner), World(NULL) {}

	// set names param indicated by Param. If Param is configured to read values from a blackboard then BlackboardComponent
	// is expected to be non-null (the function will fail a check otherwise).
	AIMODULE_API FEnvQueryRequest& SetDynamicParam(const FAIDynamicParam& Param, const UBlackboardComponent* BlackboardComponent = nullptr);

	// set named params
	inline FEnvQueryRequest& SetFloatParam(FName ParamName, float Value) { NamedParams.Add(ParamName, Value); return *this; }
	inline FEnvQueryRequest& SetIntParam(FName ParamName, int32 Value) { NamedParams.Add(ParamName, *((float*)&Value)); return *this; }
	inline FEnvQueryRequest& SetBoolParam(FName ParamName, bool Value) { NamedParams.Add(ParamName, Value ? 1.0f : -1.0f); return *this; }
	inline FEnvQueryRequest& SetNamedParam(const FEnvNamedValue& ParamData) { NamedParams.Add(ParamData.ParamName, ParamData.Value); return *this; }
	AIMODULE_API FEnvQueryRequest& SetNamedParams(const TArray<FEnvNamedValue>& Params);

	// set world (for accessing query manager) when owner can't provide it
	inline FEnvQueryRequest& SetWorldOverride(UWorld* InWorld) { World = InWorld; return *this; }

	template< class UserClass >	
	inline int32 Execute(EEnvQueryRunMode::Type Mode, UserClass* InObj, typename FQueryFinishedSignature::TMethodPtr< UserClass > InMethod)
	{
		return Execute(Mode, FQueryFinishedSignature::CreateUObject(InObj, InMethod));
	}
	template< class UserClass >	
	inline int32 Execute(EEnvQueryRunMode::Type Mode, UserClass* InObj, typename FQueryFinishedSignature::TConstMethodPtr< UserClass > InMethod)
	{
		return Execute(Mode, FQueryFinishedSignature::CreateUObject(InObj, InMethod));
	}
	AIMODULE_API int32 Execute(EEnvQueryRunMode::Type RunMode, FQueryFinishedSignature const& FinishDelegate);

protected:

	/** query to run */
	UPROPERTY()
	TObjectPtr<const UEnvQuery> QueryTemplate;

	/** querier */
	UPROPERTY()
	TObjectPtr<UObject> Owner;

	/** world */
	UPROPERTY()
	TObjectPtr<UWorld> World;

	/** list of named params */
	TMap<FName, float> NamedParams;

	friend UEnvQueryManager;
};

/** cache of instances with sorted tests */
USTRUCT()
struct FEnvQueryInstanceCache
{
	GENERATED_USTRUCT_BODY()

	/** query template, duplicated in manager's world */
	UPROPERTY()
	TObjectPtr<UEnvQuery> Template = nullptr;

	/** instance to duplicate */
	UPROPERTY(Transient)
	FEnvQueryInstance Instance;

	/** the name of the source query */
	FName AssetName;
};

#if USE_EQS_DEBUGGER
struct FEQSDebugger
{
	struct FEnvQueryInfo
	{
		TSharedPtr<FEnvQueryInstance> Instance;
		double Timestamp;
	};

	struct FStatsInfo
	{
		// most expensive run
		FEnvQueryDebugProfileData MostExpensive;
		double MostExpensiveDuration;

		// average run (sum of all runs, divide by TotalAvgCount to get values)
		FEnvQueryDebugProfileData TotalAvgData;
		double TotalAvgDuration;
		int32 TotalAvgCount;

		// EQS tick load
		TArray<uint8> TickPct;
		double LastTickTime;
		uint64 LastTickFrame;
		uint16 FirstTickEntry;
		uint16 LastTickEntry;

		FStatsInfo() :
			MostExpensiveDuration(0.0f),
			TotalAvgDuration(0.0f), TotalAvgCount(0),
			LastTickTime(0.0f), LastTickFrame(0),
			FirstTickEntry(MAX_uint16), LastTickEntry(0)
		{}
	};

	AIMODULE_API void StoreStats(const FEnvQueryInstance& QueryInstance);
	AIMODULE_API void StoreTickTime(const FEnvQueryInstance& QueryInstance, double TickTime, double MaxTickTime);
	AIMODULE_API void StoreQuery(const TSharedPtr<FEnvQueryInstance>& QueryInstance);
	
	static AIMODULE_API void SaveStats(const FString& FileName);
	static AIMODULE_API void LoadStats(const FString& FileName);

	AIMODULE_API const TArray<FEnvQueryInfo>& GetAllQueriesForOwner(const UObject* Owner);

	// map query name with profiler data
	TMap<FName, FStatsInfo> StoredStats;

protected:
	// maps owner to performed queries
	TMap<FObjectKey, TArray<FEnvQueryInfo>> StoredQueries;
};

FArchive& operator<<(FArchive& Ar, FEQSDebugger::FStatsInfo& Data);

inline bool operator== (const FEQSDebugger::FEnvQueryInfo & Left, const FEQSDebugger::FEnvQueryInfo & Right)
{
	return Left.Instance == Right.Instance;
}
#endif // USE_EQS_DEBUGGER

UCLASS(config = Game, defaultconfig, Transient, MinimalAPI)
class UEnvQueryManager : public UAISubsystem, public FSelfRegisteringExec
{
	GENERATED_UCLASS_BODY()

	// makes sure we don't have any UEnvQueryManager instances serialized in. 
	// Any loaded instance will get marked as PendingKill
	AIMODULE_API virtual void PostLoad() override;
	AIMODULE_API virtual void PostInitProperties() override;

	// FTickableGameObject begin
	AIMODULE_API virtual void Tick(float DeltaTime) override;
	AIMODULE_API virtual TStatId GetStatId() const override;
	virtual bool IsTickableInEditor() const override { return true; }
	// FTickableGameObject end

	/** execute query */
	AIMODULE_API int32 RunQuery(const FEnvQueryRequest& Request, EEnvQueryRunMode::Type RunMode, FQueryFinishedSignature const& FinishDelegate);
	AIMODULE_API int32 RunQuery(const TSharedPtr<FEnvQueryInstance>& QueryInstance, FQueryFinishedSignature const& FinishDelegate);

	/** Removed all active queries asked by Querier. No "on finished" notifications are being sent, call this function when
	 *	you no longer care about Querier's queries, like when it's "dead" */
	void SilentlyRemoveAllQueriesByQuerier(const UObject& Querier)
	{
		RemoveAllQueriesByQuerier(Querier, /*bExecuteFinishDelegate=*/false);
	}

	AIMODULE_API void RemoveAllQueriesByQuerier(const UObject& Querier, bool bExecuteFinishDelegate);

	/** alternative way to run queries. Do not use for anything other than testing
	*  or when you know exactly what you're doing! Bypasses all EQS perf controlling
	*  and time slicing mechanics. */
	AIMODULE_API TSharedPtr<FEnvQueryResult> RunInstantQuery(const FEnvQueryRequest& Request, EEnvQueryRunMode::Type RunMode);
	AIMODULE_API void RunInstantQuery(const TSharedPtr<FEnvQueryInstance>& QueryInstance);

	/** Creates a query instance configured for execution */
	AIMODULE_API TSharedPtr<FEnvQueryInstance> PrepareQueryInstance(const FEnvQueryRequest& Request, EEnvQueryRunMode::Type RunMode);

	/** finds UEnvQuery matching QueryName by first looking at instantiated queries (from InstanceCache)
	 *	falling back to looking up all UEnvQuery and testing their name */
	AIMODULE_API UEnvQuery* FindQueryTemplate(const FString& QueryName) const;

	/** creates local context object */
	AIMODULE_API UEnvQueryContext* PrepareLocalContext(TSubclassOf<UEnvQueryContext> ContextClass);

	/** find value of named param stored with active query */
	AIMODULE_API float FindNamedParam(int32 QueryId, FName ParamName) const;

	/** aborts specific query */
	AIMODULE_API bool AbortQuery(int32 RequestID);

	/** outputs active queries to log */
	AIMODULE_API void PrintActiveQueryInfo() const;

	/** fail all running queries on cleaning the world */
	AIMODULE_API virtual void OnWorldCleanup();

	/** cleanup hooks for map loading */
	AIMODULE_API virtual void FinishDestroy() override;

	/** add information for data providers about query instance run independently */
	AIMODULE_API void RegisterExternalQuery(const TSharedPtr<FEnvQueryInstance>& QueryInstance);

	/** clear information about query instance run independently */
	AIMODULE_API void UnregisterExternalQuery(const TSharedPtr<FEnvQueryInstance>& QueryInstance);

	/** list of all known item types */
	static AIMODULE_API TArray<TSubclassOf<UEnvQueryItemType> > RegisteredItemTypes;

	static AIMODULE_API UEnvQueryManager* GetCurrent(UWorld* World);
	static AIMODULE_API UEnvQueryManager* GetCurrent(const UObject* WorldContextObject);
	
	UFUNCTION(BlueprintCallable, Category = "AI|EQS", meta = (WorldContext = "WorldContextObject", AdvancedDisplay = "WrapperClass"))
	static AIMODULE_API UEnvQueryInstanceBlueprintWrapper* RunEQSQuery(UObject* WorldContextObject, UEnvQuery* QueryTemplate, UObject* Querier, TEnumAsByte<EEnvQueryRunMode::Type> RunMode, TSubclassOf<UEnvQueryInstanceBlueprintWrapper> WrapperClass);

	AIMODULE_API void RegisterActiveWrapper(UEnvQueryInstanceBlueprintWrapper& Wrapper);
	AIMODULE_API void UnregisterActiveWrapper(UEnvQueryInstanceBlueprintWrapper& Wrapper);

	static AIMODULE_API void SetAllowTimeSlicing(bool bAllowTimeSlicing);

	/** Configure config variables during runtime */
	AIMODULE_API void Configure(const FEnvQueryManagerConfig& NewConfig);

protected:
	//~ Begin FExec Interface
	AIMODULE_API virtual bool Exec_Dev(UWorld* Inworld,const TCHAR* Cmd,FOutputDevice& Ar) override;
	//~ End FExec Interface

	friend UEnvQueryInstanceBlueprintWrapper;
	AIMODULE_API TSharedPtr<FEnvQueryInstance> FindQueryInstance(const int32 QueryID);

#if USE_EQS_DEBUGGER
public:
	static AIMODULE_API void NotifyAssetUpdate(UEnvQuery* Query);
	static AIMODULE_API TMap<FName, FEQSDebugger::FStatsInfo> DebuggerStats;

	FEQSDebugger& GetDebugger() { return EQSDebugger; }

protected:
	FEQSDebugger EQSDebugger;
#endif // USE_EQS_DEBUGGER

protected:

	/** currently running queries */
	TArray<TSharedPtr<FEnvQueryInstance> > RunningQueries;

	/** count of queries aborted since last update, to be removed. */
	int32 NumRunningQueriesAbortedSinceLastUpdate;

	/** queries run independently from manager, mapped here for data providers */
	TMap<int32, TWeakPtr<FEnvQueryInstance>> ExternalQueries;

	/** cache of instances */
	UPROPERTY(transient)
	TArray<FEnvQueryInstanceCache> InstanceCache;

	/** local cache of context objects for managing BP based objects */
	UPROPERTY(transient)
	TArray<TObjectPtr<UEnvQueryContext>> LocalContexts;

	UPROPERTY()
	TArray<TObjectPtr<UEnvQueryInstanceBlueprintWrapper>> GCShieldedWrappers;

	/** local contexts mapped by class names */
	TMap<FName, UEnvQueryContext*> LocalContextMap;

	/** next ID for running query */
	int32 NextQueryID;

	/** create new instance, using cached data is possible */
	AIMODULE_API TSharedPtr<FEnvQueryInstance> CreateQueryInstance(const UEnvQuery* Template, EEnvQueryRunMode::Type RunMode);

	/** how long are we allowed to test per update, in seconds. */
	UPROPERTY(config)
	float MaxAllowedTestingTime;

	/** whether we update EQS queries based on:
	    running a test on one query and move to the next (breadth) - default behavior,
	    or test an entire query before moving to the next one (depth). */
	UPROPERTY(config)
	bool bTestQueriesUsingBreadth;

	/** if greater than zero, we will warn once when the number of queries is greater than or equal to this number, and log the queries out */
	UPROPERTY(config)
	int32 QueryCountWarningThreshold;

	/** how often (in seconds) we will warn about the number of queries (allows us to catch multiple occurrences in a session) */
	UPROPERTY(config)
	double QueryCountWarningInterval;

	/** Maximum EQS execution duration (in seconds) before a warning is reported. */
	UPROPERTY(config)
	double ExecutionTimeWarningSeconds = 0.025f;

	/** Maximum EQS Query FinishDelegate duration (in seconds) before a warning is reported. */
	UPROPERTY(config)
	double HandlingResultTimeWarningSeconds = 0.025f;

	/** Maximum EQS Generator duration (in seconds) before a warning is reported. */
	UPROPERTY(config)
	double GenerationTimeWarningSeconds = 0.01f;
private:

	/** create and bind delegates in instance */
	AIMODULE_API void CreateOptionInstance(UEnvQueryOption* OptionTemplate, int32 SourceOptionIndex, const TArray<UEnvQueryTest*>& SortedTests, FEnvQueryInstance& Instance);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	static AIMODULE_API bool bAllowEQSTimeSlicing;
#endif
#if !(UE_BUILD_SHIPPING)
	mutable double LastQueryCountWarningThresholdTime;

	AIMODULE_API void CheckQueryCount() const;
	AIMODULE_API void LogQueryInfo(bool bDisplayThresholdWarning) const;
#endif

#if WITH_EDITOR
	FDelegateHandle OnBlueprintCompiledHandle;

	// used to reset LocalContextMap in case we had BP contexts stored
	AIMODULE_API void OnBlueprintCompiled();
#endif // WITH_EDITOR
};
