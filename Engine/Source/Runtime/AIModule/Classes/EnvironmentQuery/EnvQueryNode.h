// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "EnvQueryNode.generated.h"

struct FPropertyChangedEvent;

UCLASS(Abstract, MinimalAPI)
class UEnvQueryNode : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Versioning for updating deprecated properties */
	UPROPERTY()
	int32 VerNum;

	AIMODULE_API virtual void UpdateNodeVersion();

	AIMODULE_API virtual FText GetDescriptionTitle() const;
	AIMODULE_API virtual FText GetDescriptionDetails() const;

	/**
	 * To be extended by any Node who offloads its work to another thread.
	 * Returns false by default, unless overridden.
	 * If overridden, will return whether or not this Node is currently being processed asynchronously. 
	 */
	virtual inline bool IsCurrentlyRunningAsync() const { return false; }

#if WITH_EDITOR
	AIMODULE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif //WITH_EDITOR
};
