// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "EnvQueryOption.generated.h"

class UEnvQueryGenerator;
class UEnvQueryTest;

UCLASS(MinimalAPI)
class UEnvQueryOption : public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TObjectPtr<UEnvQueryGenerator> Generator;

	UPROPERTY()
	TArray<TObjectPtr<UEnvQueryTest>> Tests;

	AIMODULE_API FText GetDescriptionTitle() const;
	AIMODULE_API FText GetDescriptionDetails() const;
};
