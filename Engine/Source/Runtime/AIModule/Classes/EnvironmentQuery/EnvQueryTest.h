// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "EnvironmentQuery/Items/EnvQueryItemType.h"
#include "EnvironmentQuery/EnvQueryContext.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "DataProviders/AIDataProvider.h"
#include "EnvironmentQuery/EnvQueryNode.h"
#include "EnvQueryTest.generated.h"

class AActor;

#if WITH_EDITOR
struct FPropertyChangedEvent;
#endif // WITH_EDITOR

UENUM()
enum class EEQSNormalizationType : uint8
{
	// Use 0 as the base of normalization range.
	Absolute,
	// Use lowest item score as the base of normalization range.
	RelativeToScores
};

namespace EnvQueryTestVersion
{
	inline const int32 Initial = 0;
	inline const int32 DataProviders = 1;
	inline const int32 ReferenceValueFix = 2;

	inline const int32 Latest = ReferenceValueFix;
}

#if WITH_EDITORONLY_DATA
struct FEnvQueryTestScoringPreview
{
	enum 
	{
		// used for built-in functions (@see EEnvTestScoreEquation)
		DefaultSamplesCount = 21
	};

	TArray<float> Samples;
	float FilterLow;
	float FilterHigh;
	float ClampMin;
	float ClampMax;
	uint32 bShowFilterLow : 1;
	uint32 bShowFilterHigh : 1;
	uint32 bShowClampMin : 1;
	uint32 bShowClampMax : 1;
};
#endif

UCLASS(Abstract, MinimalAPI)
class UEnvQueryTest : public UEnvQueryNode
{
	GENERATED_UCLASS_BODY()

	/** Number of test as defined in data asset */
	UPROPERTY()
	int32 TestOrder;

	/** The purpose of this test.  Should it be used for filtering possible results, scoring them, or both? */
	UPROPERTY(EditDefaultsOnly, Category=Test)
	TEnumAsByte<EEnvTestPurpose::Type> TestPurpose;

	/** Optional comment or explanation about what this test is for.  Useful when the purpose of tests may not be clear,
	  * especially when there are multiple tests of the same type. */
	UPROPERTY(EditDefaultsOnly, Category=Test)
	FString TestComment;

	/** Determines filtering operator when context returns multiple items */
	UPROPERTY(EditDefaultsOnly, Category=Filter, AdvancedDisplay)
	TEnumAsByte<EEnvTestFilterOperator::Type> MultipleContextFilterOp;

	/** Determines scoring operator when context returns multiple items */
	UPROPERTY(EditDefaultsOnly, Category = Score, AdvancedDisplay)
	TEnumAsByte<EEnvTestScoreOperator::Type> MultipleContextScoreOp;

	/** Does this test filter out results that are below a lower limit, above an upper limit, or both?  Or does it just look for a matching value? */
	UPROPERTY(EditDefaultsOnly, Category=Filter)
	TEnumAsByte<EEnvTestFilterType::Type> FilterType;

	/** Desired boolean value of the test for scoring to occur or filtering test to pass. */
	UPROPERTY(EditDefaultsOnly, Category=Filter)
	FAIDataProviderBoolValue BoolValue;

	/** Minimum limit (inclusive) of valid values for the raw test value. Lower values will be discarded as invalid. */
	UPROPERTY(EditDefaultsOnly, Category=Filter)
	FAIDataProviderFloatValue FloatValueMin;

	/** Maximum limit (inclusive) of valid values for the raw test value. Higher values will be discarded as invalid. */
	UPROPERTY(EditDefaultsOnly, Category=Filter)
	FAIDataProviderFloatValue FloatValueMax;

	/** Cost of test */
	TEnumAsByte<EEnvTestCost::Type> Cost;

	/** The shape of the curve equation to apply to the normalized score before multiplying by factor. */
	UPROPERTY(EditDefaultsOnly, Category=Score)
	TEnumAsByte<EEnvTestScoreEquation::Type> ScoringEquation;

	/** How should the lower bound for normalization of the raw test value before applying the scoring formula be determined?
	    Should it use the lowest value found (tested), the lower threshold for filtering, or a separate specified normalization minimum? */
	UPROPERTY(EditDefaultsOnly, Category=Score)
	TEnumAsByte<EEnvQueryTestClamping::Type> ClampMinType;

	/** How should the upper bound for normalization of the raw test value before applying the scoring formula be determined?
	    Should it use the highest value found (tested), the upper threshold for filtering, or a separate specified normalization maximum? */
	UPROPERTY(EditDefaultsOnly, Category=Score)
	TEnumAsByte<EEnvQueryTestClamping::Type> ClampMaxType;

	/** Specifies how to determine value span used to normalize scores */
	UPROPERTY(EditDefaultsOnly, Category = Score)
	EEQSNormalizationType NormalizationType;

	/** Minimum value to use to normalize the raw test value before applying scoring formula. */
	UPROPERTY(EditDefaultsOnly, Category=Score)
	FAIDataProviderFloatValue ScoreClampMin;

	/** Maximum value to use to normalize the raw test value before applying scoring formula. */
	UPROPERTY(EditDefaultsOnly, Category=Score)
	FAIDataProviderFloatValue ScoreClampMax;

	/** The weight (factor) by which to multiply the normalized score after the scoring equation is applied. */
	UPROPERTY(EditDefaultsOnly, Category=Score, Meta=(ClampMin="0.001", UIMin="0.001"))
	FAIDataProviderFloatValue ScoringFactor;

	/** When specified gets used to normalize test's results in such a way that the closer a value is to ReferenceValue
	 *	the higher normalized result it will produce. Value farthest from ReferenceValue will be normalized
	 *	to 0, and all the other values in between will get normalized linearly with the distance to ReferenceValue. */
	UPROPERTY(EditDefaultsOnly, Category = Score, Meta=(EditCondition = "bDefineReferenceValue"))
	FAIDataProviderFloatValue ReferenceValue;

	/** When set to true enables usage of ReferenceValue. It's false by default */
	UPROPERTY(EditDefaultsOnly, Category = Score, meta=(InlineEditConditionToggle))
	bool bDefineReferenceValue;
	
	/** Validation: item type that can be used with this test */
	TSubclassOf<UEnvQueryItemType> ValidItemType;

#if WITH_EDITORONLY_DATA
	/** samples of scoring function to show on graph in editor */
	FEnvQueryTestScoringPreview PreviewData;
#endif

	AIMODULE_API void SetWorkOnFloatValues(bool bWorkOnFloats);
	inline bool GetWorkOnFloatValues() const { return bWorkOnFloatValues; }

	inline bool CanRunAsFinalCondition() const 
	{ 
		return (TestPurpose != EEnvTestPurpose::Score)							// We are filtering and...
				&& ((TestPurpose == EEnvTestPurpose::Filter)					// Either we are NOT scoring at ALL or...
					|| (ScoringEquation == EEnvTestScoreEquation::Constant));	// We are giving a constant score value for passing.
	}

private:
	/** When set, test operates on float values (e.g. distance, with AtLeast, UpTo conditions),
	 *  otherwise it will accept bool values (e.g. visibility, with Equals condition) */
	UPROPERTY()
	uint32 bWorkOnFloatValues : 1;

public:

	/** Function that does the actual work */
	virtual void RunTest(FEnvQueryInstance& QueryInstance) const { checkNoEntry(); }

	/** check if test supports item type */
	bool IsSupportedItem(TSubclassOf<UEnvQueryItemType> ItemType) const;

	/** check if context needs to be updated for every item */
	AIMODULE_API bool IsContextPerItem(TSubclassOf<UEnvQueryContext> CheckContext) const;

	/** helper: get location of item */
	AIMODULE_API FVector GetItemLocation(FEnvQueryInstance& QueryInstance, int32 ItemIndex) const;

	/** helper: get location of item */
	inline FVector GetItemLocation(FEnvQueryInstance& QueryInstance, const FEnvQueryInstance::ItemIterator& Iterator) const
	{
		return GetItemLocation(QueryInstance, Iterator.GetIndex());
	}

	/** helper: get location of item */
	AIMODULE_API FRotator GetItemRotation(FEnvQueryInstance& QueryInstance, int32 ItemIndex) const;

	/** helper: get location of item */
	inline FRotator GetItemRotation(FEnvQueryInstance& QueryInstance, const FEnvQueryInstance::ItemIterator& Iterator) const
	{
		return GetItemRotation(QueryInstance, Iterator.GetIndex());
	}

	/** helper: get actor from item */
	AIMODULE_API AActor* GetItemActor(FEnvQueryInstance& QueryInstance, int32 ItemIndex) const;
		
	/** helper: get actor from item */
	inline AActor* GetItemActor(FEnvQueryInstance& QueryInstance, const FEnvQueryInstance::ItemIterator& Iterator) const
	{
		return GetItemActor(QueryInstance, Iterator.GetIndex());
	}

	/** normalize scores in range */
	AIMODULE_API void NormalizeItemScores(FEnvQueryInstance& QueryInstance);

	inline bool IsScoring() const { return (TestPurpose != EEnvTestPurpose::Filter); } 
	inline bool IsFiltering() const { return (TestPurpose != EEnvTestPurpose::Score); }

	AIMODULE_API FText DescribeFloatTestParams() const;
	AIMODULE_API FText DescribeBoolTestParams(const FString& ConditionDesc) const;

	AIMODULE_API virtual void PostLoad() override;

	/** update to latest version after spawning */
	AIMODULE_API virtual void UpdateNodeVersion() override;

#if WITH_EDITOR
	AIMODULE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	/** update preview list */
	AIMODULE_API void UpdatePreviewData();
};

//////////////////////////////////////////////////////////////////////////
// Inlines

inline bool UEnvQueryTest::IsSupportedItem(TSubclassOf<UEnvQueryItemType> ItemType) const
{
	return ItemType && (ItemType == ValidItemType || ItemType->IsChildOf(ValidItemType));
};
