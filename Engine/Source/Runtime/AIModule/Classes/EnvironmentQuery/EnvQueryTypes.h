// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "EnvironmentQuery/Items/EnvQueryItemType.h"
#include "EnvironmentQuery/EnvQueryContext.h"
#include "BehaviorTree/BehaviorTreeTypes.h"
#include "EnvQueryTypes.generated.h"

class AActor;
class ANavigationData;
class Error;
class UBlackboardComponent;
class UBlackboardData;
class UEnvQuery;
class UEnvQueryGenerator;
class UEnvQueryItemType_ActorBase;
class UEnvQueryItemType_VectorBase;
class UEnvQueryTest;
class UNavigationQueryFilter;
struct FEnvQueryInstance;

AIMODULE_API DECLARE_LOG_CATEGORY_EXTERN(LogEQS, Display, All);

// If set, execution details will be processed by debugger
#ifndef USE_EQS_DEBUGGER
#define USE_EQS_DEBUGGER				(1 && !(UE_BUILD_SHIPPING || UE_BUILD_TEST))
#endif

// If set, execution stats will also gather EQS tick load data (16k memory for each query record)
#define USE_EQS_TICKLOADDATA			(1 && USE_EQS_DEBUGGER && WITH_EDITOR)

DECLARE_STATS_GROUP(TEXT("Environment Query"), STATGROUP_AI_EQS, STATCAT_Advanced);

DECLARE_CYCLE_STAT_EXTERN(TEXT("Tick"),STAT_AI_EQS_Tick,STATGROUP_AI_EQS, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Tick - EQS work"), STAT_AI_EQS_TickWork, STATGROUP_AI_EQS, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Tick - OnFinished delegates"), STAT_AI_EQS_TickNotifies, STATGROUP_AI_EQS, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Tick - Removal of completed queries"), STAT_AI_EQS_TickQueryRemovals, STATGROUP_AI_EQS, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Load Time"),STAT_AI_EQS_LoadTime,STATGROUP_AI_EQS, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Execute One Step Time"),STAT_AI_EQS_ExecuteOneStep,STATGROUP_AI_EQS, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Generator Time"),STAT_AI_EQS_GeneratorTime,STATGROUP_AI_EQS, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Test Time"),STAT_AI_EQS_TestTime,STATGROUP_AI_EQS, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("EQS Debug StoreQuery"), STAT_AI_EQS_Debug_StoreQuery, STATGROUP_AI_EQS, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("EQS Debug StoreTickTime"), STAT_AI_EQS_Debug_StoreTickTime, STATGROUP_AI_EQS, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("EQS Debug StoreStats"), STAT_AI_EQS_Debug_StoreStats, STATGROUP_AI_EQS, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Num Instances"),STAT_AI_EQS_NumInstances,STATGROUP_AI_EQS, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Num Items"),STAT_AI_EQS_NumItems,STATGROUP_AI_EQS, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Instance memory"),STAT_AI_EQS_InstanceMemory,STATGROUP_AI_EQS, AIMODULE_API);
DECLARE_FLOAT_COUNTER_STAT_EXTERN(TEXT("Avg Instance Response Time (ms)"), STAT_AI_EQS_AvgInstanceResponseTime, STATGROUP_AI_EQS, );

UENUM()
namespace EEnvTestPurpose
{
	enum Type : int
	{
		Filter UMETA(DisplayName="Filter Only"),
		Score UMETA(DisplayName="Score Only"),
		FilterAndScore UMETA(DisplayName="Filter and Score")
	};
}

UENUM()
namespace EEnvTestFilterType
{
	enum Type : int
	{
		/** For numeric tests. */
		Minimum,
		/** For numeric tests. */
		Maximum,
		/** For numeric tests. */
		Range,
		/** For Boolean tests. */
		Match
	};
}

UENUM()
namespace EEnvTestScoreEquation
{
	enum Type : int
	{
		Linear,
		Square,
		InverseLinear,
		SquareRoot,

		Constant
		// What other curve shapes should be supported?  At first I was thinking we'd have parametric (F*V^P + C), but
		// many versions of that curve would violate the [0, 1] output range which I think we should preserve.  So instead
		// I think we should define these by "curve shape".  I'm not sure if we need to allow full tweaks to the curves,
		// such as supporting other "Exponential" curves (positive even powers).  However, I think it's likely that we'll
		// want to support "smooth LERP" / S-shaped curve of the form 2x^3 - 3x^2, and possibly a "sideways" version of
		// the same S-curve.  We also might want to allow "Sine" curves, basically adjusted to match the range and then
		// simply offset by some amount to allow a peak or valley in the middle or on the ends.  (Four Sine options are
		// probably sufficient.)  I'm not sure if Sine is really needed though, so probably we should only add it if
		// there's a need identified.  One other curve shape we might want is "Square Root", which might optionally
		// support any positive fractional power (if we also supported any positive even number for an "Exponential"
		// type.
	};
}

UENUM()
namespace EEnvTestWeight
{
	enum Type : int
	{
		None,
		Square,
		Inverse,
		Unused			UMETA(Hidden),
		Constant,
		Skip			UMETA(DisplayName = "Do not weight"),
	};
}

UENUM()
namespace EEnvTestCost
{
	enum Type : int
	{
		/** Reading data, math operations (e.g. distance). */
		Low,
		/** Processing data from multiple sources (e.g. fire tickets). */
		Medium,
		/** Really expensive calls (e.g. visibility traces, pathfinding).  */
		High,
	};
}

UENUM()
namespace EEnvTestFilterOperator
{
	enum Type : int
	{
		AllPass			UMETA(Tooltip = "All contexts must pass condition"),
		AnyPass			UMETA(Tooltip = "At least one context must pass condition"),
	};
}

UENUM()
namespace EEnvTestScoreOperator
{
	enum Type : int
	{
		AverageScore	UMETA(Tooltip = "Use average score from all contexts"),
		MinScore		UMETA(Tooltip = "Use minimum score from all contexts"),
		MaxScore		UMETA(Tooltip = "Use maximum score from all contexts"),
		Multiply		UMETA(Tooltip = "Multiply scores from all contexts"),
	};
}

namespace EEnvItemStatus
{
	enum Type
	{
		Passed,
		Failed,
	};
}

UENUM(BlueprintType)
namespace EEnvQueryStatus
{
	enum Type : int
	{
		Processing,
		Success,
		Failed,
		Aborted,
		OwnerLost,
		MissingParam,
	};
}

UENUM()
namespace EEnvQueryRunMode
{
	enum Type : int
	{
		SingleResult	UMETA(Tooltip="Pick first item with the best score", DisplayName="Single Best Item"),
		RandomBest5Pct	UMETA(Tooltip="Pick random item with score 95% .. 100% of max", DisplayName="Single Random Item from Best 5%"),
		RandomBest25Pct	UMETA(Tooltip="Pick random item with score 75% .. 100% of max", DisplayName="Single Random Item from Best 25%"),
		AllMatching		UMETA(Tooltip="Get all items that match conditions"),
	};
}

UENUM()
enum class EEnvQueryResultNormalizationOption : uint8
{
	Default,
	Normalized,
	Unaltered
};

UENUM()
namespace EEnvQueryParam
{
	enum Type : int
	{
		Float,
		Int,
		Bool,
	};
}

UENUM()
enum class EAIParamType : uint8
{
	Float,
	Int,
	Bool,
	MAX UMETA(Hidden)
};

UENUM()
namespace EEnvQueryTrace
{
	enum Type : int
	{
		None,
		Navigation,
		GeometryByChannel,
		GeometryByProfile,
		NavigationOverLedges
	};
}

UENUM()
namespace EEnvTraceShape
{
	enum Type : int
	{
		Line,
		Box,
		Sphere,
		Capsule,
	};
}

UENUM()
namespace EEnvOverlapShape
{
	enum Type : int
	{
		Box,
		Sphere,
		Capsule,
	};
}

UENUM(meta=(ScriptName="EnvDirectionType"))
namespace EEnvDirection
{
	enum Type : int
	{
		TwoPoints	UMETA(DisplayName="Two Points",ToolTip="Direction from location of one context to another."),
		Rotation	UMETA(ToolTip="Context's rotation will be used as a direction."),
	};
}

UENUM()
namespace EEnvQueryTestClamping
{
	enum Type : int
	{
		None,			
		/** Clamp to value specified in test. */
		SpecifiedValue,
		/** Clamp to test's filter threshold. */
		FilterThreshold
	};
}

USTRUCT(BlueprintType)
struct FEnvNamedValue
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Param)
	FName ParamName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Param)
	EAIParamType ParamType;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Param)
	float Value;

	FEnvNamedValue()
		: ParamType(EAIParamType::Float), Value(0.f)
	{}
};

USTRUCT()
struct FEnvDirection
{
	GENERATED_USTRUCT_BODY()

	/** line: start context */
	UPROPERTY(EditDefaultsOnly, Category=Direction)
	TSubclassOf<UEnvQueryContext> LineFrom;

	/** line: finish context */
	UPROPERTY(EditDefaultsOnly, Category=Direction)
	TSubclassOf<UEnvQueryContext> LineTo;

	/** rotation: direction context */
	UPROPERTY(EditDefaultsOnly, Category=Direction)
	TSubclassOf<UEnvQueryContext> Rotation;

	/** defines direction of second line used by test */
	UPROPERTY(EditDefaultsOnly, Category=Direction, meta=(DisplayName="Mode"))
	TEnumAsByte<EEnvDirection::Type> DirMode = EEnvDirection::Type::TwoPoints;

	AIMODULE_API FText ToText() const;
};

USTRUCT()
struct FEnvTraceData
{
	GENERATED_USTRUCT_BODY()

	enum EDescriptionMode
	{
		Brief,
		Detailed,
	};

	AIMODULE_API FEnvTraceData();

	/** version number for updates */
	UPROPERTY()
	int32 VersionNum;

	/** navigation filter for tracing */
	UPROPERTY(EditDefaultsOnly, Category=Trace)
	TSubclassOf<UNavigationQueryFilter> NavigationFilter;

	/** search height: below point */
	UPROPERTY(EditDefaultsOnly, Category=Trace, meta=(UIMin=0, ClampMin=0))
	float ProjectDown;

	/** search height: above point */
	UPROPERTY(EditDefaultsOnly, Category=Trace, meta=(UIMin=0, ClampMin=0))
	float ProjectUp;

	/** shape parameter for trace */
	UPROPERTY(EditDefaultsOnly, Category=Trace, meta=(UIMin=0, ClampMin=0))
	float ExtentX;

	/** shape parameter for trace */
	UPROPERTY(EditDefaultsOnly, Category=Trace, meta=(UIMin=0, ClampMin=0))
	float ExtentY;

	/** shape parameter for trace */
	UPROPERTY(EditDefaultsOnly, Category=Trace, meta=(UIMin=0, ClampMin=0))
	float ExtentZ;

	/** this value will be added to resulting location's Z axis. Can be useful when 
	 *	projecting points to navigation since navmesh is just an approximation of level 
	 *	geometry and items may end up being under collide-able geometry which would 
	 *	for example falsify visibility tests.*/
	UPROPERTY(EditDefaultsOnly, Category=Trace)
	float PostProjectionVerticalOffset;

	/** geometry trace channel */
	UPROPERTY(EditDefaultsOnly, Category=Trace)
	TEnumAsByte<enum ETraceTypeQuery> TraceChannel;

	/** geometry trace channel for serialization purposes */
	UPROPERTY(EditDefaultsOnly, Category=Trace)
	TEnumAsByte<enum ECollisionChannel> SerializedChannel;

	/** geometry trace profile */
	UPROPERTY(EditDefaultsOnly, Category = Trace)
	FName TraceProfileName;

	/** shape used for geometry tracing */
	UPROPERTY(EditDefaultsOnly, Category=Trace)
	TEnumAsByte<EEnvTraceShape::Type> TraceShape;

	/** shape used for geometry tracing */
	UPROPERTY(EditDefaultsOnly, Category=Trace)
	TEnumAsByte<EEnvQueryTrace::Type> TraceMode;

	/** if set, trace will run on complex collisions */
	UPROPERTY(EditDefaultsOnly, Category=Trace, AdvancedDisplay)
	uint32 bTraceComplex : 1;

	/** if set, trace will look only for blocking hits */
	UPROPERTY(EditDefaultsOnly, Category=Trace, AdvancedDisplay)
	uint32 bOnlyBlockingHits : 1;

	/** if set, editor will allow picking navmesh trace */
	UPROPERTY(EditDefaultsOnly, Category=Trace)
	uint32 bCanTraceOnNavMesh : 1;

	/** if set, editor will allow picking geometry trace */
	UPROPERTY(EditDefaultsOnly, Category=Trace)
	uint32 bCanTraceOnGeometry : 1;

	/** if set, editor will allow  */
	UPROPERTY(EditDefaultsOnly, Category=Trace)
	uint32 bCanDisableTrace : 1;

	/** if set, editor show height up/down properties for projection */
	UPROPERTY(EditDefaultsOnly, Category=Trace)
	uint32 bCanProjectDown : 1;

	AIMODULE_API FText ToText(EDescriptionMode DescMode) const;

	AIMODULE_API void SetGeometryOnly();
	AIMODULE_API void SetNavmeshOnly();
	
	AIMODULE_API void OnPostLoad();
};

USTRUCT()
struct FEnvOverlapData
{
	GENERATED_USTRUCT_BODY()

	FEnvOverlapData() :
		ExtentX(10.0f),
		ExtentY(10.0f),
		ExtentZ(10.0f),
		ShapeOffset(FVector::ZeroVector),
		OverlapChannel(ECC_WorldStatic),
		OverlapShape(EEnvOverlapShape::Box),
		bOnlyBlockingHits(true),
		bOverlapComplex(false),
		bSkipOverlapQuerier(false)
	{
	}

	/** shape parameter for overlap */
	UPROPERTY(EditDefaultsOnly, Category = Trace, meta = (UIMin = 0, ClampMin = 0))
	float ExtentX;

	/** shape parameter for overlap */
	UPROPERTY(EditDefaultsOnly, Category = Trace, meta = (UIMin = 0, ClampMin = 0))
	float ExtentY;

	/** shape parameter for overlap */
	UPROPERTY(EditDefaultsOnly, Category = Trace, meta = (UIMin = 0, ClampMin = 0))
	float ExtentZ;

	UPROPERTY(EditDefaultsOnly, Category = Trace, AdvancedDisplay, Meta =
		(Tooltip="Offset from the item location at which to test the overlap.  For example, you may need to offset vertically to avoid overlaps with flat ground."))
	FVector ShapeOffset;

	/** geometry trace channel used for overlap */
	UPROPERTY(EditDefaultsOnly, Category = Overlap)
	TEnumAsByte<enum ECollisionChannel> OverlapChannel;

	/** shape used for geometry overlap */
	UPROPERTY(EditDefaultsOnly, Category = Overlap)
	TEnumAsByte<EEnvOverlapShape::Type> OverlapShape;

	/** if set, overlap will look only for blocking hits */
	UPROPERTY(EditDefaultsOnly, Category = Overlap, AdvancedDisplay)
	uint32 bOnlyBlockingHits : 1;

	/** if set, overlap will run on complex collisions */
	UPROPERTY(EditDefaultsOnly, Category = Overlap, AdvancedDisplay)
	uint32 bOverlapComplex : 1;

	/** if set, overlap will skip querier context hits */
	UPROPERTY(EditDefaultsOnly, Category = Overlap, AdvancedDisplay)
	uint32 bSkipOverlapQuerier : 1;
};

//////////////////////////////////////////////////////////////////////////
// Returned results

struct FEnvQueryItem
{
	/** total score of item */
	float Score;

	/** raw data offset */
	int32 DataOffset:31;

	/** has this item been discarded? */
	uint32 bIsDiscarded:1;

	inline bool IsValid() const { return DataOffset >= 0 && !bIsDiscarded; }
	inline void Discard() { bIsDiscarded = true; }

	bool operator<(const FEnvQueryItem& Other) const
	{
		// sort by validity
		if (IsValid() != Other.IsValid())
		{
			// self not valid = less important
			return !IsValid();
		}

		// sort by score if not equal. As last resort sort by DataOffset to achieve stable sort.
		return Score != Other.Score ? Score < Other.Score : DataOffset < Other.DataOffset;
	}

	FEnvQueryItem() : Score(0.0f), DataOffset(-1), bIsDiscarded(false) {}
	FEnvQueryItem(int32 InOffset) : Score(0.0f), DataOffset(InOffset), bIsDiscarded(false) {}
};

template <> struct TIsZeroConstructType<FEnvQueryItem> { enum { Value = true }; };

USTRUCT(BlueprintType)
struct FEnvQueryResult
{
	GENERATED_USTRUCT_BODY()

	TArray<FEnvQueryItem> Items;

	/** type of generated items */
	UPROPERTY(BlueprintReadOnly, Category = "EQS")
	TSubclassOf<UEnvQueryItemType> ItemType;

	/** raw data of items */
	TArray<uint8> RawData;

private:
	/** query status */
	EEnvQueryStatus::Type Status;

public:
	/** index of query option, that generated items */
	UPROPERTY(BlueprintReadOnly, Category = "EQS")
	int32 OptionIndex;

	/** instance ID */
	UPROPERTY(BlueprintReadOnly, Category = "EQS")
	int32 QueryID;

	/** instance owner. Mind that it doesn't have to be the query's "Querier". This is just the object that is responsible for this query instance. */
	TWeakObjectPtr<UObject> Owner;

	inline float GetItemScore(int32 Index) const { return Items.IsValidIndex(Index) ? Items[Index].Score : 0.0f; }

	/** item accessors for basic types */
	AIMODULE_API AActor* GetItemAsActor(int32 Index) const;
	AIMODULE_API FVector GetItemAsLocation(int32 Index) const;

	template<typename TItemType>
	const typename TItemType::FValueType& GetItemAsTypeChecked(const int32 Index) const
	{
		check(Items.IsValidIndex(Index) 
			&& ItemType.Get()
			&& ItemType->IsChildOf(TItemType::StaticClass()));

		return TItemType::template GetValueFromMemory<typename TItemType::FValueType>(GetItemRawMemory(Index));
	}

	/** note that this function does not strip out the null-actors to not mess up results of GetItemScore(Index) calls*/
	AIMODULE_API void GetAllAsActors(TArray<AActor*>& OutActors) const;
	AIMODULE_API void GetAllAsLocations(TArray<FVector>& OutLocations) const;

	const uint8* GetItemRawMemory(const int32 Index) const
	{
		check(Items.IsValidIndex(Index));
		return RawData.GetData() + Items[Index].DataOffset;
	}

	FEnvQueryResult() : ItemType(NULL), Status(EEnvQueryStatus::Processing), OptionIndex(0), QueryID(0) {}
	FEnvQueryResult(const EEnvQueryStatus::Type& InStatus) : ItemType(NULL), Status(InStatus), OptionIndex(0), QueryID(0) {}

	inline bool IsFinished() const { return Status != EEnvQueryStatus::Processing; }
	inline bool IsAborted() const { return Status == EEnvQueryStatus::Aborted; }
	inline bool IsSuccessful() const { return Status == EEnvQueryStatus::Success; }
	inline void MarkAsMissingParam() { Status = EEnvQueryStatus::MissingParam; }
	inline void MarkAsAborted() { Status = EEnvQueryStatus::Aborted; }
	inline void MarkAsFailed() { Status = EEnvQueryStatus::Failed; }
	inline void MarkAsFinishedWithoutIssues() { Status = EEnvQueryStatus::Success; }
	inline void MarkAsOwnerLost() { Status = EEnvQueryStatus::OwnerLost; }

	inline EEnvQueryStatus::Type GetRawStatus() const { return Status; }

	UE_DEPRECATED(5.0, "FEnvQueryResult::IsSuccsessful is deprecated. Use FEnvQueryResult::IsSuccessful instead.")
	inline bool IsSuccsessful() const { return IsSuccessful(); }
};


//////////////////////////////////////////////////////////////////////////
// Runtime processing structures

DECLARE_DELEGATE_OneParam(FQueryFinishedSignature, TSharedPtr<FEnvQueryResult>);

struct FEnvQuerySpatialData
{
	FVector Location;
	FRotator Rotation;
};

/** Detailed information about item, used by tests */
struct FEnvQueryItemDetails
{
	/** Results assigned by option's tests, before any modifications */
	TArray<float> TestResults;

#if USE_EQS_DEBUGGER
	/** Results assigned by option's tests, after applying modifiers, normalization and weight */
	TArray<float> TestWeightedScores;

	int32 FailedTestIndex;
	int32 ItemIndex;
	FString FailedDescription;
#endif // USE_EQS_DEBUGGER

	FEnvQueryItemDetails() {}
	FEnvQueryItemDetails(int32 NumTests, int32 InItemIndex)
	{
		TestResults.AddZeroed(NumTests);
#if USE_EQS_DEBUGGER
		TestWeightedScores.AddZeroed(NumTests);
		ItemIndex = InItemIndex;
		FailedTestIndex = INDEX_NONE;
#endif
	}

	inline uint32 GetAllocatedSize() const
	{
		return sizeof(*this) +
#if USE_EQS_DEBUGGER
			TestWeightedScores.GetAllocatedSize() +
#endif
			TestResults.GetAllocatedSize();
	}
};

USTRUCT()
struct FEnvQueryContextData
{
	GENERATED_BODY()

	/** type of context values */
	UPROPERTY(Transient)
	TSubclassOf<UEnvQueryItemType> ValueType;

	/** number of stored values */
	int32 NumValues;

	/** data of stored values */
	TArray<uint8> RawData;

	FEnvQueryContextData()
		: NumValues(0)
	{}

	inline uint32 GetAllocatedSize() const { return sizeof(*this) + RawData.GetAllocatedSize(); }
};

USTRUCT()
struct FEnvQueryOptionInstance
{
	GENERATED_BODY()

	/** generator object */
	UPROPERTY(transient)
	TObjectPtr<UEnvQueryGenerator> Generator;

	/** test objects */
	UPROPERTY(transient)
	TArray<TObjectPtr<UEnvQueryTest>> Tests;

	/** index of option in source asset */
	int32 SourceOptionIndex;

	/** type of generated items */
	UPROPERTY(transient)
	TSubclassOf<UEnvQueryItemType> ItemType;

	/** if set, items will be shuffled after tests */
	uint32 bHasNavLocations : 1;

	uint32 GetAllocatedSize() const
	{
		return sizeof(*this) + Tests.GetAllocatedSize();
	}
};

#if NO_LOGGING
#define EQSHEADERLOG(...)
#else
#define EQSHEADERLOG(msg) Log(msg)
#endif // NO_LOGGING

struct FEnvQueryDebugProfileData
{
	struct FStep
	{
		float ExecutionTime;
		int32 NumProcessedItems;

		FStep() : ExecutionTime(0.0f), NumProcessedItems(0) {}
	};

	// runtime stats, can be merged
	struct FOptionStat
	{
		TArray<FStep> StepData;
		int32 NumRuns;

		FOptionStat() : NumRuns(0) {}
	};

	// setup data
	struct FOptionData
	{
		int32 NumGenerators;
		TArray<FName> GeneratorNames;

		int32 OptionIdx;
		TArray<int32> TestIndices;

		FOptionData() : NumGenerators(1), OptionIdx(INDEX_NONE) {}
	};

	TArray<FOptionStat> OptionStats;
	TArray<FOptionData> OptionData;

	void Add(const FEnvQueryDebugProfileData& Other);
};

FArchive& operator<<(FArchive& Ar, FEnvQueryDebugProfileData::FStep& Data);
FArchive& operator<<(FArchive& Ar, FEnvQueryDebugProfileData::FOptionStat& Data);
FArchive& operator<<(FArchive& Ar, FEnvQueryDebugProfileData::FOptionData& Data);
FArchive& operator<<(FArchive& Ar, FEnvQueryDebugProfileData& Data);

struct FEnvQueryDebugData : public FEnvQueryDebugProfileData
{
	TArray<FEnvQueryItem> DebugItems;
	TArray<FEnvQueryItemDetails> DebugItemDetails;
	TArray<uint8> RawData;
	TArray<FString> PerformedTestNames;

	// indicates the query was run in a single-item mode and that it has been found
	uint32 bSingleItemResult : 1;

	/** composite generator can be counted correctly */
	int32 CurrentOptionGeneratorIdx = INDEX_NONE;

	FEnvQueryDebugData() : bSingleItemResult(false) {}
	
	void Store(const FEnvQueryInstance& QueryInstance, const float ExecutionTime, const bool bStepDone);
	void PrepareOption(const FEnvQueryInstance& QueryInstance, const TArray<UEnvQueryGenerator*>& Generators, const int32 NumTests);
};

// This struct is now deprecated, please use FEnvQueryDebugData instead.
struct FEQSQueryDebugData : public FEnvQueryDebugData
{
};

UCLASS(Abstract, MinimalAPI)
class UEnvQueryTypes : public UObject
{
	GENERATED_BODY()

public:
	/** special test value assigned to items skipped by condition check */
	static AIMODULE_API const float SkippedItemValue;

	/** special value used for executing query steps to prevent them from being time sliced */
	static AIMODULE_API float UnlimitedStepTime;

	static AIMODULE_API FText GetShortTypeName(const UObject* Ob);
	static AIMODULE_API FText DescribeContext(TSubclassOf<UEnvQueryContext> ContextClass);
};

USTRUCT()
struct FEnvQueryInstanceContextCacheItem
{
	GENERATED_BODY()

	UPROPERTY(transient)
	TObjectPtr<UClass> Class;

	UPROPERTY(transient)
	FEnvQueryContextData ContextData;
};

USTRUCT()
struct FEnvQueryInstance : public FEnvQueryResult
{
	GENERATED_BODY()

	typedef float FNamedParamValueType;

	/** short name of query template - friendly name for debugging */
	FString QueryName;

	/** unique name of query template - object name */
	FName UniqueName;

	/** world owning this query instance */
	UPROPERTY(transient)
	TObjectPtr<UWorld> World;

	/** observer's delegate */
	FQueryFinishedSignature FinishDelegate;

	/** execution params */
	TMap<FName, FNamedParamValueType> NamedParams;

	/** contexts in use */
	UPROPERTY(transient)
	TArray<FEnvQueryInstanceContextCacheItem> ContextCache;

	/** list of options */
	UPROPERTY(transient)
	TArray<FEnvQueryOptionInstance> Options;

	/** currently processed test (-1 = generator) */
	int32 CurrentTest;

	/** non-zero if test run last step has been stopped mid-process. This indicates
	 *	index of the first item that needs processing when resumed */
	int32 CurrentTestStartingItem;

	/** list of item details */
	TArray<FEnvQueryItemDetails> ItemDetails;

	/** number of valid items on list */
	int32 NumValidItems;

	/** size of current value */
	uint16 ValueSize;

#if USE_EQS_DEBUGGER
	/** number of items processed in current step */
	int32 NumProcessedItems;

	/** set to true to store additional debug info */
	uint8 bStoreDebugInfo : 1;
#endif // USE_EQS_DEBUGGER

	/** used to breaking from item iterator loops */
	uint8 bFoundSingleResult : 1;

	/** set when testing final condition of an option */
	uint8 bPassOnSingleResult : 1;

	/** true if this query has logged a warning that it overran the time limit */
	uint8 bHasLoggedTimeLimitWarning : 1;

	/** true if current Generator or Test is running asynchronously */
	uint8 bIsCurrentlyRunningAsync : 1;

	/** timestamp of creating query instance */
	double StartTime;

	/** time spent executing this query */
	double TotalExecutionTime;

	/** if > 0 then it's how much time query has for performing current step */
	double CurrentStepTimeLimit;

#if !UE_BUILD_SHIPPING
	/** Maximum EQS Generator duration (in seconds) before a warning is reported in non-shipping build. */
	double GenerationTimeWarningSeconds = 0.01f;
#endif // UE_BUILD_SHIPPING

	/** run mode */
	EEnvQueryRunMode::Type Mode;

	/** item type's CDO for location tests */
	UPROPERTY(transient)
	TObjectPtr<UEnvQueryItemType_VectorBase> ItemTypeVectorCDO;

	/** item type's CDO for actor tests */
	UPROPERTY(transient)
	TObjectPtr<UEnvQueryItemType_ActorBase> ItemTypeActorCDO;

	AIMODULE_API FEnvQueryInstance();
	AIMODULE_API FEnvQueryInstance(const FEnvQueryInstance& Other);
	AIMODULE_API ~FEnvQueryInstance();

	/** execute single step of query */
	AIMODULE_API void ExecuteOneStep(double TimeLimit);

	/** update context cache */
	AIMODULE_API bool PrepareContext(UClass* Context, FEnvQueryContextData& ContextData);

	/** helpers for reading spatial data from context */
	AIMODULE_API bool PrepareContext(UClass* Context, TArray<FEnvQuerySpatialData>& Data);
	AIMODULE_API bool PrepareContext(UClass* Context, TArray<FVector>& Data);
	AIMODULE_API bool PrepareContext(UClass* Context, TArray<FRotator>& Data);
	/** helpers for reading actor data from context */
	AIMODULE_API bool PrepareContext(UClass* Context, TArray<AActor*>& Data);
	
	bool IsInSingleItemFinalSearch() const { return !!bPassOnSingleResult; }
	/** check if current test can batch its calculations */
	bool CanBatchTest() const { return !IsInSingleItemFinalSearch(); }

	bool IsCurrentlyRunningAsync() const { return bIsCurrentlyRunningAsync; }

	/** raw data operations */
	AIMODULE_API void ReserveItemData(int32 NumAdditionalItems);

	template<typename TypeItem>
	void AddItemData(typename TypeItem::FValueType ItemValue)
	{
		DEC_MEMORY_STAT_BY(STAT_AI_EQS_InstanceMemory, RawData.GetAllocatedSize() + Items.GetAllocatedSize());

		// ItemValue's size must match what is expected by class doing memory write
		check(GetDefault<TypeItem>()->GetValueSize() == sizeof(typename TypeItem::FValueType));
		// writer must fit into block allocated for single item (not 'equal' on purpose, check UEnvQueryGenerator_Composite.bAllowDifferentItemTypes)
		check(GetDefault<TypeItem>()->GetValueSize() <= ValueSize);

		const int32 DataOffset = RawData.AddZeroed(ValueSize);
		TypeItem::SetValue(RawData.GetData() + DataOffset, ItemValue);
		Items.Add(FEnvQueryItem(DataOffset));

		INC_MEMORY_STAT_BY(STAT_AI_EQS_InstanceMemory, RawData.GetAllocatedSize() + Items.GetAllocatedSize());
	}

	/** AddItemData specialization for arrays if values */
	template<typename TypeItem>
	void AddItemData(TArray<typename TypeItem::FValueType>& ItemCollection)
	{
		if (ItemCollection.Num() > 0)
		{
			DEC_MEMORY_STAT_BY(STAT_AI_EQS_InstanceMemory, RawData.GetAllocatedSize() + Items.GetAllocatedSize());

			// ItemValue's size must match what is expected by class doing memory write
			check(GetDefault<TypeItem>()->GetValueSize() == sizeof(typename TypeItem::FValueType));
			// writer must fit into block allocated for single item (not 'equal' on purpose, check UEnvQueryGenerator_Composite.bAllowDifferentItemTypes)
			check(GetDefault<TypeItem>()->GetValueSize() <= ValueSize);

			int32 DataOffset = RawData.AddZeroed(ValueSize * ItemCollection.Num());
			Items.Reserve(Items.Num() + ItemCollection.Num());

			for (typename TypeItem::FValueType& Item : ItemCollection)
			{
				TypeItem::SetValue(RawData.GetData() + DataOffset, Item);
				Items.Add(FEnvQueryItem(DataOffset));
				DataOffset += ValueSize;
			}

			INC_MEMORY_STAT_BY(STAT_AI_EQS_InstanceMemory, RawData.GetAllocatedSize() + Items.GetAllocatedSize());
		}
	}

	template<typename TypeItem, typename TypeValue>
	void AddItemData(TypeValue ItemValue)
	{
		AddItemData<TypeItem>((typename TypeItem::FValueType)(ItemValue));
	}

	template<typename TypeItem, typename TypeValue>
	void AddItemData(TArray<TypeValue>& ItemCollection)
	{
		for (auto& Item : ItemCollection)
		{
			AddItemData<TypeItem>((typename TypeItem::FValueType)(Item));
		}
	}

protected:

	/** prepare item data after generator has finished */
	AIMODULE_API void FinalizeGeneration();

	/** update costs and flags after test has finished */
	AIMODULE_API void FinalizeTest();
	
	/** final pass on items of finished query */
	AIMODULE_API void FinalizeQuery();

	/** normalize total score in range 0..1 */
	AIMODULE_API void NormalizeScores();

	/** sort all scores, from highest to lowest */
	AIMODULE_API void SortScores();

	/** pick one of items with score equal or higher than specified */
	AIMODULE_API void PickRandomItemOfScoreAtLeast(float MinScore);

	/** discard all items but one */
	AIMODULE_API void PickSingleItem(int32 ItemIndex);

public:

	/** removes all runtime data that can be used for debugging (not a part of actual query result) */
	AIMODULE_API void StripRedundantData();

#if STATS
	inline void IncStats()
	{
		INC_MEMORY_STAT_BY(STAT_AI_EQS_InstanceMemory, GetAllocatedSize());
		INC_DWORD_STAT_BY(STAT_AI_EQS_NumItems, Items.Num());
	}

	inline void DecStats()
	{
		DEC_MEMORY_STAT_BY(STAT_AI_EQS_InstanceMemory, GetAllocatedSize()); 
		DEC_DWORD_STAT_BY(STAT_AI_EQS_NumItems, Items.Num());
	}

	AIMODULE_API uint32 GetAllocatedSize() const;
	AIMODULE_API uint32 GetContextAllocatedSize() const;
#else
	inline uint32 GetAllocatedSize() const { return 0; }
	inline uint32 GetContextAllocatedSize() const { return 0; }
	inline void IncStats() {}
	inline void DecStats() {}
#endif // STATS

#if !NO_LOGGING
	AIMODULE_API void Log(const FString Msg) const;
#endif // #if !NO_LOGGING

#if USE_EQS_DEBUGGER
#	define  UE_EQS_DBGMSG(Condition, Format, ...) \
					if (Condition) \
					{ \
						Instance.ItemDetails[CurrentItem].FailedDescription = FString::Printf(Format, ##__VA_ARGS__); \
					}

#	define UE_EQS_LOG(CategoryName, Verbosity, Format, ...) \
					UE_LOG(CategoryName, Verbosity, Format, ##__VA_ARGS__); \
					UE_EQS_DBGMSG(true, Format, ##__VA_ARGS__); 
#else
#	define UE_EQS_DBGMSG(Condition, Format, ...)
#	define UE_EQS_LOG(CategoryName, Verbosity, Format, ...) UE_LOG(CategoryName, Verbosity, Format, ##__VA_ARGS__); 
#endif

	/** describe for logging purposes what the query spent time on */
	AIMODULE_API FString GetExecutionTimeDescription() const;

#if CPP || UE_BUILD_DOCS
	/** Note that this iterator is for read-only purposes. Please use FItemIterator for regular item iteration 
	 *	while performing EQS testing and scoring */
	struct FConstItemIterator
	{
		FConstItemIterator(FEnvQueryInstance& QueryInstance, int32 StartingItemIndex = INDEX_NONE)
			: Instance(QueryInstance)
			, CurrentItem(StartingItemIndex != INDEX_NONE ? StartingItemIndex : QueryInstance.CurrentTestStartingItem)
		{
			if (StartingItemIndex != INDEX_NONE)
			{
				CurrentItem = StartingItemIndex;
			}
			else
			{
				CurrentItem = QueryInstance.CurrentTestStartingItem;
				if (Instance.Items.IsValidIndex(CurrentItem) == false || Instance.Items[CurrentItem].IsValid() == false)
				{
					++(*this);
				}
			}
		}

		uint8* GetItemData()
		{
			return Instance.RawData.GetData() + Instance.Items[CurrentItem].DataOffset;
		}

		int32 GetIndex() const
		{
			return CurrentItem;
		}

		inline explicit operator bool() const
		{
			return CurrentItem < Instance.Items.Num();
		}

		void operator++()
		{
			++CurrentItem;
			for (; CurrentItem < Instance.Items.Num() && !Instance.Items[CurrentItem].IsValid(); ++CurrentItem)
				;
		}

	protected:

		FEnvQueryInstance& Instance;
		int32 CurrentItem;
	};

	struct FItemIterator : public FConstItemIterator
	{
		AIMODULE_API FItemIterator(const UEnvQueryTest* QueryTest, FEnvQueryInstance& QueryInstance, int32 StartingItemIndex = INDEX_NONE);

		~FItemIterator()
		{
			Instance.CurrentTestStartingItem = CurrentItem;
		}

		/** Filter and score an item - used by tests working on float values
		 *  (can be called multiple times for single item when processing contexts with multiple entries)
		 *  NOTE: The Score is the raw score, before clamping, normalizing, and multiplying by weight.  The FilterMin
		 *  and FilterMax values are ONLY used for filtering (if any).
		 */
		void SetScore(EEnvTestPurpose::Type TestPurpose, EEnvTestFilterType::Type FilterType, float Score, float FilterMin, float FilterMax)
		{
			if (bForced)
			{
				return;
			}

			bool bPassedTest = true;

			if (TestPurpose != EEnvTestPurpose::Score)	// May need to filter results!
			{
				switch (FilterType)
				{
					case EEnvTestFilterType::Maximum:
						bPassedTest = (Score <= FilterMax);
						UE_EQS_DBGMSG(!bPassedTest, TEXT("Value %f is above maximum value set to %f"), Score, FilterMax);
						break;

					case EEnvTestFilterType::Minimum:
						bPassedTest = (Score >= FilterMin);
						UE_EQS_DBGMSG(!bPassedTest, TEXT("Value %f is below minimum value set to %f"), Score, FilterMin);
						break;

					case EEnvTestFilterType::Range:
						bPassedTest = (Score >= FilterMin) && (Score <= FilterMax);
						UE_EQS_DBGMSG(!bPassedTest, TEXT("Value %f is out of range set to (%f, %f)"), Score, FilterMin, FilterMax);
						break;

					case EEnvTestFilterType::Match:
						UE_EQS_LOG(LogEQS, Error, TEXT("Filtering Type set to 'Match' for floating point test.  Will consider test as failed in all cases."));
						bPassedTest = false;
						break;

					default:
						UE_EQS_LOG(LogEQS, Error, TEXT("Filtering Type set to invalid value for floating point test.  Will consider test as failed in all cases."));
						bPassedTest = false;
						break;
				}
			}

			if (bPassedTest)
			{
				SetScoreInternal(Score);
				NumPassedForItem++;
			}

			NumTestsForItem++;
		}

		/** Filter and score an item - used by tests working on bool values
		 *  (can be called multiple times for single item when processing contexts with multiple entries)
		 */
		void SetScore(EEnvTestPurpose::Type TestPurpose, EEnvTestFilterType::Type FilterType, bool bScore, bool bExpected)
		{
			if (bForced)
			{
				return;
			}

			bool bPassedTest = true;
			switch (FilterType)
			{
				case EEnvTestFilterType::Match:
					bPassedTest = (bScore == bExpected);
					UE_EQS_DBGMSG(!bPassedTest, TEXT("Boolean score doesn't match (expected %s and got %s)"), bExpected ? TEXT("TRUE") : TEXT("FALSE"), bScore ? TEXT("TRUE") : TEXT("FALSE"));
					break;

				case EEnvTestFilterType::Maximum:
					UE_EQS_LOG(LogEQS, Error, TEXT("Filtering Type set to 'Maximum' for boolean test.  Will consider test as failed in all cases."));
					bPassedTest = false;
					break;

				case EEnvTestFilterType::Minimum:
					UE_EQS_LOG(LogEQS, Error, TEXT("Filtering Type set to 'Minimum' for boolean test.  Will consider test as failed in all cases."));
					bPassedTest = false;
					break;

				case EEnvTestFilterType::Range:
					UE_EQS_LOG(LogEQS, Error, TEXT("Filtering Type set to 'Range' for boolean test.  Will consider test as failed in all cases."));
					bPassedTest = false;
					break;

				default:
					UE_EQS_LOG(LogEQS, Error, TEXT("Filtering Type set to invalid value for boolean test.  Will consider test as failed in all cases."));
					bPassedTest = false;
					break;
			}

			if (bPassedTest || !bIsFiltering)
			{
				// even if the item's result is different than expected
				// but we're not filtering those items out, we still want
				// to treat this as successful test, just with different score
				SetScoreInternal(bPassedTest ? 1.0f : 0.f);
				NumPassedForItem++;
			}

			NumTestsForItem++;
		}

		/** Force state and score of item
		 *  Any following SetScore calls for current item will be ignored
		 */
		void ForceItemState(const EEnvItemStatus::Type InStatus, const float Score = UEnvQueryTypes::SkippedItemValue)
		{
			bForced = true;
			bPassed = (InStatus == EEnvItemStatus::Passed);
			ItemScore = Score;
		}

		/** Disables time slicing for this iterator, use with caution! */
		FItemIterator& IgnoreTimeLimit()
		{
			Deadline = -1.0f;
			return *this;
		}

		inline explicit operator bool() const
		{
			return CurrentItem < Instance.Items.Num() && !Instance.bFoundSingleResult && (Deadline < 0 || FPlatformTime::Seconds() < Deadline);
		}

		void operator++()
		{
			StoreTestResult();
			if (!Instance.bFoundSingleResult)
			{
				InitItemScore();
				FConstItemIterator::operator++();
			}
		}

	protected:

		double Deadline;
		float ItemScore;
		int16 NumPassedForItem;
		int16 NumTestsForItem;
		uint8 CachedFilterOp;
		uint8 CachedScoreOp;
		uint8 bPassed : 1;
		uint8 bForced : 1;
		uint8 bIsFiltering : 1;

		void InitItemScore()
		{
			NumPassedForItem = 0;
			NumTestsForItem = 0;
			ItemScore = 0.0f;
			bPassed = true;
			bForced = false;
		}

		AIMODULE_API void HandleFailedTestResult();
		AIMODULE_API void StoreTestResult();

		inline void SetScoreInternal(float Score)
		{
			switch (CachedScoreOp)
			{
			case EEnvTestScoreOperator::AverageScore:
				ItemScore += Score;
				break;

			case EEnvTestScoreOperator::MinScore:
				if (!NumPassedForItem || ItemScore > Score)
				{
					ItemScore = Score;
				}
				break;

			case EEnvTestScoreOperator::MaxScore:
				if (!NumPassedForItem || ItemScore < Score)
				{
					ItemScore = Score;
				}
				break;
			case EEnvTestScoreOperator::Multiply:
				// ItemScore defaults to 0, so for first test we need to initialize the score, otherwise we end up constantly multiplying by 0
				ItemScore = (NumTestsForItem == 0) ? Score : (ItemScore * Score);
				break;
			}
		}

		inline void CheckItemPassed()
		{
			if (!bForced)
			{
				if (NumTestsForItem == 0)
				{
					bPassed = false;
				}
				else if (!bIsFiltering)
				{
					bPassed = true;
				}
				else if (CachedFilterOp == EEnvTestFilterOperator::AllPass)
				{
					bPassed = bPassed && (NumPassedForItem == NumTestsForItem);
				}
				else
				{
					bPassed = bPassed && (NumPassedForItem > 0);
				}
			}
		}
	};
	typedef FItemIterator ItemIterator;
#endif

#undef UE_EQS_LOG
#undef UE_EQS_DBGMSG

#if USE_EQS_DEBUGGER
	FEnvQueryDebugData DebugData;
	static AIMODULE_API bool bDebuggingInfoEnabled;
#endif // USE_EQS_DEBUGGER

#if STATS
	TStatId StatId;
#endif

	AIMODULE_API FBox GetBoundingBox() const;
};

namespace FEQSHelpers
{
	AIMODULE_API const ANavigationData* FindNavigationDataForQuery(FEnvQueryInstance& QueryInstance, TSubclassOf<UEnvQueryContext> NavDataOverrideContext = nullptr);
}

USTRUCT(BlueprintType)
struct FAIDynamicParam
{
	GENERATED_USTRUCT_BODY();

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = EQS)
	FName ParamName;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = EQS)
	EAIParamType ParamType;

	UPROPERTY(BlueprintReadWrite, Category = EQS)
	uint8 bAllowBBKey : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = EQS)
	float Value;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = EQS, meta=(EditCondition="bAllowBBKey", EditConditionHides))
	FBlackboardKeySelector BBKey;


	FAIDynamicParam()
	{
		ParamType = EAIParamType::Float;
		Value = 0.f;
		BBKey.AllowNoneAsValue(true);
		bAllowBBKey = true;
	}

	AIMODULE_API void ConfigureBBKey(UObject &QueryOwner);

	static AIMODULE_API void GenerateConfigurableParamsFromNamedValues(UObject &QueryOwner, TArray<FAIDynamicParam>& OutQueryConfig, TArray<FEnvNamedValue>& InQueryParams);
};

USTRUCT()
struct FEQSParametrizedQueryExecutionRequest
{
	GENERATED_USTRUCT_BODY()

	AIMODULE_API FEQSParametrizedQueryExecutionRequest();

	UPROPERTY(Category = Node, EditAnywhere, meta = (EditCondition = "!bUseBBKeyForQueryTemplate"))
	TObjectPtr<UEnvQuery> QueryTemplate;

	UPROPERTY(Category = Node, EditAnywhere)
	TArray<FAIDynamicParam> QueryConfig;
	
	/** blackboard key storing an EQS query template */
	UPROPERTY(EditAnywhere, Category = Blackboard, meta = (EditCondition = "bUseBBKeyForQueryTemplate"))
	FBlackboardKeySelector EQSQueryBlackboardKey;

	/** determines which item will be stored (All = only first matching) */
	UPROPERTY(Category = Node, EditAnywhere)
	TEnumAsByte<EEnvQueryRunMode::Type> RunMode;

	UPROPERTY(Category = Node, EditAnywhere, meta=(InlineEditConditionToggle))
	uint32 bUseBBKeyForQueryTemplate : 1;

	uint32 bInitialized : 1;

	AIMODULE_API void InitForOwnerAndBlackboard(UObject& Owner, UBlackboardData* BBAsset);

	bool IsValid() const { return bInitialized; }

	AIMODULE_API int32 Execute(UObject& QueryOwner, const UBlackboardComponent* BlackboardComponent, FQueryFinishedSignature& QueryFinishedDelegate);

#if WITH_EDITOR
	AIMODULE_API void PostEditChangeProperty(UObject& Owner, struct FPropertyChangedEvent& PropertyChangedEvent);
#endif // WITH_EDITOR
};
