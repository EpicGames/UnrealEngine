// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "GameFramework/Actor.h"
#include "EnvironmentQuery/EnvQueryContext.h"
#include "DataProviders/AIDataProvider.h"
#include "EnvironmentQuery/EnvQueryGenerator.h"
#include "EnvQueryGenerator_ActorsOfClass.generated.h"

UCLASS(meta = (DisplayName = "Actors Of Class"), MinimalAPI)
class UEnvQueryGenerator_ActorsOfClass : public UEnvQueryGenerator
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditDefaultsOnly, Category=Generator, meta=(AllowAbstract))
	TSubclassOf<AActor> SearchedActorClass;

	/** If true, this will only returns actors of the specified class within the SearchRadius of the SearchCenter context.  If false, it will return ALL actors of the specified class in the world. */
	UPROPERTY(EditDefaultsOnly, Category=Generator)
	FAIDataProviderBoolValue GenerateOnlyActorsInRadius;

	/** Max distance of path between point and context.  NOTE: Zero and negative values will never return any results if
	  * UseRadius is true.  "Within" requires Distance < Radius.  Actors ON the circle (Distance == Radius) are excluded.
	  */
	UPROPERTY(EditDefaultsOnly, Category=Generator)
	FAIDataProviderFloatValue SearchRadius;

	/** context */
	UPROPERTY(EditAnywhere, Category=Generator)
	TSubclassOf<UEnvQueryContext> SearchCenter;

	AIMODULE_API virtual void GenerateItems(FEnvQueryInstance& QueryInstance) const override;
	virtual void ProcessItems(FEnvQueryInstance& QueryInstance, TArray<AActor*>& MatchingActors) const {}

	AIMODULE_API virtual FText GetDescriptionTitle() const override;
	AIMODULE_API virtual FText GetDescriptionDetails() const override;
};
