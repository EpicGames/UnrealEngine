// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "UObject/ScriptMacros.h"
#include "EnvironmentQuery/Items/EnvQueryItemType.h"
#include "EnvironmentQuery/EnvQueryContext.h"
#include "EnvironmentQuery/EnvQueryGenerator.h"
#include "EnvQueryGenerator_BlueprintBase.generated.h"

class AActor;

UCLASS(Abstract, Blueprintable, MinimalAPI)
class UEnvQueryGenerator_BlueprintBase : public UEnvQueryGenerator
{
	GENERATED_UCLASS_BODY()

	/** A short description of what test does, like "Generate pawn named Joe" */
	UPROPERTY(EditAnywhere, Category = Generator)
	FText GeneratorsActionDescription;

	/** context */
	UPROPERTY(EditAnywhere, Category = Generator)
	TSubclassOf<UEnvQueryContext> Context;

	/** @todo this should show up only in the generator's BP, but 
	 *	due to the way EQS editor is generating widgets it's there as well
	 *	It's a bug and we'll fix it */
	UPROPERTY(EditDefaultsOnly, Category = Generator)
	TSubclassOf<UEnvQueryItemType> GeneratedItemType;

	AIMODULE_API virtual void PostInitProperties() override;
	AIMODULE_API virtual UWorld* GetWorld() const override;

	UFUNCTION(BlueprintImplementableEvent, Category = Generator)
	AIMODULE_API void DoItemGeneration(const TArray<FVector>& ContextLocations) const;

	UFUNCTION(BlueprintImplementableEvent, Category = Generator)
	AIMODULE_API void DoItemGenerationFromActors(const TArray<AActor*>& ContextActors) const;

	AIMODULE_API virtual void GenerateItems(FEnvQueryInstance& QueryInstance) const override;

	AIMODULE_API virtual FText GetDescriptionTitle() const override;
	AIMODULE_API virtual FText GetDescriptionDetails() const override;
	
	UFUNCTION(BlueprintCallable, Category = "EQS")
	AIMODULE_API void AddGeneratedVector(FVector GeneratedVector) const;

	UFUNCTION(BlueprintCallable, Category = "EQS")
	AIMODULE_API void AddGeneratedActor(AActor* GeneratedActor) const;

	UFUNCTION(BlueprintCallable, Category = "EQS")
	AIMODULE_API UObject* GetQuerier() const;

private:
	enum class ECallMode
	{
		Invalid,
		FromVectors,
		FromActors
	};

	/** this is valid and set only within GenerateItems call */
	mutable FEnvQueryInstance* CachedQueryInstance;

	ECallMode CallMode = ECallMode::Invalid;
};
