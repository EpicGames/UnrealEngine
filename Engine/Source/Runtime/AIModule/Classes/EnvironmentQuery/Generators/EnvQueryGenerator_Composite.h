// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EnvironmentQuery/EnvQueryGenerator.h"
#include "EnvQueryGenerator_Composite.generated.h"

/**
 * Composite generator allows using multiple generators in single query option
 * All child generators must produce exactly the same item type!
 */

UCLASS(meta = (DisplayName = "Composite"), MinimalAPI)
class UEnvQueryGenerator_Composite : public UEnvQueryGenerator
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditDefaultsOnly, Instanced, Category = Generator)
	TArray<TObjectPtr<UEnvQueryGenerator>> Generators;

	/** allow generators with different item types, use at own risk!
	 *
	 *  WARNING: 
	 *  generator will use ForcedItemType for raw data, you MUST ensure proper memory layout
	 *  child generators will be writing to memory block through their own item types:
	 *  - data must fit info block allocated by ForcedItemType
	 *  - tests will read item location/properties through ForcedItemType
	 */
	UPROPERTY(EditDefaultsOnly, Category = Generator, AdvancedDisplay)
	uint32 bAllowDifferentItemTypes : 1;

	UPROPERTY()
	uint32 bHasMatchingItemType : 1;

	UPROPERTY(EditDefaultsOnly, Category = Generator, AdvancedDisplay)
	TSubclassOf<UEnvQueryItemType> ForcedItemType;

	AIMODULE_API void VerifyItemTypes();

protected:
	AIMODULE_API virtual void GenerateItems(FEnvQueryInstance& QueryInstance) const override;
	AIMODULE_API virtual FText GetDescriptionTitle() const override;
	AIMODULE_API virtual bool IsValidGenerator() const override;
};
