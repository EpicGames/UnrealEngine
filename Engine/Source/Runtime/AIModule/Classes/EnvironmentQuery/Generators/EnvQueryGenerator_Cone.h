// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Templates/SubclassOf.h"
#include "DataProviders/AIDataProvider.h"
#include "EnvironmentQuery/EnvQueryContext.h"
#include "EnvironmentQuery/Generators/EnvQueryGenerator_ProjectedPoints.h"
#include "EnvQueryGenerator_Cone.generated.h"

UCLASS(meta = (DisplayName = "Points: Cone"), MinimalAPI)
class UEnvQueryGenerator_Cone : public UEnvQueryGenerator_ProjectedPoints
{
	GENERATED_BODY()
public:
	AIMODULE_API UEnvQueryGenerator_Cone(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** Binds data to data providers 
	 *  @param QueryInstance - the instance of the query
	 */
	AIMODULE_API void BindDataToDataProviders(FEnvQueryInstance& QueryInstance) const;
	
	AIMODULE_API virtual void GenerateItems(FEnvQueryInstance& QueryInstance) const override;

	/** Returns the title of the generator on the corresponding node in the EQS Editor window */
	AIMODULE_API virtual FText GetDescriptionTitle() const override;

	/** Returns the details of the generator on the corresponding node in the EQS Editor window */
	AIMODULE_API virtual FText GetDescriptionDetails() const override;

protected:
	/** Distance between each point of the same angle */
	UPROPERTY(EditDefaultsOnly, Category = Generator)
	FAIDataProviderFloatValue AlignedPointsDistance;

	/** Maximum degrees of the generated cone */
	UPROPERTY(EditDefaultsOnly, Category = Generator)
	FAIDataProviderFloatValue ConeDegrees;

	/** The step of the angle increase. Angle step must be >=1
	 *  Smaller values generate less items
	 */
	UPROPERTY(EditDefaultsOnly, Category = Generator)
	FAIDataProviderFloatValue AngleStep;

	/** Generation distance */
	UPROPERTY(EditDefaultsOnly, Category = Generator)
	FAIDataProviderFloatValue Range;

	/** The actor (or actors) that will generate a cone in their facing direction */
	UPROPERTY(EditAnywhere, Category = Generator)
	TSubclassOf<UEnvQueryContext> CenterActor;

	/** Whether to include CenterActors' locations when generating items. 
	 *	Note that this option skips the MinAngledPointsDistance parameter. */
	UPROPERTY(EditAnywhere, Category = Generator)
	uint8 bIncludeContextLocation : 1;
};
