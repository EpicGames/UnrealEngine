// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
	#include "CoreMinimal.h"
#endif
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "EnvironmentQuery/EnvQueryContext.h"
#include "EnvironmentQuery/Generators/EnvQueryGenerator_ProjectedPoints.h"
#include "EnvQueryGenerator_CurrentLocation.generated.h"

UCLASS(meta = (DisplayName = "Current Location"), MinimalAPI)
class UEnvQueryGenerator_CurrentLocation : public UEnvQueryGenerator_ProjectedPoints
{
	GENERATED_BODY()

protected:
	/** context */
	UPROPERTY(EditDefaultsOnly, Category = Generator)
	TSubclassOf<UEnvQueryContext> QueryContext;

public:

	AIMODULE_API UEnvQueryGenerator_CurrentLocation(const FObjectInitializer& ObjectInitializer);
	
	AIMODULE_API virtual void GenerateItems(FEnvQueryInstance& QueryInstance) const override;

	AIMODULE_API virtual FText GetDescriptionTitle() const override;
	AIMODULE_API virtual FText GetDescriptionDetails() const override;
};
