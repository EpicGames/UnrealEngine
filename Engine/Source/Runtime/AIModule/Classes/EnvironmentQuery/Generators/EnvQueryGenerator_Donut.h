// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "DataProviders/AIDataProvider.h"
#include "EnvironmentQuery/Generators/EnvQueryGenerator_ProjectedPoints.h"
#include "EnvQueryGenerator_Donut.generated.h"

UCLASS(meta = (DisplayName = "Points: Donut"), MinimalAPI)
class UEnvQueryGenerator_Donut : public UEnvQueryGenerator_ProjectedPoints
{
	GENERATED_UCLASS_BODY()

	/** min distance between point and context */
	UPROPERTY(EditDefaultsOnly, Category = Generator)
	FAIDataProviderFloatValue InnerRadius;

	/** max distance between point and context */
	UPROPERTY(EditDefaultsOnly, Category = Generator)
	FAIDataProviderFloatValue OuterRadius;

	/** number of rings to generate */
	UPROPERTY(EditDefaultsOnly, Category = Generator)
	FAIDataProviderIntValue NumberOfRings;

	/** how we are choosing where the points are in the ring */
	UPROPERTY(EditDefaultsOnly, Category = Generator)
	EEnvQueryPointSpacingMethod PointOnRingSpacingMethod;

	/** items will be generated on a ring this much apart */
	UPROPERTY(EditDefaultsOnly, Category = Generator, meta = (EditCondition = "PointOnRingSpacingMethod == EEnvQueryPointSpacingMethod::BySpaceBetween", EditConditionHides))
	FAIDataProviderFloatValue SpaceBetweenPoints;

	/** number of items to generate for each ring */
	UPROPERTY(EditDefaultsOnly, Category = Generator, meta = (EditCondition = "PointOnRingSpacingMethod == EEnvQueryPointSpacingMethod::ByNumberOfPoints", EditConditionHides))
	FAIDataProviderIntValue PointsPerRing;

	/** If you generate items on a piece of circle you define direction of Arc cut here */
	UPROPERTY(EditDefaultsOnly, Category = Generator, meta = (EditCondition = "bDefineArc"))
	FEnvDirection ArcDirection;

	/** If you generate items on a piece of circle you define angle of Arc cut here */
	UPROPERTY(EditDefaultsOnly, Category = Generator)
	FAIDataProviderFloatValue ArcAngle;

	/** If true, the rings of the wheel will be rotated in a spiral pattern.  If false, they will all be at a zero
	  * rotation, looking more like the spokes on a wheel.  */
	UPROPERTY(EditDefaultsOnly, Category = Generator)
	bool bUseSpiralPattern;

	/** context */
	UPROPERTY(EditAnywhere, Category = Generator)
	TSubclassOf<class UEnvQueryContext> Center;

	UPROPERTY(EditAnywhere, Category = Generator, meta=(InlineEditConditionToggle))
	uint32 bDefineArc : 1;

	AIMODULE_API virtual void GenerateItems(FEnvQueryInstance& QueryInstance) const override;

	AIMODULE_API virtual FText GetDescriptionTitle() const override;
	AIMODULE_API virtual FText GetDescriptionDetails() const override;

#if WITH_EDITOR
	AIMODULE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

protected:
	AIMODULE_API FVector::FReal GetArcBisectorAngle(FEnvQueryInstance& QueryInstance) const;
	AIMODULE_API bool IsAngleAllowed(FVector::FReal TestAngleRad, FVector::FReal BisectAngleDeg, FVector::FReal AngleRangeDeg, bool bConstrainAngle) const;
	AIMODULE_API bool GenerateDonutNavPoints(const TArray<FVector>& CenterPoints, TArray<FNavLocation>& Points, FEnvQueryInstance& QueryInstance) const;
};
