// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "DataProviders/AIDataProvider.h"
#include "EnvironmentQuery/Generators/EnvQueryGenerator_ProjectedPoints.h"
#include "EnvQueryGenerator_OnCircle.generated.h"

class AActor;

enum class UE_DEPRECATED(5.8, "Use EEnvQueryPointSpacingMethod instead") EPointOnCircleSpacingMethod :uint8
{
	// Use the SpaceBetween value to determine how far apart points should be.
	BySpaceBetween,
	// Use a fixed number of points
	ByNumberOfPoints
};

UCLASS(meta = (DisplayName = "Points: Circle"), MinimalAPI)
class UEnvQueryGenerator_OnCircle : public UEnvQueryGenerator_ProjectedPoints
{
	GENERATED_UCLASS_BODY()

	/** max distance of path between point and context */
	UPROPERTY(EditDefaultsOnly, Category=Generator)
	FAIDataProviderFloatValue CircleRadius;

	/** how we are choosing where the points are in the circle */
	UPROPERTY(EditDefaultsOnly, Category = Generator)
	EEnvQueryPointSpacingMethod PointOnCircleSpacingMethod;

	/** items will be generated on a circle this much apart */
	UPROPERTY(EditDefaultsOnly, Category = Generator, meta = (EditCondition = "PointOnCircleSpacingMethod == EEnvQueryPointSpacingMethod::BySpaceBetween", EditConditionHides))
	FAIDataProviderFloatValue SpaceBetween;

	/** this many items will be generated on a circle */
	UPROPERTY(EditDefaultsOnly, Category = Generator, meta = (EditCondition = "PointOnCircleSpacingMethod == EEnvQueryPointSpacingMethod::ByNumberOfPoints", EditConditionHides))
	FAIDataProviderIntValue NumberOfPoints;

	/** If you generate items on a piece of circle you define direction of Arc cut here */
	UPROPERTY(EditDefaultsOnly, Category=Generator, meta=(EditCondition="bDefineArc"))
	FEnvDirection ArcDirection;

	/** Offsets the ArcDirection and starts generating points this many degrees clock-wise around the vertical axis */
	UPROPERTY(EditDefaultsOnly, Category=Generator)
	FAIDataProviderFloatValue ArcDirectionOffsetDegrees;

	/** If you generate items on a piece of circle you define angle of Arc cut here */
	UPROPERTY(EditDefaultsOnly, Category=Generator)
	FAIDataProviderFloatValue ArcAngle;

	/** context */
	UPROPERTY(EditAnywhere, Category=Generator)
	TSubclassOf<class UEnvQueryContext> CircleCenter;

	/** ignore tracing into context actors when generating the circle */
	UPROPERTY(EditDefaultsOnly, Category = Generator)
	bool bIgnoreAnyContextActorsWhenGeneratingCircle;

	/** context offset */
	UPROPERTY(EditAnywhere, Category = Generator)
	FAIDataProviderFloatValue CircleCenterZOffset;

	/** horizontal trace for nearest obstacle */
	UPROPERTY(EditAnywhere, Category=Generator)
	FEnvTraceData TraceData;

	UPROPERTY(EditAnywhere, Category=Generator, meta=(InlineEditConditionToggle))
	uint32 bDefineArc:1;

	AIMODULE_API virtual void PostLoad() override;

	AIMODULE_API virtual void GenerateItems(FEnvQueryInstance& QueryInstance) const override;

	AIMODULE_API virtual FText GetDescriptionTitle() const override;
	AIMODULE_API virtual FText GetDescriptionDetails() const override;

#if WITH_EDITOR
	AIMODULE_API virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

protected:
	AIMODULE_API FVector CalcDirection(FEnvQueryInstance& QueryInstance) const;

	AIMODULE_API void GenerateItemsForCircle(uint8* ContextRawData, UEnvQueryItemType* ContextItemType,
		const FVector& CenterLocation, const FVector& StartDirection,
		const TArray<AActor*>& IgnoredActors,
		int32 StepsCount, float AngleStep, FEnvQueryInstance& OutQueryInstance) const;

	AIMODULE_API virtual void AddItemDataForCircle(uint8* ContextRawData, UEnvQueryItemType* ContextItemType, 
		const TArray<FNavLocation>& Locations, FEnvQueryInstance& OutQueryInstance) const;
};
