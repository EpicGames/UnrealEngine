// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "AI/Navigation/NavigationTypes.h"
#include "DataProviders/AIDataProvider.h"
#include "EnvironmentQuery/Generators/EnvQueryGenerator_SimpleGrid.h"
#include "EnvQueryGenerator_PathingGrid.generated.h"

/**
 *  Navigation grid, generates points on navmesh
 *  with paths to/from context no further than given limit
 */

UCLASS(meta = (DisplayName = "Points: Pathing Grid"), MinimalAPI)
class UEnvQueryGenerator_PathingGrid : public UEnvQueryGenerator_SimpleGrid
{
	GENERATED_UCLASS_BODY()

	/** pathfinding direction */
	UPROPERTY(EditDefaultsOnly, Category = Generator)
	FAIDataProviderBoolValue PathToItem;

	/** navigation filter to use in pathfinding */
	UPROPERTY(EditDefaultsOnly, Category = Generator)
	TSubclassOf<UNavigationQueryFilter> NavigationFilter;

	/** multiplier for max distance between point and context */
	UPROPERTY(EditDefaultsOnly, Category = Pathfinding, AdvancedDisplay)
	FAIDataProviderFloatValue ScanRangeMultiplier;

	AIMODULE_API virtual void ProjectAndFilterNavPoints(TArray<FNavLocation>& Points, FEnvQueryInstance& QueryInstance) const override;
};
