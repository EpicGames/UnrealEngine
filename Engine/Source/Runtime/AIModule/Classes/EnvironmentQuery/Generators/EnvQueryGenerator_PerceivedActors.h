// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataProviders/AIDataProvider.h"
#include "EnvironmentQuery/EnvQueryGenerator.h"
#include "Perception/AISense.h"
#include "EnvQueryGenerator_PerceivedActors.generated.h"


/** Gathers actors perceived by context */
UCLASS(meta = (DisplayName = "Perceived Actors"), MinimalAPI)
class UEnvQueryGenerator_PerceivedActors : public UEnvQueryGenerator
{
	GENERATED_UCLASS_BODY()

protected:
	/** If set will be used to filter results */
	UPROPERTY(EditDefaultsOnly, Category=Generator)
	TSubclassOf<AActor> AllowedActorClass;

	/** Additional distance limit imposed on the items generated. Perception's range limit still applies. */
	UPROPERTY(EditDefaultsOnly, Category=Generator)
	FAIDataProviderFloatValue SearchRadius;

	/** The perception listener to use as a source of information */
	UPROPERTY(EditAnywhere, Category=Generator)
	TSubclassOf<UEnvQueryContext> ListenerContext;

	/** If set will be used to filter gathered results so that only actors perceived with a given sense are considered */
	UPROPERTY(EditAnywhere, Category=Generator)
	TSubclassOf<UAISense> SenseToUse;

	/**
	 * Indicates whether to include all actors known via perception (TRUE) or just the ones actively being perceived 
	 * at the moment (example "currently visible" as opposed to "seen and the perception stimulus haven't expired yet").
	 * @see FAIStimulus.bExpired
	 */
	UPROPERTY(EditAnywhere, Category=Generator)
	bool bIncludeKnownActors = true;

	AIMODULE_API virtual void GenerateItems(FEnvQueryInstance& QueryInstance) const override;

	AIMODULE_API virtual FText GetDescriptionTitle() const override;
	AIMODULE_API virtual FText GetDescriptionDetails() const override;
};
