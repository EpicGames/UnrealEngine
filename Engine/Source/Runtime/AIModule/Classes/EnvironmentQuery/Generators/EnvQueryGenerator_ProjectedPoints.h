// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "UObject/ObjectMacros.h"
#include "AI/Navigation/NavigationTypes.h"
#include "EnvironmentQuery/EnvQueryGenerator.h"

#include "EnvQueryGenerator_ProjectedPoints.generated.h"

class EnvQueryContext;

UENUM()
enum class EEnvQueryPointSpacingMethod :uint8
{
	// Use the SpaceBetween value to determine how far apart points should be.
	BySpaceBetween,
	// Use a fixed number of points
	ByNumberOfPoints
};

UCLASS(Abstract, MinimalAPI)
class UEnvQueryGenerator_ProjectedPoints : public UEnvQueryGenerator
{
	GENERATED_UCLASS_BODY()

	/** trace params */
	UPROPERTY(EditDefaultsOnly, Category = Generator)
	FEnvTraceData ProjectionData;

	struct FSortByHeight
	{
		FVector::FReal OriginalZ;

		FSortByHeight(const FVector& OriginalPt) : OriginalZ(OriginalPt.Z) {}

		inline bool operator()(const FNavLocation& A, const FNavLocation& B) const
		{
			return FMath::Abs(A.Location.Z - OriginalZ) < FMath::Abs(B.Location.Z - OriginalZ);
		}
	};

	/** project all points in array and remove those outside navmesh */
	AIMODULE_API virtual void ProjectAndFilterNavPoints(TArray<FNavLocation>& Points, FEnvQueryInstance& QueryInstance) const;

	/** store points as generator's result */
	AIMODULE_API virtual void StoreNavPoints(const TArray<FNavLocation>& Points, FEnvQueryInstance& QueryInstance) const;

	AIMODULE_API virtual void PostLoad() override;

	/** Navigation Data Override */
	UPROPERTY(EditDefaultsOnly, Category = "Generator")
	TSubclassOf<UEnvQueryContext> NavDataOverrideContext;
};
