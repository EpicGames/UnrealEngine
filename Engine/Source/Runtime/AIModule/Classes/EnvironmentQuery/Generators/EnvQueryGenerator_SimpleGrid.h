// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "DataProviders/AIDataProvider.h"
#include "EnvironmentQuery/EnvQueryContext.h"
#include "EnvironmentQuery/Generators/EnvQueryGenerator_ProjectedPoints.h"
#include "EnvQueryGenerator_SimpleGrid.generated.h"

/**
 *  Simple grid, generates points in 2D square around context
 */

UCLASS(meta = (DisplayName = "Points: Grid"), MinimalAPI)
class UEnvQueryGenerator_SimpleGrid : public UEnvQueryGenerator_ProjectedPoints
{
	GENERATED_UCLASS_BODY()

	/** half of square's extent, like a radius */
	UPROPERTY(EditDefaultsOnly, Category=Generator, meta=(DisplayName="GridHalfSize"))
	FAIDataProviderFloatValue GridSize;

	/** generation density */
	UPROPERTY(EditDefaultsOnly, Category=Generator)
	FAIDataProviderFloatValue SpaceBetween;

	/** context */
	UPROPERTY(EditDefaultsOnly, Category=Generator)
	TSubclassOf<UEnvQueryContext> GenerateAround;

	AIMODULE_API virtual void GenerateItems(FEnvQueryInstance& QueryInstance) const override;

	AIMODULE_API virtual FText GetDescriptionTitle() const override;
	AIMODULE_API virtual FText GetDescriptionDetails() const override;
};
