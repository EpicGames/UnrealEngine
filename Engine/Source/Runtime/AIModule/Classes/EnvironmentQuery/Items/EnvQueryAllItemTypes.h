// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EnvironmentQuery/Items/EnvQueryItemType.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_Actor.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_ActorBase.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_Direction.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_Point.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_VectorBase.h"
