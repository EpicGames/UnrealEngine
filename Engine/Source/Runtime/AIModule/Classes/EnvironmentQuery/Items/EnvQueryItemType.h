// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "EnvQueryItemType.generated.h"

class UBlackboardComponent;
struct FBlackboardKeySelector;

UCLASS(Abstract, MinimalAPI)
class UEnvQueryItemType : public UObject
{
	GENERATED_BODY()

public:
	/** every EQS item type needs to speficy data type it's using. 
	 *	Default is void which should trigger a compilation error if it's 
	  *	not set in a defived class*/
	typedef void FValueType;

	AIMODULE_API UEnvQueryItemType(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** get ValueSize */
	inline uint16 GetValueSize() const { return ValueSize; }

	/** add filters for blackboard key selector */
	AIMODULE_API virtual void AddBlackboardFilters(FBlackboardKeySelector& KeySelector, UObject* FilterOwner) const;

	/** store value in blackboard entry */
	AIMODULE_API virtual bool StoreInBlackboard(FBlackboardKeySelector& KeySelector, UBlackboardComponent* Blackboard, const uint8* RawData) const;

	/** unregister from known types in EnvQueryManager */
	AIMODULE_API virtual void FinishDestroy() override;

	/** describe item */
	AIMODULE_API virtual FString GetDescription(const uint8* RawData) const;

	/** helper function for reading typed data from memory block */
	template<typename T>
	static const T& GetValueFromMemory(const uint8* MemoryBlock)
	{
		return *((T*)MemoryBlock);
	}

protected:

	/** size of value for this type */
	uint16 ValueSize;

	/** helper function for writing typed data to memory block */
	template<typename T>
	static void SetValueInMemory(uint8* MemoryBlock, const T& Value)
	{
		*((T*)MemoryBlock) = Value;
	}
};
