// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_ActorBase.h"
#include "EnvQueryItemType_Actor.generated.h"

class AActor;
struct FEnvQueryContextData;
struct FWeakObjectPtr;

UCLASS(MinimalAPI)
class UEnvQueryItemType_Actor : public UEnvQueryItemType_ActorBase
{
	GENERATED_BODY()
public:
	typedef const FWeakObjectPtr& FValueType;

	AIMODULE_API UEnvQueryItemType_Actor(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	static AIMODULE_API AActor* GetValue(const uint8* RawData);
	static AIMODULE_API void SetValue(uint8* RawData, const FWeakObjectPtr& Value);

	static AIMODULE_API void SetContextHelper(FEnvQueryContextData& ContextData, const AActor* SingleActor);
	static AIMODULE_API void SetContextHelper(FEnvQueryContextData& ContextData, const TArray<const AActor*>& MultipleActors);
	static AIMODULE_API void SetContextHelper(FEnvQueryContextData& ContextData, const TArray<AActor*>& MultipleActors);

	/** In editor, outside a PIE session, will add all currently selected actors to the context data. */
	static AIMODULE_API void AddEditorSelectedActors(FEnvQueryContextData& ContextData);

	AIMODULE_API virtual FVector GetItemLocation(const uint8* RawData) const override;
	AIMODULE_API virtual FRotator GetItemRotation(const uint8* RawData) const override;
	AIMODULE_API virtual AActor* GetActor(const uint8* RawData) const override;
};
