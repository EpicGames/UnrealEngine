// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_VectorBase.h"
#include "EnvQueryItemType_ActorBase.generated.h"

class AActor;
class UBlackboardComponent;
struct FBlackboardKeySelector;

UCLASS(Abstract, MinimalAPI)
class UEnvQueryItemType_ActorBase : public UEnvQueryItemType_VectorBase
{
	GENERATED_BODY()

public:
	AIMODULE_API virtual AActor* GetActor(const uint8* RawData) const;

	AIMODULE_API virtual void AddBlackboardFilters(FBlackboardKeySelector& KeySelector, UObject* FilterOwner) const override;
	AIMODULE_API virtual bool StoreInBlackboard(FBlackboardKeySelector& KeySelector, UBlackboardComponent* Blackboard, const uint8* RawData) const override;
	AIMODULE_API virtual FString GetDescription(const uint8* RawData) const override;
};
