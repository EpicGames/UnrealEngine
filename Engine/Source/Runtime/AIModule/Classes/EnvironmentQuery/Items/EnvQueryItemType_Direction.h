// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_VectorBase.h"
#include "EnvQueryItemType_Direction.generated.h"

struct FEnvQueryContextData;

UCLASS(MinimalAPI)
class UEnvQueryItemType_Direction : public UEnvQueryItemType_VectorBase
{
	GENERATED_BODY()
public:
	typedef FVector FValueType;

	AIMODULE_API UEnvQueryItemType_Direction(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	static AIMODULE_API FVector GetValue(const uint8* RawData);
	static AIMODULE_API void SetValue(uint8* RawData, const FVector& Value);

	static AIMODULE_API FRotator GetValueRot(const uint8* RawData);
	static AIMODULE_API void SetValueRot(uint8* RawData, const FRotator& Value);

	static AIMODULE_API void SetContextHelper(FEnvQueryContextData& ContextData, const FVector& SingleDirection);
	static AIMODULE_API void SetContextHelper(FEnvQueryContextData& ContextData, const FRotator& SingleRotation);
	static AIMODULE_API void SetContextHelper(FEnvQueryContextData& ContextData, const TArray<FVector>& MultipleDirections);
	static AIMODULE_API void SetContextHelper(FEnvQueryContextData& ContextData, const TArray<FRotator>& MultipleRotations);

	AIMODULE_API virtual FRotator GetItemRotation(const uint8* RawData) const override;
	AIMODULE_API virtual FString GetDescription(const uint8* RawData) const override;
};
