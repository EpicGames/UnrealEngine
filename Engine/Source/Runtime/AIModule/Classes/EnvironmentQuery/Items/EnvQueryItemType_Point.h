// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "AI/Navigation/NavigationTypes.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_VectorBase.h"
#include "EnvQueryItemType_Point.generated.h"

UCLASS(MinimalAPI)
class UEnvQueryItemType_Point : public UEnvQueryItemType_VectorBase
{
	GENERATED_BODY()
public:
	typedef const FNavLocation FValueType;

	AIMODULE_API UEnvQueryItemType_Point(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	static AIMODULE_API FNavLocation GetValue(const uint8* RawData);
	static AIMODULE_API void SetValue(uint8* RawData, const FNavLocation& Value);

	static AIMODULE_API void SetContextHelper(FEnvQueryContextData& ContextData, const FVector& SinglePoint);
	static AIMODULE_API void SetContextHelper(FEnvQueryContextData& ContextData, const TArray<FVector>& MultiplePoints);

	AIMODULE_API virtual FVector GetItemLocation(const uint8* RawData) const override;
	AIMODULE_API virtual FNavLocation GetItemNavLocation(const uint8* RawData) const;

	/** Update location data in existing item */
	AIMODULE_API virtual void SetItemNavLocation(uint8* RawData, const FNavLocation& Value) const;
};

// a specialization to support saving locations with navigation data already gathered
template<>
AIMODULE_API void FEnvQueryInstance::AddItemData<UEnvQueryItemType_Point, FVector>(FVector ItemValue);
