// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EnvironmentQuery/Items/EnvQueryItemType.h"
#include "EnvQueryItemType_VectorBase.generated.h"

class UBlackboardComponent;
struct FBlackboardKeySelector;

UCLASS(Abstract, MinimalAPI)
class UEnvQueryItemType_VectorBase : public UEnvQueryItemType
{
	GENERATED_BODY()

public:
	AIMODULE_API virtual FVector GetItemLocation(const uint8* RawData) const;
	AIMODULE_API virtual FRotator GetItemRotation(const uint8* RawData) const;

	AIMODULE_API virtual void AddBlackboardFilters(FBlackboardKeySelector& KeySelector, UObject* FilterOwner) const override;
	AIMODULE_API virtual bool StoreInBlackboard(FBlackboardKeySelector& KeySelector, UBlackboardComponent* Blackboard, const uint8* RawData) const override;
	AIMODULE_API virtual FString GetDescription(const uint8* RawData) const override;
};
