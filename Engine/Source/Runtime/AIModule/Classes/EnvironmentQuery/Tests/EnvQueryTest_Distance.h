// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "EnvironmentQuery/EnvQueryContext.h"
#include "EnvironmentQuery/EnvQueryTest.h"
#include "EnvQueryTest_Distance.generated.h"

UENUM()
namespace EEnvTestDistance
{
	enum Type : int
	{
		Distance3D,
		Distance2D,
		DistanceZ,
		DistanceAbsoluteZ UMETA(DisplayName = "Distance Z (Absolute)")
	};
}

UCLASS(MinimalAPI)
class UEnvQueryTest_Distance : public UEnvQueryTest
{
	GENERATED_UCLASS_BODY()

	/** testing mode */
	UPROPERTY(EditDefaultsOnly, Category=Distance)
	TEnumAsByte<EEnvTestDistance::Type> TestMode;

	/** context */
	UPROPERTY(EditDefaultsOnly, Category=Distance)
	TSubclassOf<UEnvQueryContext> DistanceTo;

	virtual void RunTest(FEnvQueryInstance& QueryInstance) const override;

	virtual FText GetDescriptionTitle() const override;
	virtual FText GetDescriptionDetails() const override;
};
