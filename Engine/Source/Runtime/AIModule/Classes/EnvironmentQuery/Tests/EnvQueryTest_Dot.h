// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "EnvironmentQuery/EnvQueryContext.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "EnvironmentQuery/EnvQueryTest.h"
#include "EnvQueryTest_Dot.generated.h"

UENUM(BlueprintType)
enum class EEnvTestDot : uint8
{
	Dot3D UMETA(DisplayName="Dot (3D)", Tooltip="Fully 3D dot-product"),
	Dot2D UMETA(DisplayName="Dot 2D (Heading)", Tooltip="Dot Product in the XY-plane, which is equivalent to the cosine of the heading or yaw angle.")
	// We could add additional tests here, such as Pitch (Dot of XY-length, Z).
};

UCLASS(MinimalAPI)
class UEnvQueryTest_Dot : public UEnvQueryTest
{
	GENERATED_UCLASS_BODY()

protected:
	/** defines direction of first line used by test */
	UPROPERTY(EditDefaultsOnly, Category=Dot)
	FEnvDirection LineA;

	/** defines direction of second line used by test */
	UPROPERTY(EditDefaultsOnly, Category=Dot)
	FEnvDirection LineB;

	UPROPERTY(EditDefaultsOnly, Category=Dot)
	EEnvTestDot TestMode;

	/** If true, this test uses the absolute value of the dot product rather than the dot product itself.  Useful
	  * when you want to compare "how lateral" something is.  I.E. values closer to zero are further to the side, 
	  * and values closer to 1 are more in front or behind (without distinguishing forward/backward).
	  */
	UPROPERTY(EditDefaultsOnly, Category=Dot)
	bool bAbsoluteValue;

	virtual void RunTest(FEnvQueryInstance& QueryInstance) const override;

	virtual FText GetDescriptionTitle() const override;
	virtual FText GetDescriptionDetails() const override;

	/** helper function: gather directions from context pairs */
	void GatherLineDirections(TArray<FVector>& Directions, FEnvQueryInstance& QueryInstance, const FVector& ItemLocation,
		TSubclassOf<UEnvQueryContext> LineFrom, TSubclassOf<UEnvQueryContext> LineTo) const;

	/** helper function: gather directions from context */
	void GatherLineDirections(TArray<FVector>& Directions, FEnvQueryInstance& QueryInstance, const FRotator& ItemRotation,
		TSubclassOf<UEnvQueryContext> LineDirection) const;

	/** helper function: gather directions from proper contexts */
	void GatherLineDirections(TArray<FVector>& Directions, FEnvQueryInstance& QueryInstance,
		TSubclassOf<UEnvQueryContext> LineFrom, TSubclassOf<UEnvQueryContext> LineTo, TSubclassOf<UEnvQueryContext> LineDirection, bool bUseDirectionContext,
		const FVector& ItemLocation = FVector::ZeroVector, const FRotator& ItemRotation = FRotator::ZeroRotator) const;

	/** helper function: check if contexts are updated per item */
	bool RequiresPerItemUpdates(TSubclassOf<UEnvQueryContext> LineFrom, TSubclassOf<UEnvQueryContext> LineTo, TSubclassOf<UEnvQueryContext> LineDirection, bool bUseDirectionContext) const;
};
