// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameplayTagContainer.h"
#include "EnvironmentQuery/EnvQueryTest.h"
#include "EnvQueryTest_GameplayTags.generated.h"

class IGameplayTagAssetInterface;

/** 
 * EnvQueryTest_GameplayTags attempts to cast items to IGameplayTagAssetInterface and test their tags with TagQueryToMatch.
 * The behavior of IGameplayTagAssetInterface-less items is configured by bRejectIncompatibleItems.
 */
UCLASS(MinimalAPI, config = Game, defaultconfig)
class UEnvQueryTest_GameplayTags : public UEnvQueryTest
{
	GENERATED_UCLASS_BODY()

public:
	/**
	 * @note Calling function only makes sense before first run of given query
	 * by the EQS manager. The query gets preprocessed and cached then so the query 
	 * value will get stored and calling this function will not change it (unless 
	 * you call it on the cached test's instance, see UEnvQueryManager::CreateQueryInstance).
	 */
	void SetTagQueryToMatch(FGameplayTagQuery& GameplayTagQuery);

protected:
	virtual void RunTest(FEnvQueryInstance& QueryInstance) const override;

	virtual FText GetDescriptionDetails() const override;

	bool SatisfiesTest(const IGameplayTagAssetInterface* ItemGameplayTagAssetInterface) const;

	/**
	 * Presave function. Gets called once before an object gets serialized for saving. This function is necessary
	 * for save time computation as Serialize gets called three times per object from within SavePackage.
	 *
	 * @warning: Objects created from within PreSave will NOT have PreSave called on them!!!
	 */
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;

	virtual void PostLoad() override;

	UPROPERTY(EditAnywhere, Category=GameplayTagCheck)
	FGameplayTagQuery TagQueryToMatch;

	// If the actor-item being tested doesn't implement IGameplayTagAssetInterface this flag controls whether the actor's components will be traversed in search of a one implementing IGameplayTagAssetInterface. 
	// Note that only the first found component will be used. 
	UPROPERTY(EditAnywhere, Config, Category=GameplayTagCheck)
	bool bSearchComponentsForTags = false;

	// This controls how to treat actors that do not implement IGameplayTagAssetInterface.
	// When set to True, actors that do not implement the interface will be ignored, meaning
	// they will not be scored and will not be considered when filtering.
	// When set to False, actors that do not implement the interface will be included in
	// filter and score operations with a zero score.
	UPROPERTY(EditAnywhere, Category=GameplayTagCheck)
	bool bRejectIncompatibleItems;

	// Used to determine whether the file format needs to be updated to move data into TagQueryToMatch or not.
	UPROPERTY()
	bool bUpdatedToUseQuery;

	// Deprecated property.  Used only to load old data into TagQueryToMatch.
	UPROPERTY()
	EGameplayContainerMatchType TagsToMatch;

	// Deprecated property.  Used only to load old data into TagQueryToMatch.
	UPROPERTY()
	FGameplayTagContainer GameplayTags;
};
