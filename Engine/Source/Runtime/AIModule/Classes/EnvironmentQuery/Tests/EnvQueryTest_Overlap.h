// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "EnvironmentQuery/EnvQueryTest.h"
#include "EnvQueryTest_Overlap.generated.h"

class AActor;
struct FCollisionQueryParams;
struct FCollisionShape;

UCLASS(MinimalAPI)
class UEnvQueryTest_Overlap : public UEnvQueryTest
{
	GENERATED_UCLASS_BODY()

	/** Overlap data */
	UPROPERTY(EditDefaultsOnly, Category=Overlap)
	FEnvOverlapData OverlapData;

	virtual void RunTest(FEnvQueryInstance& QueryInstance) const override;

	virtual FText GetDescriptionTitle() const override;
	virtual FText GetDescriptionDetails() const override;

protected:

	bool RunOverlap(const FVector& ItemPos, const FCollisionShape& CollisionShape, const TArray<AActor*>& IgnoredActors, UWorld* World, enum ECollisionChannel Channel, const FCollisionQueryParams& Params) const;
	bool RunOverlapBlocking(const FVector& ItemPos, const FCollisionShape& CollisionShape, const TArray<AActor*>& IgnoredActors, UWorld* World, enum ECollisionChannel Channel, const FCollisionQueryParams& Params) const;
};
