// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "AI/Navigation/NavigationTypes.h"
#include "NavigationSystemTypes.h"
#include "EnvironmentQuery/EnvQueryContext.h"
#include "DataProviders/AIDataProvider.h"
#include "EnvironmentQuery/EnvQueryTest.h"
#include "EnvQueryTest_Pathfinding.generated.h"

class ANavigationData;
class UNavigationSystemV1;

UENUM()
namespace EEnvTestPathfinding
{
	enum Type : int
	{
		PathExist,
		PathCost,
		PathLength,
	};
}

UCLASS(MinimalAPI)
class UEnvQueryTest_Pathfinding : public UEnvQueryTest
{
	GENERATED_UCLASS_BODY()

	/** testing mode */
	UPROPERTY(EditDefaultsOnly, Category=Pathfinding)
	TEnumAsByte<EEnvTestPathfinding::Type> TestMode;

	/** context: other end of pathfinding test */
	UPROPERTY(EditDefaultsOnly, Category=Pathfinding)
	TSubclassOf<UEnvQueryContext> Context;

	/** pathfinding direction */
	UPROPERTY(EditDefaultsOnly, Category=Pathfinding)
	FAIDataProviderBoolValue PathFromContext;

	/** if set, items with failed path will be invalidated (PathCost, PathLength) */
	UPROPERTY(EditDefaultsOnly, Category=Pathfinding, AdvancedDisplay)
	FAIDataProviderBoolValue SkipUnreachable;

	/** navigation filter to use in pathfinding */
	UPROPERTY(EditDefaultsOnly, Category=Pathfinding)
	TSubclassOf<UNavigationQueryFilter> FilterClass;

	/** Navigation Data Override */
	UPROPERTY(EditDefaultsOnly, Category = "Generator")
	TSubclassOf<UEnvQueryContext> NavDataOverrideContext;

	AIMODULE_API virtual void RunTest(FEnvQueryInstance& QueryInstance) const override;

	AIMODULE_API virtual FText GetDescriptionTitle() const override;
	AIMODULE_API virtual FText GetDescriptionDetails() const override;

#if WITH_EDITOR
	/** update test properties after changing mode */
	AIMODULE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	AIMODULE_API virtual void PostLoad() override;

protected:

	DECLARE_DELEGATE_RetVal_SevenParams(bool, FTestPathSignature, const FVector&, const FVector&, EPathFindingMode::Type, const ANavigationData&, UNavigationSystemV1&, FSharedConstNavQueryFilter, const UObject*);
	DECLARE_DELEGATE_RetVal_SevenParams(float, FFindPathSignature, const FVector&, const FVector&, EPathFindingMode::Type, const ANavigationData&, UNavigationSystemV1&, FSharedConstNavQueryFilter, const UObject*);

	AIMODULE_API bool TestPathFrom(const FVector& ItemPos, const FVector& ContextPos, EPathFindingMode::Type Mode, const ANavigationData& NavData, UNavigationSystemV1& NavSys, FSharedConstNavQueryFilter NavFilter, const UObject* PathOwner) const;
	AIMODULE_API bool TestPathTo(const FVector& ItemPos, const FVector& ContextPos, EPathFindingMode::Type Mode, const ANavigationData& NavData, UNavigationSystemV1& NavSys, FSharedConstNavQueryFilter NavFilter, const UObject* PathOwner) const;
	AIMODULE_API float FindPathCostFrom(const FVector& ItemPos, const FVector& ContextPos, EPathFindingMode::Type Mode, const ANavigationData& NavData, UNavigationSystemV1& NavSys, FSharedConstNavQueryFilter NavFilter, const UObject* PathOwner) const;
	AIMODULE_API float FindPathCostTo(const FVector& ItemPos, const FVector& ContextPos, EPathFindingMode::Type Mode, const ANavigationData& NavData, UNavigationSystemV1& NavSys, FSharedConstNavQueryFilter NavFilter, const UObject* PathOwner) const;
	AIMODULE_API float FindPathLengthFrom(const FVector& ItemPos, const FVector& ContextPos, EPathFindingMode::Type Mode, const ANavigationData& NavData, UNavigationSystemV1& NavSys, FSharedConstNavQueryFilter NavFilter, const UObject* PathOwner) const;
	AIMODULE_API float FindPathLengthTo(const FVector& ItemPos, const FVector& ContextPos, EPathFindingMode::Type Mode, const ANavigationData& NavData, UNavigationSystemV1& NavSys, FSharedConstNavQueryFilter NavFilter, const UObject* PathOwner) const;

	UE_DEPRECATED(5.8, "Please use FEQSHelpers::FindNavigationDataForQuery instead")
	AIMODULE_API ANavigationData* FindNavigationData(UNavigationSystemV1& NavSys, UObject* Owner) const;

	AIMODULE_API virtual TSubclassOf<UNavigationQueryFilter> GetNavFilterClass(FEnvQueryInstance& QueryInstance) const;
};
