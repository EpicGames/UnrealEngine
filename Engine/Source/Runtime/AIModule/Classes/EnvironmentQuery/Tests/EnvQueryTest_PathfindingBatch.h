// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "DataProviders/AIDataProvider.h"
#include "EnvironmentQuery/Tests/EnvQueryTest_Pathfinding.h"
#include "EnvQueryTest_PathfindingBatch.generated.h"

UCLASS(MinimalAPI)
class UEnvQueryTest_PathfindingBatch : public UEnvQueryTest_Pathfinding
{
	GENERATED_UCLASS_BODY()

	/** multiplier for max distance between point and context */
	UPROPERTY(EditDefaultsOnly, Category = Pathfinding, AdvancedDisplay)
	FAIDataProviderFloatValue ScanRangeMultiplier;

	AIMODULE_API virtual void RunTest(FEnvQueryInstance& QueryInstance) const override;
	AIMODULE_API virtual FText GetDescriptionTitle() const override;
};
