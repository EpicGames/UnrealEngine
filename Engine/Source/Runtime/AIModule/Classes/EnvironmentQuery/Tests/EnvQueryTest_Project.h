// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "EnvironmentQuery/EnvQueryTest.h"
#include "EnvQueryTest_Project.generated.h"

/**
 * Projects points on navigation or geometry, will modify value of projected items.
 * Works only on item type: point
 */
UCLASS(MinimalAPI)
class UEnvQueryTest_Project : public UEnvQueryTest
{
	GENERATED_UCLASS_BODY()

protected:

	/** trace params */
	UPROPERTY(EditDefaultsOnly, Category = Test)
	FEnvTraceData ProjectionData;

	virtual void RunTest(FEnvQueryInstance& QueryInstance) const override;
	virtual FText GetDescriptionDetails() const override;
};
