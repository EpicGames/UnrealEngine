// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EnvironmentQuery/EnvQueryTest.h"
#include "EnvQueryTest_Random.generated.h"

UCLASS(MinimalAPI, meta=(DeprecatedNode, DeprecationMessage = "This class is now deprecated, please use RunMode supporting random results instead."))
class UEnvQueryTest_Random : public UEnvQueryTest
{
	GENERATED_UCLASS_BODY()

protected:
	virtual void RunTest(FEnvQueryInstance& QueryInstance) const override;

	virtual FText GetDescriptionDetails() const override;
};
