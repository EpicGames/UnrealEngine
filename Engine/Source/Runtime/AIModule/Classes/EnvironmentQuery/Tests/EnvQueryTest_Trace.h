// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "EnvironmentQuery/EnvQueryContext.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "DataProviders/AIDataProvider.h"
#include "EnvironmentQuery/EnvQueryTest.h"
#include "EnvQueryTest_Trace.generated.h"

class AActor;
struct FCollisionQueryParams;

UCLASS(MinimalAPI)
class UEnvQueryTest_Trace : public UEnvQueryTest
{
	GENERATED_UCLASS_BODY()

	/** trace data */
	UPROPERTY(EditDefaultsOnly, Category=Trace)
	FEnvTraceData TraceData;

	/** trace direction */
	UPROPERTY(EditDefaultsOnly, Category=Trace)
	FAIDataProviderBoolValue TraceFromContext;

	/** Z offset from item */
	UPROPERTY(EditDefaultsOnly, Category=Trace, AdvancedDisplay)
	FAIDataProviderFloatValue ItemHeightOffset;

	/** Z offset from querier */
	UPROPERTY(EditDefaultsOnly, Category=Trace, AdvancedDisplay)
	FAIDataProviderFloatValue ContextHeightOffset;

	/** context: other end of trace test */
	UPROPERTY(EditDefaultsOnly, Category=Trace)
	TSubclassOf<UEnvQueryContext> Context;

	AIMODULE_API virtual void RunTest(FEnvQueryInstance& QueryInstance) const override;

	AIMODULE_API virtual FText GetDescriptionTitle() const override;
	AIMODULE_API virtual FText GetDescriptionDetails() const override;

	AIMODULE_API virtual void PostLoad() override;

protected:

	UE_DEPRECATED(5.4, "This delegate type is not used anymore in UEnvQueryTest_Trace, new code is done in EnvQueryTest_Trace_Helpers namespace helper functions")
	DECLARE_DELEGATE_RetVal_SevenParams(bool, FRunTraceSignature, const FVector&, const FVector&, AActor*, UWorld*, enum ECollisionChannel, const FCollisionQueryParams&, const FVector&);

	UE_DEPRECATED(5.4, "This function will be deleted")
	AIMODULE_API bool RunLineTraceTo(const FVector& ItemPos, const FVector& ContextPos, AActor* ItemActor, UWorld* World, enum ECollisionChannel Channel, const FCollisionQueryParams& Params, const FVector& Extent);
	UE_DEPRECATED(5.4, "This function will be deleted")
	AIMODULE_API bool RunLineTraceFrom(const FVector& ItemPos, const FVector& ContextPos, AActor* ItemActor, UWorld* World, enum ECollisionChannel Channel, const FCollisionQueryParams& Params, const FVector& Extent);
	UE_DEPRECATED(5.4, "This function will be deleted")
	AIMODULE_API bool RunBoxTraceTo(const FVector& ItemPos, const FVector& ContextPos, AActor* ItemActor, UWorld* World, enum ECollisionChannel Channel, const FCollisionQueryParams& Params, const FVector& Extent);
	UE_DEPRECATED(5.4, "This function will be deleted")
	AIMODULE_API bool RunBoxTraceFrom(const FVector& ItemPos, const FVector& ContextPos, AActor* ItemActor, UWorld* World, enum ECollisionChannel Channel, const FCollisionQueryParams& Params, const FVector& Extent);
	UE_DEPRECATED(5.4, "This function will be deleted")
	AIMODULE_API bool RunSphereTraceTo(const FVector& ItemPos, const FVector& ContextPos, AActor* ItemActor, UWorld* World, enum ECollisionChannel Channel, const FCollisionQueryParams& Params, const FVector& Extent);
	UE_DEPRECATED(5.4, "This function will be deleted")
	AIMODULE_API bool RunSphereTraceFrom(const FVector& ItemPos, const FVector& ContextPos, AActor* ItemActor, UWorld* World, enum ECollisionChannel Channel, const FCollisionQueryParams& Params, const FVector& Extent);
	UE_DEPRECATED(5.4, "This function will be deleted")
	AIMODULE_API bool RunCapsuleTraceTo(const FVector& ItemPos, const FVector& ContextPos, AActor* ItemActor, UWorld* World, enum ECollisionChannel Channel, const FCollisionQueryParams& Params, const FVector& Extent);
	UE_DEPRECATED(5.4, "This function will be deleted")
	AIMODULE_API bool RunCapsuleTraceFrom(const FVector& ItemPos, const FVector& ContextPos, AActor* ItemActor, UWorld* World, enum ECollisionChannel Channel, const FCollisionQueryParams& Params, const FVector& Extent);
};
