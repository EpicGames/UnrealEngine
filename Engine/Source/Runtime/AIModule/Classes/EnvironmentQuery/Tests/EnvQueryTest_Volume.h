// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "EnvironmentQuery/EnvQueryTest.h"
#include "EnvQueryTest_Volume.generated.h"

class AVolume;

UCLASS(MinimalAPI)
class UEnvQueryTest_Volume : public UEnvQueryTest
{
	GENERATED_BODY()

public:

	UEnvQueryTest_Volume(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	virtual void RunTest(FEnvQueryInstance& QueryInstance) const override;

	virtual FText GetDescriptionTitle() const override;
	virtual FText GetDescriptionDetails() const override;

private:

	/** Context that populates a list of Actors derived from AVolume to test against */
	UPROPERTY(EditAnywhere, Category = Volumes, meta = (AllowPrivateAccess = "true"))
	TSubclassOf<class UEnvQueryContext> VolumeContext;

	/** If VolumeContext is null, AVolume Class will be used to populate a list of AVolume to test against */
	UPROPERTY(EditAnywhere, Category = Volumes, meta = (AllowPrivateAccess = "true"))
	TSubclassOf<AVolume> VolumeClass;

	/** If bDoComplexVolumeTest is set, it will use a full volume physic test (not only a bounding box test)  */
	UPROPERTY(EditAnywhere, Category = Volumes, meta = (AllowPrivateAccess = "true"))
	uint32 bDoComplexVolumeTest : 1;

	/** If no volumes are returned to evaluate the points against then just ignore the test rather than failing each item */
	UPROPERTY(EditAnywhere, Category = Volumes, meta = (AllowPrivateAccess = "true"))
	uint32 bSkipTestIfNoVolumes : 1;
};
