// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "GameFramework/Actor.h"
#include "GenericTeamAgentInterface.generated.h"

UENUM(BlueprintType)
namespace ETeamAttitude
{
	enum Type : int
	{
		Friendly,
		Neutral,
		Hostile,
	};
}

USTRUCT(BlueprintType)
struct FGenericTeamId
{
	GENERATED_USTRUCT_BODY()

private:
	enum EPredefinedId
	{
		// if you want to change NoTeam's ID update FGenericTeamId::NoTeam

		NoTeamId = 255
	};

protected:
	UPROPERTY(Category = "TeamID", EditAnywhere, BlueprintReadWrite)
	uint8 TeamID;

public:
	FGenericTeamId(uint8 InTeamID = NoTeamId) : TeamID(InTeamID)
	{}

	inline operator uint8() const { return TeamID; }

	inline uint8 GetId() const { return TeamID; }
	//FORCEINLINE void SetId(uint8 NewId) { TeamID = NewId; }
	
	static AIMODULE_API FGenericTeamId GetTeamIdentifier(const AActor* TeamMember);
	static AIMODULE_API ETeamAttitude::Type GetAttitude(const AActor* A, const AActor* B);
	static ETeamAttitude::Type GetAttitude(FGenericTeamId TeamA, FGenericTeamId TeamB)
	{
		return AttitudeSolverImpl ? (AttitudeSolverImpl)(TeamA, TeamB) : ETeamAttitude::Neutral;
	}

	typedef TFunction<ETeamAttitude::Type(FGenericTeamId, FGenericTeamId)> FAttitudeSolverFunction;
	
	static AIMODULE_API void SetAttitudeSolver(const FAttitudeSolverFunction& Solver);
	static AIMODULE_API void ResetAttitudeSolver();

protected:
	// the default implementation makes all teams hostile
	// @note that for consistency IGenericTeamAgentInterface should be using the same function 
	//	(by default it does)
	static AIMODULE_API FAttitudeSolverFunction AttitudeSolverImpl;

public:
	static AIMODULE_API const FGenericTeamId NoTeam;

	friend inline uint32 GetTypeHash(const FGenericTeamId Value)
	{
		return Value.GetId();
	}
};

UINTERFACE(MinimalAPI)
class UGenericTeamAgentInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IGenericTeamAgentInterface
{
	GENERATED_IINTERFACE_BODY()

	/** Assigns Team Agent to given TeamID */
	virtual void SetGenericTeamId(const FGenericTeamId& TeamID) {}
	
	/** Retrieve team identifier in form of FGenericTeamId */
	virtual FGenericTeamId GetGenericTeamId() const { return FGenericTeamId::NoTeam; }

	/** Retrieved owner attitude toward given Other object */
	virtual ETeamAttitude::Type GetTeamAttitudeTowards(const AActor& Other) const
	{ 
		const IGenericTeamAgentInterface* OtherTeamAgent = Cast<const IGenericTeamAgentInterface>(&Other);
		return OtherTeamAgent ? FGenericTeamId::GetAttitude(GetGenericTeamId(), OtherTeamAgent->GetGenericTeamId())
			: ETeamAttitude::Neutral;
	}
};
