// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "AIHotSpotManager.generated.h"

UCLASS(MinimalAPI)
class UAIHotSpotManager : public UObject
{
	GENERATED_UCLASS_BODY()
};
