// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "AITypes.h"
#include "CrowdAgentInterface.generated.h"

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UCrowdAgentInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class ICrowdAgentInterface
{
	GENERATED_IINTERFACE_BODY()

	/** @return current location of crowd agent */
	virtual FVector GetCrowdAgentLocation() const { return FAISystem::InvalidLocation; }

	/** @return current velocity of crowd agent */
	virtual FVector GetCrowdAgentVelocity() const { return FVector::ZeroVector; }

	/** fills information about agent's collision cylinder */
	virtual void GetCrowdAgentCollisions(float& CylinderRadius, float& CylinderHalfHeight) const {}

	/** @return max speed of crowd agent */
	virtual float GetCrowdAgentMaxSpeed() const { return 0.0f; }

	/** Group mask for this agent */
	virtual int32 GetCrowdAgentAvoidanceGroup() const { return 1; }

	/** Will avoid other agents if they are in one of specified groups */
	virtual int32 GetCrowdAgentGroupsToAvoid() const { return MAX_int32; }

	/** Will NOT avoid other agents if they are in one of specified groups, higher priority than GroupsToAvoid */
	virtual int32 GetCrowdAgentGroupsToIgnore() const { return 0; }
};
