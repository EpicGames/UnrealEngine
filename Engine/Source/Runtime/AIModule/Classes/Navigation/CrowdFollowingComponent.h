// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakInterfacePtr.h"
#include "EngineDefines.h"
#include "AITypes.h"
#include "Navigation/PathFollowingComponent.h"
#include "Navigation/CrowdAgentInterface.h"
#include "AI/Navigation/NavigationAvoidanceTypes.h"
#include "CrowdFollowingComponent.generated.h"

class INavLinkCustomInterface;
class IRVOAvoidanceInterface;
class UCrowdManager;

namespace ECrowdAvoidanceQuality
{
	enum Type
	{
		Low,
		Medium,
		Good,
		High,
	};
}

enum class ECrowdSimulationState : uint8
{
	Enabled,
	ObstacleOnly, // Disabled, avoided by others
	Disabled, // Disabled, ignored by others
};

UCLASS(BlueprintType, MinimalAPI)
class UCrowdFollowingComponent : public UPathFollowingComponent, public ICrowdAgentInterface
{
	GENERATED_UCLASS_BODY()

	AIMODULE_API virtual void BeginDestroy() override;

	// ICrowdAgentInterface BEGIN
	AIMODULE_API virtual FVector GetCrowdAgentLocation() const override;
	AIMODULE_API virtual FVector GetCrowdAgentVelocity() const override;
	AIMODULE_API virtual void GetCrowdAgentCollisions(float& CylinderRadius, float& CylinderHalfHeight) const override;
	AIMODULE_API virtual float GetCrowdAgentMaxSpeed() const override;
	AIMODULE_API virtual int32 GetCrowdAgentAvoidanceGroup() const override;
	AIMODULE_API virtual int32 GetCrowdAgentGroupsToAvoid() const override;
	AIMODULE_API virtual int32 GetCrowdAgentGroupsToIgnore() const override;
	// ICrowdAgentInterface END

	// PathFollowingComponent BEGIN
	AIMODULE_API virtual void Initialize() override;
	AIMODULE_API virtual void Cleanup() override;
	AIMODULE_API virtual void AbortMove(const UObject& Instigator, FPathFollowingResultFlags::Type AbortFlags, FAIRequestID RequestID = FAIRequestID::CurrentRequest, EPathFollowingVelocityMode VelocityMode = EPathFollowingVelocityMode::Reset) override;
	AIMODULE_API virtual void PauseMove(FAIRequestID RequestID = FAIRequestID::CurrentRequest, EPathFollowingVelocityMode VelocityMode = EPathFollowingVelocityMode::Reset) override;
	AIMODULE_API virtual void ResumeMove(FAIRequestID RequestID = FAIRequestID::CurrentRequest) override;
	AIMODULE_API virtual FVector GetMoveFocus(bool bAllowStrafe) const override;
	AIMODULE_API virtual void OnLanded() override;
	AIMODULE_API virtual void FinishUsingCustomLink(INavLinkCustomInterface* CustomNavLink) override;
	AIMODULE_API virtual void OnPathFinished(const FPathFollowingResult& Result) override;
	AIMODULE_API virtual void OnPathUpdated() override;
	AIMODULE_API virtual void OnPathfindingQuery(FPathFindingQuery& Query) override;
	virtual int32 GetCurrentPathElement() const override { return LastPathPolyIndex; }
	AIMODULE_API virtual void OnNavigationInitDone() override;
	// PathFollowingComponent END

	/** update params in crowd manager */
	AIMODULE_API void UpdateCrowdAgentParams() const;

	/** pass agent velocity to movement component */
	AIMODULE_API virtual void ApplyCrowdAgentVelocity(const FVector& NewVelocity, const FVector& DestPathCorner, bool bTraversingLink, bool bIsNearEndOfPath);
	
	/** pass desired position to movement component (after resolving collisions between crowd agents) */
	AIMODULE_API virtual void ApplyCrowdAgentPosition(const FVector& NewPosition);

	/** main switch for crowd steering & avoidance */
	UFUNCTION(BlueprintCallable, Category = "Crowd")
	AIMODULE_API virtual void SuspendCrowdSteering(bool bSuspend);

	/** switch between crowd simulation and parent implementation (following path segments) */
	AIMODULE_API virtual void SetCrowdSimulationState(ECrowdSimulationState NewState);

	/** called when agent moved to next nav node (poly) */
	AIMODULE_API virtual void OnNavNodeChanged(NavNodeRef NewPolyRef, NavNodeRef PrevPolyRef, int32 CorridorSize);

	AIMODULE_API void SetCrowdAnticipateTurns(bool bEnable, bool bUpdateAgent = true);
	AIMODULE_API void SetCrowdObstacleAvoidance(bool bEnable, bool bUpdateAgent = true);
	AIMODULE_API void SetCrowdSeparation(bool bEnable, bool bUpdateAgent = true);
	AIMODULE_API void SetCrowdOptimizeVisibility(bool bEnable, bool bUpdateAgent = true);
	AIMODULE_API void SetCrowdOptimizeTopology(bool bEnable, bool bUpdateAgent = true);
	AIMODULE_API void SetCrowdPathOffset(bool bEnable, bool bUpdateAgent = true);
	AIMODULE_API void SetCrowdSlowdownAtGoal(bool bEnable, bool bUpdateAgent = true);
	AIMODULE_API void SetCrowdSeparationWeight(float Weight, bool bUpdateAgent = true);
	AIMODULE_API void SetCrowdCollisionQueryRange(float Range, bool bUpdateAgent = true);
	AIMODULE_API void SetCrowdPathOptimizationRange(float Range, bool bUpdateAgent = true);
	AIMODULE_API void SetCrowdAvoidanceQuality(ECrowdAvoidanceQuality::Type Quality, bool bUpdateAgent = true);
	AIMODULE_API void SetCrowdAvoidanceRangeMultiplier(float Multiplier, bool bUpdateAgent = true);
	AIMODULE_API void SetCrowdAffectFallingVelocity(bool bEnable);
	AIMODULE_API void SetCrowdRotateToVelocity(bool bEnable);
	AIMODULE_API void SetAvoidanceGroup(int32 GroupFlags, bool bUpdateAgent = true);
	AIMODULE_API void SetGroupsToAvoid(int32 GroupFlags, bool bUpdateAgent = true);
	AIMODULE_API void SetGroupsToIgnore(int32 GroupFlags, bool bUpdateAgent = true);

	inline bool IsCrowdSimulationEnabled() const { return SimulationState == ECrowdSimulationState::Enabled; }
	inline bool IsCrowdSimulatioSuspended() const { return bSuspendCrowdSimulation; }
	inline bool IsCrowdAnticipateTurnsEnabled() const { return bEnableAnticipateTurns; }
	inline bool IsCrowdObstacleAvoidanceEnabled() const { return bEnableObstacleAvoidance; }
	inline bool IsCrowdSeparationEnabled() const { return bEnableSeparation; }
	inline bool IsCrowdOptimizeVisibilityEnabled() const { return bEnableOptimizeVisibility; /** don't check suspend here! */ }
	inline bool IsCrowdOptimizeTopologyEnabled() const { return bEnableOptimizeTopology; }
	inline bool IsCrowdPathOffsetEnabled() const { return bEnablePathOffset; }
	inline bool IsCrowdSlowdownAtGoalEnabled() const { return bEnableSlowdownAtGoal; }
	inline bool IsCrowdAffectFallingVelocityEnabled() const { return bAffectFallingVelocity; }
	inline bool IsCrowdRotateToVelocityEnabled() const { return bRotateToVelocity; }

	inline ECrowdSimulationState GetCrowdSimulationState() const { return SimulationState; }
	inline bool IsCrowdSimulationActive() const { return IsCrowdSimulationEnabled() && !IsCrowdSimulatioSuspended(); }
	/** checks if bEnableAnticipateTurns is set to true, and if crowd simulation is not suspended */
	inline bool IsCrowdAnticipateTurnsActive() const { return IsCrowdAnticipateTurnsEnabled() && !IsCrowdSimulatioSuspended(); }
	/** checks if bEnableObstacleAvoidance is set to true, and if crowd simulation is not suspended */
	inline bool IsCrowdObstacleAvoidanceActive() const { return IsCrowdObstacleAvoidanceEnabled() && !IsCrowdSimulatioSuspended(); }
	/** checks if bEnableSeparation is set to true, and if crowd simulation is not suspended */
	inline bool IsCrowdSeparationActive() const { return IsCrowdSeparationEnabled() && !IsCrowdSimulatioSuspended(); }
	/** checks if bEnableOptimizeTopology is set to true, and if crowd simulation is not suspended */
	inline bool IsCrowdOptimizeTopologyActive() const { return IsCrowdOptimizeTopologyEnabled() && !IsCrowdSimulatioSuspended(); }

	inline float GetCrowdSeparationWeight() const { return SeparationWeight; }
	inline float GetCrowdCollisionQueryRange() const { return CollisionQueryRange; }
	inline float GetCrowdPathOptimizationRange() const { return PathOptimizationRange; }
	inline ECrowdAvoidanceQuality::Type GetCrowdAvoidanceQuality() const { return AvoidanceQuality; }
	inline float GetCrowdAvoidanceRangeMultiplier() const { return AvoidanceRangeMultiplier; }
	AIMODULE_API int32 GetAvoidanceGroup() const;
	AIMODULE_API int32 GetGroupsToAvoid() const;
	AIMODULE_API int32 GetGroupsToIgnore() const;

	AIMODULE_API virtual void GetDebugStringTokens(TArray<FString>& Tokens, TArray<EPathFollowingDebugTokens::Type>& Flags) const override;
#if ENABLE_VISUAL_LOG
	AIMODULE_API virtual void DescribeSelfToVisLog(struct FVisualLogEntry* Snapshot) const override;
#endif // ENABLE_VISUAL_LOG

	UE_DEPRECATED_FORGAME(4.16, "Use ApplyCrowdAgentVelocity function with bIsNearEndOfPath param instead.")
	virtual void ApplyCrowdAgentVelocity(const FVector& NewVelocity, const FVector& DestPathCorner, bool bTraversingLink) {}

	AIMODULE_API void UpdateDestinationForMovingGoal(const FVector& NewDestination);

protected:

	TWeakInterfacePtr<IRVOAvoidanceInterface> AvoidanceInterface;

public:
	UPROPERTY()
	FVector CrowdAgentMoveDirection;

protected:
#if WITH_EDITORONLY_DATA
	/** DEPRECATED: Group mask for this agent - use IRVOAvoidanceInterface instead */
	UPROPERTY()
	FNavAvoidanceMask AvoidanceGroup_DEPRECATED;

	/** DEPRECATED: Will avoid other agents if they are in one of specified groups - use IRVOAvoidanceInterface instead */
	UPROPERTY()
	FNavAvoidanceMask GroupsToAvoid_DEPRECATED;

	/** DEPRECATED: Will NOT avoid other agents if they are in one of specified groups, higher priority than GroupsToAvoid - use IRVOAvoidanceInterface instead */
	UPROPERTY()
	FNavAvoidanceMask GroupsToIgnore_DEPRECATED;
#endif

	/** if set, velocity will be updated even if agent is falling */
	uint8 bAffectFallingVelocity : 1;

	/** if set, move focus will match velocity direction */
	uint8 bRotateToVelocity : 1;

	/** if set, move velocity will be updated in every tick */
	uint8 bUpdateDirectMoveVelocity : 1;

	/** set when agent is registered in crowd simulation (either controlled or an obstacle) */
	uint8 bRegisteredWithCrowdSimulation : 1;

	/** if set, avoidance and steering will be suspended (used for direct move requests) */
	uint8 bSuspendCrowdSimulation : 1;

	uint8 bEnableAnticipateTurns : 1;
	uint8 bEnableObstacleAvoidance : 1;
	uint8 bEnableSeparation : 1;
	uint8 bEnableOptimizeVisibility : 1;
	uint8 bEnableOptimizeTopology : 1;
	uint8 bEnablePathOffset : 1;
	uint8 bEnableSlowdownAtGoal : 1;

	/** if set, agent if moving on final path part, skip further updates (runtime flag) */
	uint8 bFinalPathPart : 1;

	/** if set, destination overshot can be tested */
	uint8 bCanCheckMovingTooFar : 1;

	/** if set, path parts can be switched in UpdatePathSegment, based on distance */
	uint8 bCanUpdatePathPartInTick : 1;

	/** if set, movement will be finished when velocity is opposite to path direction (runtime flag) */
	uint8 bCheckMovementAngle : 1;

	uint8 bEnableSimulationReplanOnResume : 1;

	uint8 bIsTraversingLink: 1;

	TEnumAsByte<ECrowdAvoidanceQuality::Type> AvoidanceQuality;
	ECrowdSimulationState SimulationState;

	float SeparationWeight;
	float CollisionQueryRange;
	float PathOptimizationRange;

	/** multiplier for avoidance samples during detection, doesn't affect actual velocity */
	float AvoidanceRangeMultiplier;

	/** start index of current path part */
	int32 PathStartIndex;

	/** last visited poly on path */
	int32 LastPathPolyIndex;

	// PathFollowingComponent BEGIN
	AIMODULE_API virtual int32 DetermineStartingPathPoint(const FNavigationPath* ConsideredPath) const override;
	AIMODULE_API virtual void SetMoveSegment(int32 SegmentStartIndex) override;
	AIMODULE_API virtual void UpdatePathSegment() override;
	AIMODULE_API virtual void FollowPathSegment(float DeltaTime) override;
	AIMODULE_API virtual bool ShouldCheckPathOnResume() const override;
	AIMODULE_API virtual bool IsOnPath() const override;
	AIMODULE_API virtual bool UpdateMovementComponent(bool bForce) override;
	AIMODULE_API virtual void Reset() override;
	AIMODULE_API virtual bool IsFollowingNavLink() const override;
	// PathFollowingComponent END

	AIMODULE_API void SwitchToNextPathPart();
	AIMODULE_API bool ShouldSwitchPathPart(int32 CorridorSize) const;
	AIMODULE_API bool HasMovedDuringPause() const;
	AIMODULE_API void UpdateCachedDirections(const FVector& NewVelocity, const FVector& NextPathCorner, bool bTraversingLink);

	AIMODULE_API virtual bool ShouldTrackMovingGoal(FVector& OutGoalLocation) const;
	
	AIMODULE_API void RegisterCrowdAgent();

	friend UCrowdManager;
};
