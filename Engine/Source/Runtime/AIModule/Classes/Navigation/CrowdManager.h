// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CrowdManagerBase.h"
#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "AI/Navigation/NavigationTypes.h"
#include "Tickable.h"
#include "DrawDebugHelpers.h"
#include "CrowdManager.generated.h"

class ANavigationData;
class dtCrowd;
class dtObstacleAvoidanceDebugData;
class ICrowdAgentInterface;
class UCrowdFollowingComponent;
class UCrowdManager;
struct dtCrowdAgent;
struct dtCrowdAgentDebugInfo;
struct dtCrowdAgentParams;
struct FNavMeshPath;

#if WITH_RECAST
struct dtQuerySpecialLinkFilter;
struct dtCrowdAgentParams;
class dtCrowd;
struct dtCrowdAgent;
struct dtCrowdAgentDebugInfo;
class dtObstacleAvoidanceDebugData;
#endif

struct FNavigationQueryFilter;
typedef TSharedPtr<const FNavigationQueryFilter, ESPMode::ThreadSafe> FSharedConstNavQueryFilter;

/**
 *  Crowd manager is responsible for handling crowds using Detour (Recast library)
 *
 *  Agents will respect navmesh for all steering and avoidance updates, 
 *  but it's slower than AvoidanceManager solution (RVO, cares only about agents)
 *
 *  All agents will operate on the same navmesh data, which will be picked from
 *  navigation system defaults (UNavigationSystemV1::SupportedAgents[0])
 *
 *  To use it, you have to add CrowdFollowingComponent to your agent
 *  (usually: replace class of PathFollowingComponent in AIController by adding 
 *   those lines in controller's constructor
 *
 *   ACrowdAIController::ACrowdAIController(const FObjectInitializer& ObjectInitializer)
 *       : Super(ObjectInitializer.SetDefaultSubobjectClass<UCrowdFollowingComponent>(TEXT("PathFollowingComponent")))
 *
 *   or simply add both components and switch move requests between them)
 *
 *  Actors that should be avoided, but are not being simulated by crowd (like players)
 *  should implement CrowdAgentInterface AND register/unregister themselves with crowd manager:
 *  
 *   UCrowdManager* CrowdManager = UCrowdManager::GetCurrent(this);
 *   if (CrowdManager)
 *   {
 *      CrowdManager->RegisterAgent(this);
 *   }
 *
 *   Check flags in FCrowdDebug namespace (CrowdManager.cpp) for debugging options.
 */

USTRUCT()
struct FCrowdAvoidanceConfig
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=Crowd)
	float VelocityBias;

	UPROPERTY(EditAnywhere, Category=Crowd)
	float DesiredVelocityWeight;
	
	UPROPERTY(EditAnywhere, Category=Crowd)
	float CurrentVelocityWeight;
	
	UPROPERTY(EditAnywhere, Category=Crowd)
	float SideBiasWeight;
	
	UPROPERTY(EditAnywhere, Category=Crowd)
	float ImpactTimeWeight;

	UPROPERTY(EditAnywhere, Category=Crowd)
	float ImpactTimeRange;

	/** Threshold for side selection, larger values widen the dead zone for head-on encounters */
	UPROPERTY(EditAnywhere, Category=Crowd, meta=(ClampMin="0.001", ClampMax="1.0"))
	float SideBiasThreshold;

	/** Weight for preferring right-side passing, reduces oscillation (0 = disabled) */
	UPROPERTY(EditAnywhere, Category=Crowd, meta=(ClampMin="0.0"))
	float RightSideBias;

	// index in SamplingPatterns array or 0xff for adaptive sampling
	UPROPERTY(EditAnywhere, Category=Crowd)
	uint8 CustomPatternIdx;

	// adaptive sampling: number of divisions per ring
	UPROPERTY(EditAnywhere, Category=Crowd)
	uint8 AdaptiveDivisions;

	// adaptive sampling: number of rings
	UPROPERTY(EditAnywhere, Category=Crowd)
	uint8 AdaptiveRings;
	
	// adaptive sampling: number of iterations at best velocity
	UPROPERTY(EditAnywhere, Category=Crowd)
	uint8 AdaptiveDepth;

	FCrowdAvoidanceConfig() :
		VelocityBias(0.4f), DesiredVelocityWeight(2.0f), CurrentVelocityWeight(0.75f),
		SideBiasWeight(0.75f), ImpactTimeWeight(2.5f), ImpactTimeRange(2.5f),
		SideBiasThreshold(0.01f), RightSideBias(0.0f),
		CustomPatternIdx(0xff), AdaptiveDivisions(7), AdaptiveRings(2), AdaptiveDepth(5)
	{}
};

USTRUCT()
struct FCrowdAvoidanceSamplingPattern
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=Crowd)
	TArray<float> Angles;

	UPROPERTY(EditAnywhere, Category = Crowd)
	TArray<float> Radii;

	AIMODULE_API void AddSample(float AngleInDegrees, float NormalizedRadius);
	AIMODULE_API void AddSampleWithMirror(float AngleInDegrees, float NormalizedRadius);
};

struct FCrowdAgentData
{
#if WITH_RECAST
	/** special filter for checking offmesh links */
	TSharedPtr<dtQuerySpecialLinkFilter> LinkFilter;
#endif

	/** poly ref that agent is standing on from previous update */
	NavNodeRef PrevPoly;

	/** index of agent in detour crowd */
	int32 AgentIndex;

	/** remaining time for next path optimization */
	float PathOptRemainingTime;

	/** is this agent fully simulated by crowd? */
	uint32 bIsSimulated : 1;

	/** if set, agent wants path optimizations */
	uint32 bWantsPathOptimization : 1;

	FCrowdAgentData() :	PrevPoly(0), AgentIndex(-1), PathOptRemainingTime(0), bIsSimulated(false), bWantsPathOptimization(false) {}

	bool IsValid() const { return AgentIndex >= 0; }
	AIMODULE_API void ClearFilter();
};

struct FCrowdTickHelper : FTickableGameObject
{
	TWeakObjectPtr<UCrowdManager> Owner;

	FCrowdTickHelper() : Owner(NULL) {}
	virtual void Tick(float DeltaTime);
	virtual bool IsTickable() const { return Owner.IsValid(); }
	virtual bool IsTickableInEditor() const { return true; }
	virtual bool IsTickableWhenPaused() const { return true; }
	virtual TStatId GetStatId() const;
};

UCLASS(config = Engine, defaultconfig, MinimalAPI)
class UCrowdManager : public UCrowdManagerBase
{
	GENERATED_BODY()

public:
	AIMODULE_API UCrowdManager(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	AIMODULE_API virtual void Tick(float DeltaTime) override;
	AIMODULE_API virtual void BeginDestroy() override;

	/** adds new agent to crowd */
	AIMODULE_API void RegisterAgent(ICrowdAgentInterface* Agent);

	/** removes agent from crowd */
	AIMODULE_API void UnregisterAgent(const ICrowdAgentInterface* Agent);

	/** updates agent data */
	AIMODULE_API void UpdateAgentParams(const ICrowdAgentInterface* Agent) const;

	/** refresh agent state */
	AIMODULE_API void UpdateAgentState(const ICrowdAgentInterface* Agent) const;

	/** update agent after using custom link */
	AIMODULE_API void OnAgentFinishedCustomLink(const ICrowdAgentInterface* Agent) const;

	/** sets move target for crowd agent (only for fully simulated) */
	AIMODULE_API bool SetAgentMoveTarget(const UCrowdFollowingComponent* AgentComponent, const FVector& MoveTarget, FSharedConstNavQueryFilter Filter) const;

	/** sets move direction for crowd agent (only for fully simulated) */
	AIMODULE_API bool SetAgentMoveDirection(const UCrowdFollowingComponent* AgentComponent, const FVector& MoveDirection) const;

	/** sets move target using path (only for fully simulated) */
	AIMODULE_API bool SetAgentMovePath(const UCrowdFollowingComponent* AgentComponent, const FNavMeshPath* Path, int32 PathSectionStart, int32 PathSectionEnd, const FVector& PathSectionEndLocation) const;

	/** clears move target for crowd agent (only for fully simulated) */
	AIMODULE_API void ClearAgentMoveTarget(const UCrowdFollowingComponent* AgentComponent) const;

	/** switch agent to waiting state */
	AIMODULE_API void PauseAgent(const UCrowdFollowingComponent* AgentComponent) const;

	/** resumes agent movement */
	AIMODULE_API void ResumeAgent(const UCrowdFollowingComponent* AgentComponent, bool bForceReplanPath = true) const;

	/** check if object is a valid crowd agent */
	AIMODULE_API bool IsAgentValid(const UCrowdFollowingComponent* AgentComponent) const;
	AIMODULE_API bool IsAgentValid(const ICrowdAgentInterface* Agent) const;

	/** returns number of nearby agents */
	AIMODULE_API int32 GetNumNearbyAgents(const ICrowdAgentInterface* Agent) const;

	/** returns a list of locations of nearby agents */
	AIMODULE_API int32 GetNearbyAgentLocations(const ICrowdAgentInterface* Agent, TArray<FVector>& OutLocations) const;

	/** reads existing avoidance config or returns false */
	AIMODULE_API bool GetAvoidanceConfig(int32 Idx, FCrowdAvoidanceConfig& Data) const;

	/** updates existing avoidance config or returns false */
	AIMODULE_API bool SetAvoidanceConfig(int32 Idx, const FCrowdAvoidanceConfig& Data);

	/** remove started offmesh connections from corridor */
	AIMODULE_API void SetOffmeshConnectionPruning(bool bRemoveFromCorridor);

	/** block path visibility raycasts when crossing different nav areas */
	AIMODULE_API void SetSingleAreaVisibilityOptimization(bool bEnable);

	/** adjust current position in path's corridor, starting test from PathStartIdx */
	AIMODULE_API void AdjustAgentPathStart(const UCrowdFollowingComponent* AgentComponent, const FNavMeshPath* Path, int32& PathStartIdx) const;

#if WITH_EDITOR
	AIMODULE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

	AIMODULE_API void DebugTick() const;
#endif

	/** notify called when detour navmesh is changed */
	AIMODULE_API void OnNavMeshUpdate();

	/** Tests if NavData is a suitable nav data type to be used by this CrowdManager
	 *	instance. */
	AIMODULE_API virtual bool IsSuitableNavData(const ANavigationData& NavData) const;

	/** Called by the nav system when a new navigation data instance is registered. 
	 *	If the CrowdManager instance had no nav data cached it will consider this
	 *	NavDataInstance and update if necesary. */
	AIMODULE_API virtual void OnNavDataRegistered(ANavigationData& NavData) override;

	/** Called by the nav system when a navigation data instance is removed. The 
	 *	crowd manager will see if it's the nav data being used by it an if so try
	 *	to find another one. If there's none the crowd manager will stop working. */
	AIMODULE_API virtual void OnNavDataUnregistered(ANavigationData& NavData) override;

	virtual void CleanUp(float DeltaTime) override {};

	const ANavigationData* GetNavData() const { return MyNavData; }

	AIMODULE_API UWorld* GetWorld() const override;

	static AIMODULE_API UCrowdManager* GetCurrent(UObject* WorldContextObject);
	static AIMODULE_API UCrowdManager* GetCurrent(UWorld* World);

protected:

	UPROPERTY(transient)
	TObjectPtr<ANavigationData> MyNavData;

	/** obstacle avoidance params */
	UPROPERTY(config, EditAnywhere, Category = Config)
	TArray<FCrowdAvoidanceConfig> AvoidanceConfig;

	/** obstacle avoidance params */
	UPROPERTY(config, EditAnywhere, Category = Config)
	TArray<FCrowdAvoidanceSamplingPattern> SamplingPatterns;

	/** max number of agents supported by crowd */
	UPROPERTY(config, EditAnywhere, Category = Config)
	int32 MaxAgents;

	/** max radius of agent that can be added to crowd */
	UPROPERTY(config, EditAnywhere, Category = Config)
	float MaxAgentRadius;

	/** max number of neighbor agents for velocity avoidance */
	UPROPERTY(config, EditAnywhere, Category = Config)
	int32 MaxAvoidedAgents;

	/** max number of wall segments for velocity avoidance */
	UPROPERTY(config, EditAnywhere, Category = Config)
	int32 MaxAvoidedWalls;

	/** how often should agents check their position after moving off navmesh? */
	UPROPERTY(config, EditAnywhere, Category = Config)
	float NavmeshCheckInterval;

	/** how often should agents try to optimize their paths? */
	UPROPERTY(config, EditAnywhere, Category = Config)
	float PathOptimizationInterval;

	/** clamp separation force to left/right when neighbor is behind (dot between forward and dirToNei, -1 = disabled) */
	UPROPERTY(config, EditAnywhere, Category = Config)
	float SeparationDirClamp;

	/** agent radius multiplier for offsetting path around corners */
	UPROPERTY(config, EditAnywhere, Category = Config)
	float PathOffsetRadiusMultiplier;

	uint32 bPruneStartedOffmeshConnections : 1;
	uint32 bSingleAreaVisibilityOptimization : 1;
	uint32 bEarlyReachTestOptimization : 1;
	uint32 bAllowPathReplan : 1;
	
	/** should crowd simulation resolve collisions between agents? if not, this will be handled by their movement components */
	UPROPERTY(config, EditAnywhere, Category = Config)
	uint32 bResolveCollisions : 1;

	/** agents registered in crowd manager */
	TMap<ICrowdAgentInterface*, FCrowdAgentData> ActiveAgents;

	/** temporary flags for crowd agents */
	TArray<uint8> AgentFlags;

#if WITH_RECAST
	/** crowd manager */
	dtCrowd* DetourCrowd;

	/** debug data */
	dtCrowdAgentDebugInfo* DetourAgentDebug;
	dtObstacleAvoidanceDebugData* DetourAvoidanceDebug;
#endif

#if WITH_EDITOR
	FCrowdTickHelper* TickHelper;
#endif // WITH_EDITOR

	/** try to initialize nav data from already existing ones */
	AIMODULE_API virtual void UpdateNavData();

	/** setup params of crowd avoidance */
	AIMODULE_API virtual void UpdateAvoidanceConfig();

	/** called from tick, just after updating agents proximity data */
	AIMODULE_API virtual void PostProximityUpdate();

	/** called from tick, after move points were updated, before any steering/avoidance */
	AIMODULE_API virtual void PostMovePointUpdate();

	/** Sets NavData as MyNavData. If Null and bFindNewNavDataIfNull is true then
	 *	the manager will search for a new NavData instance that meets the 
	 *	IsSuitableNavData() condition. */
	AIMODULE_API void SetNavData(ANavigationData* NavData, const bool bFindNewNavDataIfNull = true);

#if WITH_RECAST
	AIMODULE_API void AddAgent(const ICrowdAgentInterface* Agent, FCrowdAgentData& AgentData) const;
	AIMODULE_API void RemoveAgent(const ICrowdAgentInterface* Agent, FCrowdAgentData* AgentData) const;
	AIMODULE_API void GetAgentParams(const ICrowdAgentInterface* Agent, dtCrowdAgentParams& AgentParams) const;

	/** prepare agent for next step of simulation */
	AIMODULE_API void PrepareAgentStep(const ICrowdAgentInterface* Agent, FCrowdAgentData& AgentData, float DeltaTime) const;

	/** pass new velocity to movement components */
	AIMODULE_API virtual void ApplyVelocity(UCrowdFollowingComponent* AgentComponent, int32 AgentIndex) const;

	/** check changes in crowd simulation and adjust UE specific properties (smart links, poly updates) */
	AIMODULE_API void UpdateAgentPaths();

	/** switch debugger to object selected in PIE */
	AIMODULE_API void UpdateSelectedDebug(const ICrowdAgentInterface* Agent, int32 AgentIndex) const;

	AIMODULE_API void CreateCrowdManager();
	AIMODULE_API void DestroyCrowdManager();

#if ENABLE_DRAW_DEBUG
	AIMODULE_API UWorld* GetDebugDrawingWorld() const;
	AIMODULE_API void DrawDebugCorners(const dtCrowdAgent* CrowdAgent) const;
	AIMODULE_API void DrawDebugCollisionSegments(const dtCrowdAgent* CrowdAgent) const;
	AIMODULE_API void DrawDebugPath(const dtCrowdAgent* CrowdAgent) const;
	AIMODULE_API void DrawDebugVelocityObstacles(const dtCrowdAgent* CrowdAgent) const;
	AIMODULE_API void DrawDebugPathOptimization(const dtCrowdAgent* CrowdAgent) const;
	AIMODULE_API void DrawDebugNeighbors(const dtCrowdAgent* CrowdAgent) const;
	AIMODULE_API void DrawDebugSharedBoundary() const;
#endif // ENABLE_DRAW_DEBUG

#endif
};
