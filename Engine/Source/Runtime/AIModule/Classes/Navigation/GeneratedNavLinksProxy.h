// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseGeneratedNavLinksProxy.h"
#include "GeneratedNavLinksProxy.generated.h"

class AActor;
class UObject;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FLinkReachedSignature, AActor*, MovingActor, const FVector, DestinationPoint);

/**
 * Experimental
 * Blueprintable class used to handle generated links as custom links.
 */
UCLASS(Blueprintable, BlueprintType, MinimalAPI)
class UGeneratedNavLinksProxy : public UBaseGeneratedNavLinksProxy
{
	GENERATED_UCLASS_BODY()

	virtual UWorld* GetWorld() const override;

	// BEGIN INavLinkCustomInterface
	AIMODULE_API virtual bool OnLinkMoveStarted(class UObject* PathComp, const FVector& DestPoint) override;
	// END INavLinkCustomInterface

	//////////////////////////////////////////////////////////////////////////
	// Blueprint interface for smart links
	
	/** Called when agent reaches smart link during path following. */
	UFUNCTION(BlueprintImplementableEvent)
	AIMODULE_API void ReceiveSmartLinkReached(AActor* Agent, const FVector Destination);

protected:
	void NotifySmartLinkReached(UObject* PathingAgent, const FVector DestPoint);
	
	UPROPERTY(BlueprintAssignable)
	FLinkReachedSignature OnSmartLinkReached;
};
