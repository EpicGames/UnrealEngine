// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Navigation/PathFollowingComponent.h"
#include "GridPathFollowingComponent.generated.h"

class UNavLocalGridManager;

/**
 *  Path following augmented with local navigation grids
 *
 *  Keeps track of nearby grids and use them instead of navigation path when agent is inside.
 *  Once outside grid, regular path following is resumed.
 *
 *  This allows creating dynamic navigation obstacles with fully static navigation (e.g. static navmesh),
 *  as long as they are minor modifications for path. Not recommended for blocking off entire corridors.
 *
 *  Does not replace proper avoidance for dynamic obstacles!
 */
UCLASS(BlueprintType, Experimental, MinimalAPI)
class UGridPathFollowingComponent : public UPathFollowingComponent
{
	GENERATED_UCLASS_BODY()
public:

	AIMODULE_API virtual void Initialize() override;
	AIMODULE_API virtual void UpdatePathSegment() override;
	AIMODULE_API virtual void Reset() override;
	AIMODULE_API virtual void ResumeMove(FAIRequestID RequestID = FAIRequestID::CurrentRequest) override;
	AIMODULE_API virtual void OnPathUpdated();

	bool HasActiveGrid() const { return ActiveGridIdx != INDEX_NONE; }
	int32 GetActiveGridIdx() const { return ActiveGridIdx; }

	const TArray<FVector>& GetGridPathPoints() const { return GridPathPoints; }
	int32 GetNextGridPathIndex() const { return GridMoveSegmentEndIndex; }

protected:

	AIMODULE_API void UpdateActiveGrid(const FVector& CurrentLocation);

	UPROPERTY(Transient)
	TObjectPtr<UNavLocalGridManager> GridManager;

	/** index of current grid */
	int32 ActiveGridIdx;

	/** Id of current grid */
	int32 ActiveGridId;

	/** set when end of followed Path ends inside current grid */
	uint32 bIsPathEndInsideGrid : 1;

	/** set when grid path is valid */
	uint32 bHasGridPath : 1;

	/** path points for moving through grid */
	TArray<FVector> GridPathPoints;
	
	/** index of current destination grid path point */
	int32 GridMoveSegmentEndIndex;

	/** expected start of path segment after leaving grid */
	int32 MoveSegmentStartIndexOffGrid;
};
