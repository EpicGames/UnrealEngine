// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "AI/Navigation/NavigationTypes.h"
#include "NavMesh/RecastNavMesh.h"

class AController;
class UCanvas;
struct FVisualLogEntry;

enum class EMetaPathUpdateReason : uint8 
{
	PathFinished,
	MoveTick,
};

struct FMetaPathWayPoint : public FVector
{
	uint32 UserFlags;

	FMetaPathWayPoint()
		: FVector()
		, UserFlags(0)
	{}

	FMetaPathWayPoint(const FVector& Location, const uint32 InUserFlags = 0)
		: FVector(Location)
		, UserFlags(InUserFlags)
	{}
};

/** 
 * FMetaNavMeshPath allows creating hierarchical or guided navmesh paths
 *  
 * Waypoints array defines list of locations that will be used to create actual FNavMeshPath data during path following.
 * On path following start and upon reaching waypoint, path will request update from owning navmesh with following parameters:
 * - start location set to agent location
 * - end location set to target waypoint (or goal actor for last one)
 * - goal actor set only for last section
 *
 * Since path updates itself for moving agent it really shouldn't be reused by others.
 */
struct FMetaNavMeshPath : public FNavMeshPath
{
	typedef FNavMeshPath Super;
	static AIMODULE_API const FNavPathType Type;

	AIMODULE_API FMetaNavMeshPath();
	AIMODULE_API FMetaNavMeshPath(const TArray<FMetaPathWayPoint>& InWaypoints, const ANavigationData& NavData);
	AIMODULE_API FMetaNavMeshPath(const TArray<FMetaPathWayPoint>& InWaypoints, const AController& Owner);
	AIMODULE_API FMetaNavMeshPath(const TArray<FVector>& InWaypoints, const ANavigationData& NavData);
	AIMODULE_API FMetaNavMeshPath(const TArray<FVector>& InWaypoints, const AController& Owner);

	/** initialize path for path following */
	AIMODULE_API virtual void Initialize(const FVector& AgentLocation);

	/** try switching to next waypoint, depends on WaypointSwitchRadius */
	AIMODULE_API virtual bool ConditionalMoveToNextSection(const FVector& AgentLocation, EMetaPathUpdateReason Reason);

	/** force switching to next waypoint */
	AIMODULE_API bool ForceMoveToNextSection(const FVector& AgentLocation);

	/** updates underlying navmesh path for current target waypoint */
	AIMODULE_API virtual bool UpdatePath(const FVector& AgentLocation);

	/** copy properties of other meta path */
	AIMODULE_API virtual void CopyFrom(const FMetaNavMeshPath& Other);

	/** returns true if path at last waypoint */
	bool IsLastSection() const { return (TargetWaypointIdx == (Waypoints.Num() - 1)) && (Waypoints.Num() > 0); }

	/** returns index of current target waypoint */
	int32 GetTargetWaypointIndex() const { return TargetWaypointIdx; }

	/** returns number of waypoints */
	int32 GetNumWaypoints() const { return Waypoints.Num(); }

	/** returns waypoint array */
	const TArray<FMetaPathWayPoint>& GetWaypointArray() const { return Waypoints; }
	
	/** returns cached path goal */
	AActor* GetMetaPathGoal() const { return PathGoal.Get(); }

	/** tries to set waypoints, fails when path is ready being followed */
	AIMODULE_API bool SetWaypoints(const TArray<FMetaPathWayPoint>& InWaypoints);

	/** tries to set waypoints, fails when path is ready being followed */
	AIMODULE_API bool SetWaypoints(const TArray<FVector>& InWaypoints);
	
	/** returns radius for switching to next waypoint during path following */
	float GetWaypointSwitchRadius() const { return WaypointSwitchRadius; }

	/** sets radius for switching to next waypoint during path following */
	void SetWaypointSwitchRadius(float InSwitchRadius) { WaypointSwitchRadius = InSwitchRadius; }

	/** returns approximate length of path, ignores parameters */
	AIMODULE_API virtual FVector::FReal GetLengthFromPosition(FVector SegmentStart, uint32 NextPathPointIndex) const override;
	
	/** returns approximate cost of path, ignores parameter */
	AIMODULE_API virtual FVector::FReal GetCostFromIndex(int32 PathPointIndex) const override;

#if ENABLE_VISUAL_LOG
	AIMODULE_API virtual void DescribeSelfToVisLog(FVisualLogEntry* Snapshot) const override;
#endif
	AIMODULE_API virtual void DebugDraw(const ANavigationData* NavData, const FColor PathColor, UCanvas* Canvas, const bool bPersistent, const float LifeTime, const uint32 NextPathPointIndex = 0) const override;

protected:

	/** list of waypoints, including start and end of path */
	TArray<FMetaPathWayPoint> Waypoints;

	/** sum of 3D distance along waypoints, used for approximating length of path */
	FVector::FReal ApproximateLength;

	/** update navmesh path when this close to target waypoint */
	float WaypointSwitchRadius;

	/** current target of path following */
	int32 TargetWaypointIdx;

	/** stored goal actor */
	TWeakObjectPtr<AActor> PathGoal;

	/** stored goal actor's tether distance */
	float PathGoalTetherDistance;

	/** switch to next waypoint  */
	AIMODULE_API bool MoveToNextSection(const FVector& AgentLocation);
};
