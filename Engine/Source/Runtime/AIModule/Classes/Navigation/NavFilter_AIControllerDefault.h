// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Templates/SubclassOf.h"
#include "NavFilters/NavigationQueryFilter.h"
#include "NavFilter_AIControllerDefault.generated.h"

UCLASS(MinimalAPI)
class UNavFilter_AIControllerDefault : public UNavigationQueryFilter
{
	GENERATED_BODY()
public:
	AIMODULE_API UNavFilter_AIControllerDefault(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	AIMODULE_API virtual TSubclassOf<UNavigationQueryFilter> GetSimpleFilterForAgent(const UObject& Querier) const;
};
