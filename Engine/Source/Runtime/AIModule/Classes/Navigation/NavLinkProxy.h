// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "GameFramework/Actor.h"
#include "EngineDefines.h"
#include "AI/Navigation/NavRelevantInterface.h"
#include "AI/Navigation/NavLinkDefinition.h"
#include "NavLinkHostInterface.h"
#include "NavLinkProxy.generated.h"

class UBillboardComponent;
class UNavLinkCustomComponent;
class UNavLinkRenderingComponent;
class UPathFollowingComponent;
struct FNavigationRelevantData;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams( FSmartLinkReachedSignature, AActor*, MovingActor, const FVector&, DestinationPoint );

/** 
 *  ANavLinkProxy connects areas of Navmesh that don't have a direct navigation path.
 *  It directly supports Simple Links (see PointLinks array). There can be multiple Simple links per ANavLinkProxy instance.
 *  Simple links are designed to statically link areas of Navmesh and are associated with a particular area class that the link provides.
 *  Smart Link functionality is provided via UNavLinkCustomComponent, see SmartLinkComp. They are designed to be able to be dynamically toggled
 *  between enabled and disabled and provide different area classes for both cases. The area classes can be dynamically modified 
 *  without navmesh rebuilds.
 *  There can only be at most one smart link per ANavLinkProxy instance.
 *  Both simple and smart links on a single ANavLinkProxy instance, can be set / enabled at once, as well as either or neither of them.
 */
UCLASS(Blueprintable, autoCollapseCategories=(SmartLink, Actor), hideCategories=(Input), MinimalAPI)
class ANavLinkProxy : public AActor, public INavLinkHostInterface, public INavRelevantInterface
{
	GENERATED_UCLASS_BODY()

	/** Navigation links (point to point) added to navigation data */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=SimpleLink)
	TArray<FNavigationLink> PointLinks;
	
	/** Navigation links (segment to segment) added to navigation data
	*	@todo hidden from use until we fix segment links. Not really working now*/
	UPROPERTY()
	TArray<FNavigationSegmentLink> SegmentLinks;

private:
	/** Smart link: can affect path following */
	UPROPERTY(VisibleAnywhere, Category=SmartLink)
	TObjectPtr<UNavLinkCustomComponent> SmartLinkComp;
public:

	/** Smart link: toggle relevancy */
	UPROPERTY(EditAnywhere, Category=SmartLink)
	bool bSmartLinkIsRelevant;

#if WITH_EDITORONLY_DATA
private:
	/** Editor Preview */
	UPROPERTY()
	TObjectPtr<UNavLinkRenderingComponent> EdRenderComp;

	UPROPERTY()
	TObjectPtr<UBillboardComponent> SpriteComponent;
public:
#endif // WITH_EDITORONLY_DATA

	// BEGIN INavRelevantInterface
	AIMODULE_API virtual void GetNavigationData(FNavigationRelevantData& Data) const override;
	AIMODULE_API virtual FBox GetNavigationBounds() const override;
	AIMODULE_API virtual bool IsNavigationRelevant() const override;
	// END INavRelevantInterface

	// BEGIN INavLinkHostInterface
	AIMODULE_API virtual bool GetNavigationLinksClasses(TArray<TSubclassOf<UNavLinkDefinition> >& OutClasses) const override;
	AIMODULE_API virtual bool GetNavigationLinksArray(TArray<FNavigationLink>& OutLink, TArray<FNavigationSegmentLink>& OutSegments) const override;
	// END INavLinkHostInterface

#if WITH_EDITOR
	AIMODULE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	AIMODULE_API virtual void PostEditUndo() override;
	AIMODULE_API virtual void PostEditImport() override;
#endif // WITH_EDITOR
	AIMODULE_API virtual void PostInitProperties() override;
	AIMODULE_API virtual void BeginDestroy() override;

	AIMODULE_API virtual void PostRegisterAllComponents() override;
	AIMODULE_API virtual void PostLoad() override;

#if ENABLE_VISUAL_LOG
protected:
	AIMODULE_API virtual void BeginPlay() override;
public:
#endif // ENABLE_VISUAL_LOG

	AIMODULE_API virtual FBox GetComponentsBoundingBox(bool bNonColliding = false, bool bIncludeFromChildActors = false) const override;

	//////////////////////////////////////////////////////////////////////////
	// Blueprint interface for smart links

	/** called when agent reaches smart link during path following, use ResumePathFollowing() to give control back */
	UFUNCTION(BlueprintImplementableEvent)
	AIMODULE_API void ReceiveSmartLinkReached(AActor* Agent, const FVector& Destination);

	/** resume normal path following */
	UFUNCTION(BlueprintCallable, Category="AI|Navigation")
	AIMODULE_API void ResumePathFollowing(AActor* Agent);

	/** check if smart link is enabled */
	UFUNCTION(BlueprintCallable, Category="AI|Navigation")
	AIMODULE_API bool IsSmartLinkEnabled() const;

	/** change state of smart link */
	UFUNCTION(BlueprintCallable, Category="AI|Navigation")
	AIMODULE_API void SetSmartLinkEnabled(bool bEnabled);

	/** check if any agent is moving through smart link right now */
	UFUNCTION(BlueprintCallable, Category="AI|Navigation")
	AIMODULE_API bool HasMovingAgents() const;

#if WITH_EDITOR
	/** Copies navlink end points from the first entry in PointLinks array. This 
	 *	function is a helper function making up for smart links not drawing
	 *	the FVector widgets in the editor. */
	UFUNCTION(CallInEditor, Category = SmartLink, meta = (DisplayName="CopyEndPointsFromSimpleLink"))
	AIMODULE_API void CopyEndPointsFromSimpleLinkToSmartLink();
#endif // WITH_EDITOR

protected:

	UPROPERTY(BlueprintAssignable)
	FSmartLinkReachedSignature OnSmartLinkReached;

	AIMODULE_API void NotifySmartLinkReached(UNavLinkCustomComponent* LinkComp, UObject* PathingAgent, const FVector& DestPoint);

#if WITH_EDITOR
	AIMODULE_API void OnNavAreaRegistered(const UWorld& World, const UClass* NavAreaClass);
	AIMODULE_API void OnNavAreaUnregistered(const UWorld& World, const UClass* NavAreaClass);
#endif // WITH_EDITOR

public:
	/** Returns SmartLinkComp subobject **/
	UNavLinkCustomComponent* GetSmartLinkComp() const { return SmartLinkComp; }
#if WITH_EDITORONLY_DATA
	/** Returns EdRenderComp subobject **/
	UNavLinkRenderingComponent* GetEdRenderComp() const { return EdRenderComp; }
	/** Returns SpriteComponent subobject **/
	UBillboardComponent* GetSpriteComponent() const { return SpriteComponent; }
#endif

protected:
#if WITH_EDITOR
	FDelegateHandle OnNavAreaRegisteredDelegateHandle;
	FDelegateHandle OnNavAreaUnregisteredDelegateHandle;
#endif // WITH_EDITOR
};
