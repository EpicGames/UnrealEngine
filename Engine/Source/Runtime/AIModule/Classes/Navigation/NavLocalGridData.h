// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SimpleCellGrid.h"

class ANavigationData;
struct FNavigationPath;

/**
 *  Local navigation grid - simple 2D grid used for navigation.
 * 
 *  Cell can be either free or marked as obstacle, connected with 8 neighbors (no walls in between)
 *
 *  When used as source for UNavLocalGridManager, each obstacle should define its own grid data.
 *  Corresponding category in gameplay debugger is hidden by default, please adjust project configs to change that.  
 *
 *  TODO: helpers for marking different basic shapes
 *  TODO: serialization (with maps?)
 *  TODO: FNavigationPath support?
 */
struct FNavLocalGridData : public TSimpleCellGrid<uint8, MAX_uint8>
{
	FNavLocalGridData() : GridId(0) {}
	AIMODULE_API FNavLocalGridData(const FVector& Center, float Extent2D);
	AIMODULE_API FNavLocalGridData(const FVector& Center, const FVector2D& Extent2D);
	AIMODULE_API FNavLocalGridData(const TArray<FNavLocalGridData>& SourceGrids);

	/** mark single cell as obstacle */
	AIMODULE_API void MarkPointObstacle(const FVector& Center);

	/** mark box (AABB or rotated) shape as obstacle */
	AIMODULE_API void MarkBoxObstacle(const FVector& Center, const FVector& Extent, const FQuat& Quat = FQuat::Identity);

	/** mark capsule shape as obstacle */
	AIMODULE_API void MarkCapsuleObstacle(const FVector& Center, float Radius, float HalfHeight);

	/** set height of bounds, if not set: ProjectCells will use height of default query box */
	AIMODULE_API void SetHeight(float ExtentZ);

	/** get unique Id of grid */
	const int32 GetGridId() const
	{
		return GridId; 
	}

	/** check if there's an obstacle at cell coords */
	bool HasObstacleUnsafe(int32 LocationX, int32 LocationY) const
	{
		return GetCellAtIndexUnsafe(GetCellIndexUnsafe(LocationX, LocationY)) > 0;
	}

	/** convert cell index to global world coords with origin in (0,0,0) */
	FIntVector GetGlobalCoords(int32 CellIdx) const
	{
		return FIntVector(GetCellCoordX(CellIdx) + OriginWorldCoord.X, GetCellCoordY(CellIdx) + OriginWorldCoord.Y, OriginWorldCoord.Z); 
	}

	/** convert global world coords to cell index, return -1 if outside */
	AIMODULE_API int32 GetCellIndexFromGlobalCoords2D(const FIntVector& WorldCoords) const;

	/** convert cell index to world location using projected heights */
	FVector GetProjectedCellCenter(int32 CellIdx) const 
	{
		return GetProjectedCellCenter(GetCellCoordX(CellIdx), GetCellCoordY(CellIdx));
	}

	/** convert cell coords on grid to world location using projected heights */
	FVector GetProjectedCellCenter(int32 LocationX, int32 LocationY) const
	{
		const FVector WorldCoords = GetWorldCellCenter(LocationX, LocationY);
		return FVector(WorldCoords.X, WorldCoords.Y, CellZ[GetCellIndexUnsafe(LocationX, LocationY)]);
	}

	/** creates path points from navigation path going through grid
	 *  @param SourcePath - [in] full navigation path
	 *  @param EntryLocation - [in] location of agent
	 *  @param EntrySegmentStart - [in] current move segment on path
	 *  @param PathPointsInside - [out] points inside grid
	 *  @param NextSegmentStart - [out] next move segment on path after leaving grid or -1 if path ends inside
	 */
	AIMODULE_API void FindPathForMovingAgent(const FNavigationPath& SourcePath, const FVector& EntryLocation, int32 EntrySegmentStart, TArray<FVector>& PathPointsInside, int32& NextSegmentStart) const;

	/** create path points from StartCoords to EndCoord, returns false when failed */
	AIMODULE_API bool FindPath(const FIntVector& StartCoords, const FIntVector& EndCoords, TArray<FIntVector>& PathCoords) const;

	/** project cells on navigation data and marks failed ones as obstacles */
	AIMODULE_API void ProjectCells(const ANavigationData& NavData);

	//////////////////////////////////////////////////////////////////////////
	// FGraphAStar: TGraph
	typedef int32 FNodeRef;

	int32 GetNeighbourCount(FNodeRef NodeRef) const { return 8; }
	bool IsValidRef(FNodeRef NodeRef) const { return IsValidIndex(NodeRef); }
	AIMODULE_API FNodeRef GetNeighbour(const FNodeRef NodeRef, const int32 NeiIndex) const;
	//////////////////////////////////////////////////////////////////////////

protected:
	friend class UNavLocalGridManager;
	TArray<FVector::FReal> CellZ;
	double LastAccessTime;

	/** convert PathIndices into pruned PathCoords */
	AIMODULE_API void PostProcessPath(const FIntVector& StartCoords, const FIntVector& EndCoords, const TArray<int32>& PathIndices, TArray<FIntVector>& PathCoords) const;

	/** check if line trace between local coords on grid hits any obstacles, doesn't validate coords! */
	AIMODULE_API bool IsLineObstructed(const FIntVector& StartCoords, const FIntVector& EndCoords) const;

	/** set unique Id of grid */
	AIMODULE_API void SetGridId(int32 NewId);

private:
	int32 GridId;
	FIntVector OriginWorldCoord;
};
