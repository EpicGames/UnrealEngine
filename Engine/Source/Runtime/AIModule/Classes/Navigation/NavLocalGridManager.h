// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "NavLocalGridData.h"
#include "NavLocalGridManager.generated.h"

struct FCombinedNavGridData : public FNavLocalGridData
{
	TArray<int32> SourceIds;

	FCombinedNavGridData() {}
	FCombinedNavGridData(const TArray<FNavLocalGridData>& SourceGrids) : FNavLocalGridData(SourceGrids) {}
};

/**
 *  Manager for local navigation grids
 * 
 *  Builds non overlapping grid from multiple sources, that can be used later for pathfinding.
 *  Check also: UGridPathFollowingComponent, FNavLocalGridData
 */

UCLASS(Experimental, MinimalAPI)
class UNavLocalGridManager : public UObject
{
	GENERATED_UCLASS_BODY()
public:

	/** adds new grid */
	AIMODULE_API int32 AddGridData(const FNavLocalGridData& GridData, bool bUpdate = true);

	/** removes grid */
	AIMODULE_API void RemoveGridData(int32 GridId, bool bUpdate = true);

	/** rebuild overlapping grids if needed */
	AIMODULE_API void RebuildGrids();

	/** tries to find a path using grids, returns false when start and end locations are not on the same grid */
	AIMODULE_API bool FindPath(const FVector& Start, const FVector& End, TArray<FVector>& PathPoints) const;

	/** set shared size of grid cell, allowed only when there's no grid added */
	AIMODULE_API bool SetCellSize(float CellSize);

	/** get shared size of grid cell, static but there's only one active manager */
	static float GetCellSize() { return GridCellSize; }

	/** set limit of source grids, 0 or negative means unlimited */
	AIMODULE_API void SetMaxActiveSources(int32 NumActiveSources);

	/** get limit of source grids */
	int32 GetMaxActiveSources() const { return MaxActiveSourceGrids; }

	/** check if source grid limit is set */
	bool HasSourceGridLimit() const { return MaxActiveSourceGrids > 0; }

	/** updates LastAccessTime in all source grids */
	AIMODULE_API void UpdateAccessTime(int32 CombinedGridIdx);

	/** get number of known source grids */
	int32 GetNumSources() const { return SourceGrids.Num(); }

	/** get number of combined, non overlapping grids */
	int32 GetNumGrids() const { return CombinedGrids.Num(); }

	/** get source grid by index */
	const FNavLocalGridData& GetSourceData(int32 SourceIdx) const { return SourceGrids[SourceIdx]; }

	/** get combined, non overlapping grid by index */
	const FNavLocalGridData& GetGridData(int32 GridIdx) const { return CombinedGrids[GridIdx]; }

	/** find combined grid value at world location */
	AIMODULE_API uint8 GetGridValueAt(const FVector& WorldLocation) const;

	/** find combined grid at location */
	AIMODULE_API int32 GetGridIndex(const FVector& WorldLocation) const;

	/** get version of grid data, incremented with each rebuild */
	int32 GetVersion() const { return VersionNum; }

	UFUNCTION(BlueprintCallable, Category = "AI|Navigation", meta = (WorldContext = "WorldContextObject"))
	static AIMODULE_API bool SetLocalNavigationGridDensity(UObject* WorldContextObject, float CellSize);

	/** creates new grid data for single point */
	UFUNCTION(BlueprintCallable, Category = "AI|Navigation", meta = (WorldContext = "WorldContextObject"))
	static AIMODULE_API int32 AddLocalNavigationGridForPoint(UObject* WorldContextObject, const FVector& Location, const int32 Radius2D = 5, const float Height = 100.0f, bool bRebuildGrids = true);

	/** creates single grid data for set of points */
	UFUNCTION(BlueprintCallable, Category = "AI|Navigation", meta = (WorldContext = "WorldContextObject"))
	static AIMODULE_API int32 AddLocalNavigationGridForPoints(UObject* WorldContextObject, const TArray<FVector>& Locations, const int32 Radius2D = 5, const float Height = 100.0f, bool bRebuildGrids = true);

	UFUNCTION(BlueprintCallable, Category = "AI|Navigation", meta = (WorldContext = "WorldContextObject"))
	static AIMODULE_API int32 AddLocalNavigationGridForBox(UObject* WorldContextObject, const FVector& Location, FVector Extent = FVector(1,1,1), FRotator Rotation = FRotator::ZeroRotator, const int32 Radius2D = 5, const float Height = 100.0f, bool bRebuildGrids = true);

	UFUNCTION(BlueprintCallable, Category = "AI|Navigation", meta = (WorldContext = "WorldContextObject"))
	static AIMODULE_API int32 AddLocalNavigationGridForCapsule(UObject* WorldContextObject, const FVector& Location, float CapsuleRadius, float CapsuleHalfHeight, const int32 Radius2D = 5, const float Height = 100.0f, bool bRebuildGrids = true);

	UFUNCTION(BlueprintCallable, Category = "AI|Navigation", meta = (WorldContext = "WorldContextObject"))
	static AIMODULE_API void RemoveLocalNavigationGrid(UObject* WorldContextObject, int32 GridId, bool bRebuildGrids = true);

	UFUNCTION(BlueprintCallable, Category = "AI|Navigation", meta = (WorldContext = "WorldContextObject"))
	static AIMODULE_API bool FindLocalNavigationGridPath(UObject* WorldContextObject, const FVector& Start, const FVector& End, TArray<FVector>& PathPoints);

	static AIMODULE_API UNavLocalGridManager* GetCurrent(UWorld* World);
	static AIMODULE_API UNavLocalGridManager* GetCurrent(const UObject* WorldContextObject);

#if WITH_ENGINE
	AIMODULE_API virtual UWorld* GetWorld() const override;
#endif

protected:

	TArray<FCombinedNavGridData> CombinedGrids;
	TArray<FNavLocalGridData> SourceGrids;
	
	static AIMODULE_API float GridCellSize;

	int32 VersionNum;
	int32 NextGridId;
	int32 MaxActiveSourceGrids;
	uint32 bNeedsRebuilds : 1;

	/** projects combined grids to navigation data */
	AIMODULE_API virtual void ProjectGrids(const TArray<int32>& GridIndices);

	/** ensures limit of source grids, removing oldest entries (LastAccessTime) */
	AIMODULE_API bool UpdateSourceGrids();
};
