// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtr.h"
#include "Engine/EngineTypes.h"
#include "Components/ActorComponent.h"
#include "EngineDefines.h"
#include "AI/Navigation/NavigationTypes.h"
#include "NavigationData.h"
#include "AITypes.h"
#include "AIResourceInterface.h"
#include "GameFramework/NavMovementComponent.h"
#include "AI/Navigation/PathFollowingAgentInterface.h"
#include "PathFollowingComponent.generated.h"

class Error;
class FDebugDisplayInfo;
class INavLinkCustomInterface;
class UCanvas;
class ANavigationData;

AIMODULE_API DECLARE_LOG_CATEGORY_EXTERN(LogPathFollowing, Warning, All);

class UCanvas;
class AActor;
class INavLinkCustomInterface;
class INavAgentInterface;
class UNavigationComponent;

UENUM(BlueprintType)
namespace EPathFollowingStatus
{
	enum Type : int
	{
		/** No requests */
		Idle,

		/** Request with incomplete path, will start after UpdateMove() */
		Waiting,

		/** Request paused, will continue after ResumeMove() */
		Paused,

		/** Following path */
		Moving,
	};
}

UENUM(BlueprintType)
namespace EPathFollowingResult
{
	enum Type : int
	{
		/** Reached destination */
		Success,

		/** Movement was blocked */
		Blocked,

		/** Agent is not on path */
		OffPath,

		/** Aborted and stopped (failure) */
		Aborted,

		/** DEPRECATED, use Aborted result instead */
		Skipped_DEPRECATED UMETA(Hidden),

		/** Request was invalid */
		Invalid,
	};
}

namespace FPathFollowingResultFlags
{
	typedef uint16 Type;

	inline const Type None = 0;

	/** Reached destination (EPathFollowingResult::Success) */
	inline const Type Success = (1 << 0);

	/** Movement was blocked (EPathFollowingResult::Blocked) */
	inline const Type Blocked = (1 << 1);

	/** Agent is not on path (EPathFollowingResult::OffPath) */
	inline const Type OffPath = (1 << 2);

	/** Aborted (EPathFollowingResult::Aborted) */
	inline const Type UserAbort = (1 << 3);

	/** Abort details: owner no longer wants to move */
	inline const Type OwnerFinished = (1 << 4);

	/** Abort details: path is no longer valid */
	inline const Type InvalidPath = (1 << 5);

	/** Abort details: unable to move */
	inline const Type MovementStop = (1 << 6);

	/** Abort details: new movement request was received */
	inline const Type NewRequest = (1 << 7);

	/** Abort details: blueprint MoveTo function was called */
	inline const Type ForcedScript = (1 << 8);

	/** Finish details: never started, agent was already at goal */
	inline const Type AlreadyAtGoal = (1 << 9);

	/** Can be used to create project specific reasons */
	inline const Type FirstGameplayFlagShift = 10;

	inline const Type UserAbortFlagMask = ~(Success | Blocked | OffPath);

	FString ToString(uint16 Value);
}

struct FPathFollowingResult
{
	FPathFollowingResultFlags::Type Flags;
	TEnumAsByte<EPathFollowingResult::Type> Code;

	FPathFollowingResult() : Flags(0), Code(EPathFollowingResult::Invalid)  {}
	AIMODULE_API FPathFollowingResult(FPathFollowingResultFlags::Type InFlags);
	AIMODULE_API FPathFollowingResult(EPathFollowingResult::Type ResultCode, FPathFollowingResultFlags::Type ExtraFlags);

	bool HasFlag(FPathFollowingResultFlags::Type Flag) const { return (Flags & Flag) != 0; }

	bool IsSuccess() const { return HasFlag(FPathFollowingResultFlags::Success); }
	bool IsFailure() const { return !HasFlag(FPathFollowingResultFlags::Success); }
	bool IsInterrupted() const { return HasFlag(FPathFollowingResultFlags::UserAbort | FPathFollowingResultFlags::NewRequest); }
	
	AIMODULE_API FString ToString() const;
};

// DEPRECATED, will be removed with GetPathActionType function
UENUM(BlueprintType)
namespace EPathFollowingAction
{
	enum Type : int
	{
		Error,
		NoMove,
		DirectMove,
		PartialPath,
		PathToGoal,
	};
}

UENUM(BlueprintType)
namespace EPathFollowingRequestResult
{
	enum Type : int
	{
		Failed,
		AlreadyAtGoal,
		RequestSuccessful
	};
}

struct FPathFollowingRequestResult
{
	FAIRequestID MoveId;
	TEnumAsByte<EPathFollowingRequestResult::Type> Code;

	FPathFollowingRequestResult() : MoveId(FAIRequestID::InvalidRequest), Code(EPathFollowingRequestResult::Failed) {}
	operator EPathFollowingRequestResult::Type() const { return Code; }
};

namespace EPathFollowingDebugTokens
{
	enum Type : int
	{
		Description,
		ParamName,
		FailedValue,
		PassedValue,
	};
}

// DEPRECATED, please use EPathFollowingResultDetails instead, will be removed with deprecated override of AbortMove function
namespace EPathFollowingMessage
{
	enum Type : int
	{
		NoPath,
		OtherRequest,
	};
}

enum class EPathFollowingVelocityMode : uint8
{
	Reset,
	Keep,
};

enum class EPathFollowingReachMode : uint8
{
	/** reach test uses only AcceptanceRadius */
	ExactLocation,

	/** reach test uses AcceptanceRadius increased by modified agent radius */
	OverlapAgent,

	/** reach test uses AcceptanceRadius increased by goal actor radius */
	OverlapGoal,

	/** reach test uses AcceptanceRadius increased by modified agent radius AND goal actor radius */
	OverlapAgentAndGoal,
};

UCLASS(config=Engine, MinimalAPI)
class UPathFollowingComponent : public UActorComponent, public IAIResourceInterface, public IPathFollowingAgentInterface
{
	GENERATED_UCLASS_BODY()

	DECLARE_DELEGATE_TwoParams(FPostProcessMoveSignature, UPathFollowingComponent* /*comp*/, FVector& /*velocity*/);
	DECLARE_DELEGATE_OneParam(FRequestCompletedSignature, EPathFollowingResult::Type /*Result*/);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FMoveCompletedSignature, FAIRequestID /*RequestID*/, EPathFollowingResult::Type /*Result*/);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FMoveComplete, FAIRequestID /*RequestID*/, const FPathFollowingResult& /*Result*/);

	/** delegate for modifying path following velocity */
	FPostProcessMoveSignature PostProcessMove;

	/** delegate for move completion notify */
	FMoveComplete OnRequestFinished;

	//~ Begin UActorComponent Interface
	AIMODULE_API virtual void OnRegister() override;
	AIMODULE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	//~ End UActorComponent Interface

	/** initialize component to use */
	AIMODULE_API virtual void Initialize();

	/** cleanup component before destroying */
	AIMODULE_API virtual void Cleanup();

	/** updates cached pointers to relevant owner's components */
	AIMODULE_API virtual void UpdateCachedComponents();

	/** start movement along path
	 * @return MoveId of requested move
	 */
	AIMODULE_API virtual FAIRequestID RequestMove(const FAIMoveRequest& RequestData, FNavPathSharedPtr InPath);

	/** aborts following path */
	AIMODULE_API virtual void AbortMove(const UObject& Instigator, FPathFollowingResultFlags::Type AbortFlags, FAIRequestID RequestID = FAIRequestID::CurrentRequest, EPathFollowingVelocityMode VelocityMode = EPathFollowingVelocityMode::Reset);

	/** create new request and finish it immediately (e.g. already at goal)
	 *  @return MoveId of requested (and already finished) move
	 */
	AIMODULE_API FAIRequestID RequestMoveWithImmediateFinish(EPathFollowingResult::Type Result, EPathFollowingVelocityMode VelocityMode = EPathFollowingVelocityMode::Reset);

	/** pause path following
	 *  @param RequestID - request to pause, FAIRequestID::CurrentRequest means pause current request, regardless of its ID */
	AIMODULE_API virtual void PauseMove(FAIRequestID RequestID = FAIRequestID::CurrentRequest, EPathFollowingVelocityMode VelocityMode = EPathFollowingVelocityMode::Reset);

	/** resume path following
	 *  @param RequestID - request to resume, FAIRequestID::CurrentRequest means restor current request, regardless of its ID */
	AIMODULE_API virtual void ResumeMove(FAIRequestID RequestID = FAIRequestID::CurrentRequest);

	/** notify about finished movement */
	AIMODULE_API virtual void OnPathFinished(const FPathFollowingResult& Result);

	inline void OnPathFinished(EPathFollowingResult::Type ResultCode, uint16 ExtraResultFlags) { OnPathFinished(FPathFollowingResult(ResultCode, ExtraResultFlags)); }

	/** notify about finishing move along current path segment */
	AIMODULE_API virtual void OnSegmentFinished();

	/** notify about changing current path: new pointer or update from path event */
	AIMODULE_API virtual void OnPathUpdated();

	/**
	 * set associated movement component.
	 * Note: This function is deprecated and was marked final to purposely alert users that this function should no longer be overriden
	 * Users should instead override SetNavMovementInterface(INavMovementInterface* NavMovementInterface)!
	 */
	UE_DEPRECATED(5.5, "SetMovementComponent(UNavMovementComponent* MoveComp) is deprecated, please use SetNavMoveInterface(INavMoveInterface* NavMoveInterface) instead.")
	AIMODULE_API virtual void SetMovementComponent(UNavMovementComponent* MoveComp) final;

	/** set associated nav movement interface */
	AIMODULE_API virtual void SetNavMovementInterface(INavMovementInterface* NavMoveInterface);

	/** get current focal point of movement */
	AIMODULE_API virtual FVector GetMoveFocus(bool bAllowStrafe) const;

	/** simple test for stationary agent (used as early finish condition), check if reached given point
	 *  @param TestPoint - point to test
	 *  @param AcceptanceRadius - allowed 2D distance
	 *  @param ReachMode - modifiers for AcceptanceRadius
	 */
	AIMODULE_API bool HasReached(const FVector& TestPoint, EPathFollowingReachMode ReachMode, float AcceptanceRadius = UPathFollowingComponent::DefaultAcceptanceRadius) const;

	/** simple test for stationary agent (used as early finish condition), check if reached given goal
	 *  @param TestGoal - actor to test
	 *  @param AcceptanceRadius - allowed 2D distance
	 *  @param ReachMode - modifiers for AcceptanceRadius
	 *  @param bUseNavAgentGoalLocation - true: if the goal is a nav agent, we will use their nav agent location rather than their actual location
	 */
	AIMODULE_API bool HasReached(const AActor& TestGoal, EPathFollowingReachMode ReachMode, float AcceptanceRadius = UPathFollowingComponent::DefaultAcceptanceRadius, bool bUseNavAgentGoalLocation = true) const;

	/** simple test for stationary agent (used as early finish condition), check if reached target specified in move request */
	AIMODULE_API bool HasReached(const FAIMoveRequest& MoveRequest) const;

	/** update state of block detection */
	AIMODULE_API void SetBlockDetectionState(bool bEnable);

	/** @returns state of block detection */
	bool IsBlockDetectionActive() const { return bUseBlockDetection; }

	/** set block detection params */
	AIMODULE_API void SetBlockDetection(float DistanceThreshold, float Interval, int32 NumSamples);

	/** Returns true if pathfollowing is doing deceleration at the end of the path. */
	bool IsDecelerating() const { return bIsDecelerating; };

	/** @returns state of movement stopping on finish */
	inline bool IsStopMovementOnFinishActive() const { return bStopMovementOnFinish; }
	
	/** set whether movement is stopped on finish of move. */
	inline void SetStopMovementOnFinish(bool bEnable) { bStopMovementOnFinish = bEnable; }

	/** set threshold for precise reach tests in intermediate goals (minimal test radius)  */
	AIMODULE_API void SetPreciseReachThreshold(float AgentRadiusMultiplier, float AgentHalfHeightMultiplier);

	/** set status of last requested move, works only in Idle state */
	AIMODULE_API void SetLastMoveAtGoal(bool bFinishedAtGoal);

	/** @returns estimated cost of unprocessed path segments
	 *	@NOTE 0 means, that component is following final path segment or doesn't move */
	AIMODULE_API FVector::FReal GetRemainingPathCost() const;
	
	/** Returns current location on navigation data */
	AIMODULE_API FNavLocation GetCurrentNavLocation() const;

	inline EPathFollowingStatus::Type GetStatus() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return Status;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	inline float GetAcceptanceRadius() const { return AcceptanceRadius; }
	inline float GetDefaultAcceptanceRadius() const { return MyDefaultAcceptanceRadius; }
	AIMODULE_API void SetAcceptanceRadius(const float InAcceptanceRadius);
	inline AActor* GetMoveGoal() const { return DestinationActor.Get(); }
	inline bool HasPartialPath() const { return Path.IsValid() && Path->IsPartial(); }
	inline bool DidMoveReachGoal() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return bLastMoveReachedGoal && (Status == EPathFollowingStatus::Idle);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	inline FAIRequestID GetCurrentRequestId() const { return CurrentRequestId; }
	inline uint32 GetCurrentPathIndex() const { return MoveSegmentStartIndex; }
	inline uint32 GetNextPathIndex() const { return MoveSegmentEndIndex; }
	inline UObject* GetCurrentCustomLinkOb() const { return CurrentCustomLinkOb.Get(); }
	inline FVector GetCurrentTargetLocation() const { return *CurrentDestination; }
	inline FBasedPosition GetCurrentTargetLocationBased() const { return CurrentDestination; }
	inline FVector GetMoveGoalLocationOffset() const { return MoveOffset; }
	bool HasStartedNavLinkMove() const { return bWalkingNavLinkStart; }
	AIMODULE_API bool IsCurrentSegmentNavigationLink() const;
	AIMODULE_API FVector GetCurrentDirection() const;
	/** note that CurrentMoveInput is only valid if MovementComp->UseAccelerationForPathFollowing() == true */
	FVector GetCurrentMoveInput() const { return CurrentMoveInput; }

	/** check if path following has authority over movement (e.g. not falling) and can update own state */
	inline bool HasMovementAuthority() const { return (NavMovementInterface == nullptr) || NavMovementInterface->CanStopPathFollowing(); }

	inline const FNavPathSharedPtr GetPath() const { return Path; }
	inline bool HasValidPath() const { return Path.IsValid() && Path->IsValid(); }
	AIMODULE_API bool HasDirectPath() const;

	/** readable name of current status */
	AIMODULE_API FString GetStatusDesc() const;
	/** readable name of result enum */
	AIMODULE_API FString GetResultDesc(EPathFollowingResult::Type Result) const;

	AIMODULE_API void SetDestinationActor(const AActor* InDestinationActor);

	/** returns index of the currently followed element of path. Depending on the actual 
	 *	path it may represent different things, like a path point or navigation corridor index */
	virtual int32 GetCurrentPathElement() const { return MoveSegmentEndIndex; }

	AIMODULE_API virtual void GetDebugStringTokens(TArray<FString>& Tokens, TArray<EPathFollowingDebugTokens::Type>& Flags) const;
	AIMODULE_API virtual FString GetDebugString() const;

	AIMODULE_API virtual void DisplayDebug(UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos) const;
#if ENABLE_VISUAL_LOG
	AIMODULE_API virtual void DescribeSelfToVisLog(struct FVisualLogEntry* Snapshot) const;
#endif // ENABLE_VISUAL_LOG

	/** called when moving agent collides with another actor */
	UFUNCTION()
	AIMODULE_API virtual void OnActorBump(AActor* SelfActor, AActor* OtherActor, FVector NormalImpulse, const FHitResult& Hit);

	//~ IPathFollowingAgentInterface begin
	AIMODULE_API virtual void OnUnableToMove(const UObject& Instigator) override;
	AIMODULE_API virtual void OnStartedFalling() override;
	virtual void OnLanded() override {}
	AIMODULE_API virtual bool IsFollowingNavLink() const override;
	//~ IPathFollowingAgentInterface end

	/** Check if path following can be activated */
	AIMODULE_API virtual bool IsPathFollowingAllowed() const;

	/** call when moving agent finishes using custom nav link, returns control back to path following */
	AIMODULE_API virtual void FinishUsingCustomLink(INavLinkCustomInterface* CustomNavLink);

	/** called when owner is preparing new pathfinding request */
	virtual void OnPathfindingQuery(FPathFindingQuery& Query) {}

	//~ IAIResourceInterface begin
	AIMODULE_API virtual void LockResource(EAIRequestPriority::Type LockSource) override;
	AIMODULE_API virtual void ClearResourceLock(EAIRequestPriority::Type LockSource) override;
	AIMODULE_API virtual void ForceUnlockResource() override;
	AIMODULE_API virtual bool IsResourceLocked() const override;
	//~ IAIResourceInterface end

	/** path observer */
	AIMODULE_API void OnPathEvent(FNavigationPath* InPath, ENavPathEvent::Type Event);

	/** helper function for sending a path for visual log */
	static AIMODULE_API void LogPathHelper(const AActor* LogOwner, FNavPathSharedPtr InLogPath, const AActor* LogGoalActor);
	static AIMODULE_API void LogPathHelper(const AActor* LogOwner, FNavigationPath* InLogPath, const AActor* LogGoalActor);

	UFUNCTION(BlueprintCallable, Category="AI|Components|PathFollowing", meta = (DeprecatedFunction, DeprecationMessage = "This function is now deprecated, please use AIController.GetMoveStatus instead"))
	AIMODULE_API EPathFollowingAction::Type GetPathActionType() const;

	UFUNCTION(BlueprintCallable, Category="AI|Components|PathFollowing", meta = (DeprecatedFunction, DeprecationMessage = "This function is now deprecated, please use AIController.GetImmediateMoveDestination instead"))
	AIMODULE_API FVector GetPathDestination() const;

#if WITH_EDITORONLY_DATA
	// This delegate is now deprecated, please use OnRequestFinished instead
	FMoveCompletedSignature OnMoveFinished_DEPRECATED;
#endif

protected:
	/** Sets the current status and activates/deactivates the component if necessary */
	AIMODULE_API void SetStatus(EPathFollowingStatus::Type InStatus);

	/** deprecated associated movement component */
	UE_DEPRECATED(5.5, "MovementComp is deprecated, please use NavMovementComp and the INavMoveInterface instead.")
	UPROPERTY(transient, meta = (DeprecatedProperty, DeprecationMessage = "MovementComp is deprecated, please use NavMovementInterface and the INavMoveInterface instead."))
	TObjectPtr<UNavMovementComponent> MovementComp;
	
	/** associated movement interface */
	TWeakInterfacePtr<INavMovementInterface> NavMovementInterface;

	/** currently traversed custom nav link */
	FWeakObjectPtr CurrentCustomLinkOb;

	/** the custom link for the next segment if there is one */
	FWeakObjectPtr MoveSegmentCustomLinkOb;

	/** navigation data for agent described in movement component */
	UPROPERTY(transient)
	TObjectPtr<ANavigationData> MyNavData;

	/** requested path */
	FNavPathSharedPtr Path;

	/** Navigation query filter of the current move request */
	FSharedConstNavQueryFilter NavigationFilter;

	/** value based on navigation agent's properties that's used for AcceptanceRadius when DefaultAcceptanceRadius is requested */
	float MyDefaultAcceptanceRadius;

	/** min distance to destination to consider request successful.
	 *	If following a partial path movement request will finish
	 *	when the original goal gets within AcceptanceRadius or 
	 *	pathfollowing agent gets within MyDefaultAcceptanceRadius 
	 *	of the end of the path*/
	float AcceptanceRadius;

	/** min distance to end of current path segment to consider segment finished */
	float CurrentAcceptanceRadius;

	/** part of agent radius used as min acceptance radius */
	float MinAgentRadiusPct;

	/** part of agent height used as min acceptable height difference */
	float MinAgentHalfHeightPct;

	/** timeout for Waiting state, negative value = infinite */
	float WaitingTimeout;

	/** game specific data */
	FCustomMoveSharedPtr GameData;

	/** destination actor. Use SetDestinationActor to set this */
	TWeakObjectPtr<AActor> DestinationActor;

	/** cached DestinationActor cast to INavAgentInterface. Use SetDestinationActor to set this */
	const INavAgentInterface* DestinationAgent;

	/** destination for current path segment */
	FBasedPosition CurrentDestination;

	/** last MoveInput calculated and passed over to MovementComponent. Valid only if MovementComp->UseAccelerationForPathFollowing() == true */
	FVector CurrentMoveInput;

	/** relative offset from goal actor's location to end of path */
	FVector MoveOffset;

	/** agent location when movement was paused */
	FVector LocationWhenPaused;

	/** This is needed for partial paths when trying to figure out if following a path should finish
	 *	before reaching path end, due to reaching requested acceptance radius away from original
	 *	move goal
	 *	Is being set for non-partial paths as well */
	FVector OriginalMoveRequestGoalLocation;

	/** timestamp of path update when movement was paused */
	double PathTimeWhenPaused;

	/** Indicates a path node index at which precise "is at goal"
	 *	tests are going to be performed every frame, in regards
	 *	to acceptance radius */
	int32 PreciseAcceptanceRadiusCheckStartNodeIndex;

	UE_DEPRECATED_FORGAME(5.6, "This member should no longer be accessed directly, please use GetStatus/SetStatus instead.")
	/** current status */
	TEnumAsByte<EPathFollowingStatus::Type> Status;

	/** increase acceptance radius with agent's radius */
	uint8 bReachTestIncludesAgentRadius : 1;

	/** increase acceptance radius with goal's radius */
	uint8 bReachTestIncludesGoalRadius : 1;

	/** if set, target location will be constantly updated to match goal actor while following last segment of full path */
	uint8 bMoveToGoalOnLastSegment : 1;

	/** Whether to clamp the goal location to reachable navigation data when trying to track the goal actor (Only used when bMoveToGoalOnLastSegment is true)
	 *  False: (default) while following the last segment, the path is allowed to adjust through obstacles and off navigation data without checks.
	 *  True: the last segment's destination will be clamped at the furthest reachable location towards the goal actor. */
	uint8 bMoveToGoalClampedToNavigation : 1;

	/** if set, movement block detection will be used */
	uint8 bUseBlockDetection : 1;

	/** set when agent collides with goal actor */
	uint8 bCollidedWithGoal : 1;

	/** set when last move request was finished at goal */
	uint8 bLastMoveReachedGoal : 1;

	/** if set, movement will be stopped on finishing path */
	uint8 bStopMovementOnFinish : 1;

	/** if set, path following is using FMetaNavMeshPath */
	uint8 bIsUsingMetaPath : 1;

	/** gets set when agent starts following a navigation link. Cleared after agent starts falling or changes segment to a non-link one */
	uint8 bWalkingNavLinkStart : 1;

	/** True if pathfollowing is doing deceleration at the end of the path. @see FollowPathSegment(). */
	uint8 bIsDecelerating : 1;

	/** True if the next segment is a custom link that has its own reach conditions. */
	uint8 bMoveSegmentIsUsingCustomLinkReachCondition : 1;

	/**
	 * Indicates if the component activation/ticking is controlled by setting the Status (Moving --> Activated, other --> Deactivated)
	 * Note that this optimization is not used for any derived classes since those might need to tick in other cases.
	 */
	uint8 bTickComponentOnlyWhenMoving : 1;

	/** detect blocked movement when distance between center of location samples and furthest one (centroid radius) is below threshold */
	float BlockDetectionDistance;

	/** interval for collecting location samples */
	float BlockDetectionInterval;

	/** number of samples required for block detection */
	int32 BlockDetectionSampleCount;

	/** timestamp of last location sample */
	double LastSampleTime;

	/** index of next location sample in array */
	int32 NextSampleIdx;

	/** location samples for stuck detection */
	TArray<FBasedPosition> LocationSamples;

	/** index of path point being current move beginning */
	int32 MoveSegmentStartIndex;

	/** index of path point being current move target */
	int32 MoveSegmentEndIndex;

	/** reference of node at segment start */
	NavNodeRef MoveSegmentStartRef;

	/** reference of node at segment end */
	NavNodeRef MoveSegmentEndRef;

	/** direction of current move segment */
	FVector MoveSegmentDirection;

	/** braking distance for acceleration driven path following */
	float CachedBrakingDistance;

	/** max speed used for CachedBrakingDistance */
	float CachedBrakingMaxSpeed;

	/** index of path point for starting deceleration */
	int32 DecelerationSegmentIndex;

	/** extract the navigation filter that should be used based on the provided MoveRequest and NavPath */
	AIMODULE_API FSharedConstNavQueryFilter ExtractNavigationFilter(const FAIMoveRequest& RequestData, FNavPathSharedPtr InPath) const;

	/** reset path following data */
	AIMODULE_API virtual void Reset();

	/** Called if owning Controller possesses new pawn or ends up pawn-less. 
	 *	Doesn't get called if owner is not an AContoller */
	AIMODULE_API virtual void OnNewPawn(APawn* NewPawn);

	/** should verify if agent if still on path ater movement has been resumed? */
	AIMODULE_API virtual bool ShouldCheckPathOnResume() const;

	/** sets variables related to current move segment */
	AIMODULE_API virtual void SetMoveSegment(int32 SegmentStartIndex);
	
	/** follow current path segment */
	AIMODULE_API virtual void FollowPathSegment(float DeltaTime);

	/** check state of path following, update move segment if needed */
	AIMODULE_API virtual void UpdatePathSegment();

	/** next path segment if custom nav link, try passing control to it */
	AIMODULE_API virtual void StartUsingCustomLink(INavLinkCustomInterface* CustomNavLink, const FVector& DestPoint);

	/** update blocked movement detection, @returns true if new sample was added */
	AIMODULE_API virtual bool UpdateBlockDetection();

	/** updates braking distance and deceleration segment */
	AIMODULE_API virtual void UpdateDecelerationData();

	/** check if move is completed */
	AIMODULE_API virtual bool HasReachedDestination(const FVector& CurrentLocation) const;

	/** check if segment is completed */
	AIMODULE_API virtual bool HasReachedCurrentTarget(const FVector& CurrentLocation) const;

	/** check if moving agent has reached goal defined by cylinder */
	AIMODULE_API bool HasReachedInternal(const FVector& GoalLocation, float GoalRadius, float GoalHalfHeight, const FVector& AgentLocation, float RadiusThreshold, float AgentRadiusMultiplier) const;

	/** reset the cached information about CustomLinks on the next MoveSegment */
	AIMODULE_API void ResetMoveSegmentCustomLinkCache();

	/** check if agent is on path */
	AIMODULE_API virtual bool IsOnPath() const;

	/** check if movement is blocked */
	AIMODULE_API bool IsBlocked() const;

	/** switch to next segment on path */
	inline void SetNextMoveSegment() { SetMoveSegment(GetNextPathIndex()); }

	/** assign new request Id */
	inline void StoreRequestId() { CurrentRequestId = UPathFollowingComponent::GetNextRequestId(); }

	inline static uint32 GetNextRequestId() { return NextRequestId++; }

	/** Checks if this PathFollowingComponent is already on path, and
	*	if so determines index of next path point
	*	@return what PathFollowingComponent thinks should be next path point. INDEX_NONE if given path is invalid
	*	@note this function does not set MoveSegmentEndIndex */
	AIMODULE_API virtual int32 DetermineStartingPathPoint(const FNavigationPath* ConsideredPath) const;

	/** @return index of path point, that should be target of current move segment */
	AIMODULE_API virtual int32 DetermineCurrentTargetPathPoint(int32 StartIndex);

	/** check if movement component is valid or tries to grab one from owner 
	 *	@param bForce results in looking for owner's movement component even if pointer to one is already cached */
	AIMODULE_API virtual bool UpdateMovementComponent(bool bForce = false);

	/** called after receiving update event from current path
	 *  @return false if path was not accepted and move request needs to be aborted */
	AIMODULE_API virtual bool HandlePathUpdateEvent();

	/** called from timer if component spends too much time in Waiting state */
	AIMODULE_API virtual void OnWaitingPathTimeout();

	/** clears Block Detection stored data effectively resetting the mechanism */
	AIMODULE_API void ResetBlockDetectionData();

	/** force creating new location sample for block detection */
	AIMODULE_API void ForceBlockDetectionUpdate();

	/** set move focus in AI owner */
	AIMODULE_API virtual void UpdateMoveFocus();

	/** defines if the agent should reset his velocity when the path is finished*/
	AIMODULE_API virtual bool ShouldStopMovementOnPathFinished() const; 

	/** For given path finds a path node at which
	 *	PathfollowingComponent should start doing 
	 *	precise is-goal-in-acceptance-radius  tests */
	AIMODULE_API int32 FindPreciseAcceptanceRadiusTestsStartNodeIndex(const FNavigationPath& PathInstance, const FVector& GoalLocation) const;

	/** Based on Path's properties, original move goal location and requested AcceptanceRadius
	 *	this function calculates actual acceptance radius to apply when testing if the agent
	 *	has successfully reached requested goal's vicinity */
	AIMODULE_API float GetFinalAcceptanceRadius(const FNavigationPath& PathInstance, const FVector OriginalGoalLocation, const FVector* PathEndOverride = nullptr) const;

	/** debug point reach test values */
	AIMODULE_API void DebugReachTest(float& CurrentDot, float& CurrentDistance, float& CurrentHeight, uint8& bDotFailed, uint8& bDistanceFailed, uint8& bHeightFailed) const;

	/** called when NavigationSystem finishes initial navigation data registration.
	 *	This is usually required by AI agents hand-placed on levels to find MyNavData */
	AIMODULE_API virtual void OnNavigationInitDone();

	/** called when NavigationSystem registers new navigation data type while this component
	 *	instance has empty MyNavData. This is usually the case for AI agents hand-placed
	 *	on levels. */
	UFUNCTION()
	AIMODULE_API void OnNavDataRegistered(ANavigationData* NavData);


	/** used to keep track of which subsystem requested this AI resource be locked */
	FAIResourceLock ResourceLock;

	/** timer handle for OnWaitingPathTimeout function */
	FTimerHandle WaitingForPathTimer;

private:

	/** used for debugging purposes to be able to identify which logged information
	 *	results from which request, if there was multiple ones during one frame */
	static AIMODULE_API uint32 NextRequestId;
	FAIRequestID CurrentRequestId;

	/** Current location on navigation data.  Lazy-updated, so read this via GetCurrentNavLocation(). 
	 *	Since it makes conceptual sense for GetCurrentNavLocation() to be const but we may 
	 *	need to update the cached value, CurrentNavLocation is mutable. */
	mutable FNavLocation CurrentNavLocation;

public:
	/** special float constant to symbolize "use default value". This does not contain 
	 *	value to be used, it's used to detect the fact that it's requested, and 
	 *	appropriate value from querier/doer will be pulled */
	static AIMODULE_API const float DefaultAcceptanceRadius;

#if !UE_BUILD_SHIPPING
	uint8 DEBUG_bMovingDirectlyToGoal : 1;
#endif // !UE_BUILD_SHIPPING
};
