// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "AI/NavigationSystemBase.h"
#include "PathFollowingManager.generated.h"

class AController;


UCLASS(MinimalAPI)
class UPathFollowingManager : public UObject, public IPathFollowingManagerInterface
{
	GENERATED_BODY()
public:
	AIMODULE_API UPathFollowingManager(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	static AIMODULE_API void StopMovement(const AController& Controller);
	static AIMODULE_API bool IsFollowingAPath(const AController& Controller);
};
