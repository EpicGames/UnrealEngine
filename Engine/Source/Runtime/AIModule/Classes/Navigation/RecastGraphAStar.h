// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "AI/Navigation/NavigationTypes.h"
#include "GraphAStar.h"
#include "NavMesh/RecastQueryFilter.h"

#include "RecastGraphAStar.generated.h"

struct FRecastGraphPolicy
{
	static const int32 NodePoolSize = 2048;
	static const int32 OpenSetSize = 2048;
	static const int32 FatalPathLength = 10000;
	static const bool bReuseNodePoolInSubsequentSearches = false;
};

class ARecastNavMesh;
struct FRecastGraphWrapper;
struct FRecastGraphAStarFilter;
struct FRecastAStarSearchNode;
struct FRecastAStar;

typedef uint64 dtPolyRef;
typedef unsigned int dtStatus;
struct dtPoly;
struct dtMeshTile;
struct dtLink;
class dtNavMesh;

#if WITH_RECAST
struct FRecastNeighbour
{
	friend FRecastGraphWrapper;
	friend FRecastGraphAStarFilter;

	inline operator dtPolyRef() const
	{
		return NodeRef;
	}

protected:
	inline FRecastNeighbour(dtPolyRef InNodeRef, unsigned char InSide = 0)
		: NodeRef{ InNodeRef }
		, Side{ InSide }
	{}
public:
	dtPolyRef NodeRef;
	unsigned char Side;
};

struct FRecastAStarResult : public dtQueryResult
{
	void Reset(const int32 PathLength)
	{
		data.resize(0);
	}
	void AddZeroed(const int32 PathLength)
	{
		data.resize(PathLength);
	}

	AIMODULE_API dtPolyRef SetPathInfo(const int32 Index, const FRecastAStarSearchNode& SearchNode);
};
#endif // WITH_RECAST

USTRUCT()
struct FRecastGraphWrapper
{
	GENERATED_BODY()

public:
	FRecastGraphWrapper() {}

#if WITH_RECAST
	/** Initialization of the wrapper from the RecastNavMesh pointer */
	AIMODULE_API void Initialize(const ARecastNavMesh* InRecastNavMeshActor);

	/** Implementation that converts EGraphAStarResult into a dtStatus */
	AIMODULE_API dtStatus ConvertToRecastStatus(const FRecastAStar& Algo, const FRecastGraphAStarFilter& Filter, const EGraphAStarResult AStarResult) const;

	//////////////////////////////////////////////////////////////////////////
	// FGraphAStar: TGraph
	typedef dtPolyRef FNodeRef;

	inline bool IsValidRef(const dtPolyRef& NodeRef) const
	{
		return NodeRef != INVALID_NAVNODEREF;
	}
	AIMODULE_API FRecastNeighbour GetNeighbour(const FRecastAStarSearchNode& Node, const int32 NeighbourIndex) const;
	//////////////////////////////////////////////////////////////////////////

	inline const dtNavMeshQuery& GetRecastQuery() const { return RecastQuery; }

protected:

	friend FRecastGraphAStarFilter;
	friend FRecastAStarSearchNode;

	inline const ARecastNavMesh* GetRecastNavMeshActor() const { checkSlow(RecastNavMeshActor);  return RecastNavMeshActor; }
	inline const dtNavMesh* GetDetourNavMesh() const { checkSlow(DetourNavMesh); return DetourNavMesh; }
	AIMODULE_API void BindFilter(FRecastGraphAStarFilter& AStarFilter);
#endif // WITH_RECAST

private:
	UPROPERTY(Transient)
	TObjectPtr<const ARecastNavMesh> RecastNavMeshActor = nullptr;

#if WITH_RECAST
	const dtNavMesh* DetourNavMesh = nullptr;

	dtNavMeshQuery RecastQuery;

	mutable unsigned int CachedNextLink = DT_NULL_LINK;
#endif // WITH_RECAST
};

#if WITH_RECAST
struct FRecastAStarSearchNode : public FGraphAStarDefaultNode<FRecastGraphWrapper>
{
	typedef FGraphAStarDefaultNode<FRecastGraphWrapper> Super;

	inline FRecastAStarSearchNode(const dtPolyRef InNodeRef = INVALID_NAVNODEREF, FVector InPosition = FVector(TNumericLimits<FVector::FReal>::Max(), TNumericLimits<FVector::FReal>::Max(), TNumericLimits<FVector::FReal>::Max()) )
		: Super(InNodeRef)
		, Tile{ nullptr }
		, Poly{ nullptr }
		, Position { InPosition[0], InPosition[1], InPosition[2] }
	{}

	FRecastAStarSearchNode(const FRecastAStarSearchNode& Other) = default;
	FGraphAStarDefaultNode& operator=(const FGraphAStarDefaultNode& Other) = delete;

	mutable const dtMeshTile* Tile;
	mutable const dtPoly* Poly;
	mutable FVector::FReal Position[3]; // Position in Recast World coordinate system

	inline operator dtPolyRef() const
	{
		return NodeRef;
	}

	inline void Initialize(const FRecastGraphWrapper& RecastGraphWrapper) const
	{
		RecastGraphWrapper.GetDetourNavMesh()->getTileAndPolyByRefUnsafe(NodeRef, &Tile, &Poly);
	}

	inline bool HasValidCacheInfo() const
	{
		return Tile != nullptr && Poly != nullptr && Position[0] != TNumericLimits<FVector::FReal>::Max() && Position[1] != TNumericLimits<FVector::FReal>::Max() && Position[2] != TNumericLimits<FVector::FReal>::Max();
	}

	inline void CacheInfo(const FRecastGraphWrapper& RecastGraphWrapper, const FRecastAStarSearchNode& FromNode) const
	{
		checkSlow(FromNode.HasValidCacheInfo());
		if (!HasValidCacheInfo())
		{
			Initialize(RecastGraphWrapper);
			RecastGraphWrapper.GetRecastQuery().getEdgeMidPoint(FromNode.NodeRef, FromNode.Poly, FromNode.Tile, NodeRef, Poly, Tile, Position);
		}
	}
};

struct FRecastAStar : public FGraphAStar<FRecastGraphWrapper, FRecastGraphPolicy, FRecastAStarSearchNode>
{
	typedef FGraphAStar<FRecastGraphWrapper, FRecastGraphPolicy, FRecastAStarSearchNode> Super;
	FRecastAStar(const FRecastGraphWrapper& Graph)
		: Super(Graph)
	{}
};

struct FRecastGraphAStarFilter
{
	AIMODULE_API FRecastGraphAStarFilter(FRecastGraphWrapper& InRecastGraphWrapper, const FRecastQueryFilter& InFilter, uint32 InMaxSearchNodes, const FVector::FReal InCostLimit, const UObject* Owner);

	inline bool WantsPartialSolution() const
	{ 
		return true; 
	}
	inline FVector::FReal GetHeuristicScale() const
	{ 
		return Filter.getHeuristicScale();
	}

	inline FVector::FReal GetHeuristicCost(const FRecastAStarSearchNode& StartNode, const FRecastAStarSearchNode& EndNode) const
	{
		check(EndNode.HasValidCacheInfo());

		const FVector::FReal Cost = dtVdist(StartNode.Position, EndNode.Position);

		return Cost;
	}

	inline FVector::FReal GetTraversalCost(const FRecastAStarSearchNode& StartNode, const FRecastAStarSearchNode& EndNode) const
	{
		EndNode.CacheInfo(RecastGraphWrapper, StartNode);
		const FVector::FReal Cost = Filter.getCost(
			StartNode.Position, EndNode.Position,
			INVALID_NAVNODEREF, nullptr, nullptr,
			StartNode.NodeRef, StartNode.Tile, StartNode.Poly,
			EndNode.NodeRef, EndNode.Tile, EndNode.Poly);

		return Cost;
	}

	inline bool IsTraversalAllowed(const dtPolyRef& NodeA, const FRecastNeighbour& NodeB) const
	{
		if (!Filter.isValidLinkSide(NodeB.Side))
		{
			return false;
		}
		const dtMeshTile* TileB;
		const dtPoly* PolyB;
		RecastGraphWrapper.GetDetourNavMesh()->getTileAndPolyByRefUnsafe(NodeB.NodeRef, &TileB, &PolyB);
		return Filter.passFilter(NodeB.NodeRef, TileB, PolyB) && RecastGraphWrapper.GetRecastQuery().passLinkFilterByRef(TileB, NodeB.NodeRef);
	}

	inline bool ShouldIgnoreClosedNodes() const
	{
		return Filter.getShouldIgnoreClosedNodes();
	}

	inline bool ShouldIncludeStartNodeInPath() const
	{
		return true;
	}

	inline uint32 GetMaxSearchNodes() const
	{
		return MaxSearchNodes;
	}

	inline FVector::FReal GetCostLimit() const
	{
		return CostLimit;
	}

protected:
	friend FRecastAStarSearchNode;
	friend FRecastGraphWrapper;

	inline const FRecastQueryFilter& GetFilter() const { return Filter; }
	inline FRecastSpeciaLinkFilter& GetLinkFilter() { return LinkFilter; }

private:
	const FRecastQueryFilter& Filter;
	FRecastSpeciaLinkFilter LinkFilter;
	const FRecastGraphWrapper& RecastGraphWrapper;
	uint32 MaxSearchNodes;
	FVector::FReal CostLimit;
};
#endif // WITH_RECAST
