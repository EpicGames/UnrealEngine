// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectKey.h"
#include "Templates/SubclassOf.h"
#include "Components/ActorComponent.h"
#include "EngineDefines.h"
#include "GenericTeamAgentInterface.h"
#include "Perception/AIPerceptionTypes.h"
#include "Perception/AISense.h"
#include "Perception/AIPerceptionSystem.h"
#include "AIPerceptionComponent.generated.h"

class AAIController;
class FGameplayDebuggerCategory;
class UAISenseConfig;
struct FVisualLogEntry;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FPerceptionUpdatedDelegate, const TArray<AActor*>&, UpdatedActors);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FActorPerceptionUpdatedDelegate, AActor*, Actor, FAIStimulus, Stimulus);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FActorPerceptionForgetUpdatedDelegate, AActor*, Actor);

USTRUCT(BlueprintType, meta = (DisplayName = "Sensed Actor's Update Data"))
struct FActorPerceptionUpdateInfo
{
	GENERATED_USTRUCT_BODY()

	/** Id of to the stimulus source */
	UPROPERTY(BlueprintReadWrite, Category = "AI|Perception")
	int32 TargetId = -1;

	/** Actor associated to the stimulus (can be null) */
	UPROPERTY(BlueprintReadWrite, Category = "AI|Perception")
	TWeakObjectPtr<AActor> Target;

	/** Updated stimulus */
	UPROPERTY(BlueprintReadWrite, Category = "AI|Perception")
	FAIStimulus Stimulus;

	FActorPerceptionUpdateInfo() = default;
	FActorPerceptionUpdateInfo(const int32 TargetId, const TWeakObjectPtr<AActor>& Target, const FAIStimulus& Stimulus);
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FActorPerceptionInfoUpdatedDelegate, const FActorPerceptionUpdateInfo&, UpdateInfo);

struct FActorPerceptionInfo
{
	TWeakObjectPtr<AActor> Target;

	TArray<FAIStimulus> LastSensedStimuli;

	/** if != MAX indicates the sense that takes precedence over other senses when it comes
		to determining last stimulus location */
	FAISenseID DominantSense;

	/** indicates whether this Actor is hostile to perception holder */
	uint32 bIsHostile : 1;

	/** indicates whether this Actor is friendly to perception holder */
	uint32 bIsFriendly : 1;

	explicit FActorPerceptionInfo(AActor* InTarget = nullptr)
		: Target(InTarget), DominantSense(FAISenseID::InvalidID()), bIsHostile(false), bIsFriendly(false)
	{
		LastSensedStimuli.AddDefaulted(FAISenseID::GetSize());
	}

	/** Retrieves last known location. Active (last reported as "successful")
	 *	stimuli are preferred. */
	FVector GetLastStimulusLocation(float* OptionalAge = nullptr) const 
	{
		FVector Location(FAISystem::InvalidLocation);
		float BestAge = FLT_MAX;
		bool bBestWasSuccessfullySensed = false;
		for (int32 Sense = 0; Sense < LastSensedStimuli.Num(); ++Sense)
		{
			const float Age = LastSensedStimuli[Sense].GetAge();
			const bool bWasSuccessfullySensed = LastSensedStimuli[Sense].WasSuccessfullySensed();

			if (Age >= 0 && (Age < BestAge 
				|| (bBestWasSuccessfullySensed == false && bWasSuccessfullySensed)
				|| (Sense == DominantSense && bWasSuccessfullySensed)))
			{
				BestAge = Age;
				Location = LastSensedStimuli[Sense].StimulusLocation;
				bBestWasSuccessfullySensed = bWasSuccessfullySensed;

				if (Sense == DominantSense && bWasSuccessfullySensed)
				{
					// if dominant sense is active we don't want to look any further 
					break;
				}
			}
		}

		if (OptionalAge)
		{
			*OptionalAge = BestAge;
		}

		return Location;
	}

	/** it includes both currently live (visible) stimulus, as well as "remembered" ones */
	bool HasAnyKnownStimulus() const
	{
		for (const FAIStimulus& Stimulus : LastSensedStimuli)
		{
			// not that WasSuccessfullySensed will return 'false' for expired stimuli
			if (Stimulus.IsValid() && (Stimulus.WasSuccessfullySensed() == true || Stimulus.IsExpired() == false))
			{
				return true;
			}
		}

		return false;
	}

	/** Indicates currently live (visible) stimulus from any sense */
	bool HasAnyCurrentStimulus() const
	{
		for (const FAIStimulus& Stimulus : LastSensedStimuli)
		{
			// not that WasSuccessfullySensed will return 'false' for expired stimuli
			if (Stimulus.IsValid() && Stimulus.WasSuccessfullySensed() == true && Stimulus.IsExpired() == false)
			{
				return true;
			}
		}

		return false;
	}

	/** Retrieves location of the last sensed stimuli for a given sense
	* @param Sense	The AISenseID of the sense
	*
	* @return Location of the last sensed stimuli or FAISystem::InvalidLocation if given sense has never registered related Target actor or if last stimuli has expired.
	*/
	inline FVector GetStimulusLocation(const FAISenseID Sense) const
	{
		return LastSensedStimuli.IsValidIndex(Sense) && (LastSensedStimuli[Sense].IsValid() && (LastSensedStimuli[Sense].IsExpired() == false)) ? LastSensedStimuli[Sense].StimulusLocation : FAISystem::InvalidLocation;
	}

	/** Retrieves receiver location of the last sense stimuli for a given sense
	* @param Sense	The AISenseID of the sense
	*
	* @return Location of the receiver for the last sensed stimuli or FAISystem::InvalidLocation if given sense has never registered related Target actor or last stimuli has expired.
	*/
	inline FVector GetReceiverLocation(const FAISenseID Sense) const
	{
		return LastSensedStimuli.IsValidIndex(Sense) && (LastSensedStimuli[Sense].IsValid() && (LastSensedStimuli[Sense].IsExpired() == false)) ? LastSensedStimuli[Sense].ReceiverLocation : FAISystem::InvalidLocation;
	}

	/** Indicates a currently active or "remembered" stimuli for a given sense
	* @param Sense	The AISenseID of the sense
	*
	* @return True if a target has been registered (even if not currently sensed) for the given sense and the stimuli is not expired.
	*/
	inline bool HasKnownStimulusOfSense(const FAISenseID Sense) const
	{
		return LastSensedStimuli.IsValidIndex(Sense) && (LastSensedStimuli[Sense].IsValid() && (LastSensedStimuli[Sense].IsExpired() == false));
	}

	/** Indicates a currently active stimuli for a given sense
	* @param Sense	The AISenseID of the sense
	*
	* @return True if a target is still sensed for the given sense and the stimuli is not expired.
	*/
	inline bool IsSenseActive(const FAISenseID Sense) const
	{
		return LastSensedStimuli.IsValidIndex(Sense) && LastSensedStimuli[Sense].IsActive();
	}
	
	/** takes all "newer" info from Other and absorbs it */
	AIMODULE_API void Merge(const FActorPerceptionInfo& Other);
};

USTRUCT(BlueprintType, meta = (DisplayName = "Sensed Actor's Data"))
struct FActorPerceptionBlueprintInfo
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadWrite, Category = "AI|Perception")
	TObjectPtr<AActor> Target;

	UPROPERTY(BlueprintReadWrite, Category = "AI|Perception")
	TArray<FAIStimulus> LastSensedStimuli;

	UPROPERTY(BlueprintReadWrite, Category = "AI|Perception")
	uint32 bIsHostile : 1;

	UPROPERTY(BlueprintReadWrite, Category = "AI|Perception")
	uint32 bIsFriendly : 1;

	FActorPerceptionBlueprintInfo() : Target(nullptr), bIsHostile(false), bIsFriendly(false)
	{}
	FActorPerceptionBlueprintInfo(const FActorPerceptionInfo& Info);
};

/**
 *	AIPerceptionComponent is used to register as stimuli listener in AIPerceptionSystem
 *	and gathers registered stimuli. UpdatePerception is called when component gets new stimuli (batched)
 */
UCLASS(ClassGroup=AI, HideCategories=(Activation, Collision), meta=(BlueprintSpawnableComponent), config=Game, MinimalAPI)
class UAIPerceptionComponent : public UActorComponent
{
	GENERATED_UCLASS_BODY()
	
	static AIMODULE_API const int32 InitialStimuliToProcessArraySize;

	typedef TMap<TObjectKey<AActor>, FActorPerceptionInfo> TActorPerceptionContainer;
	typedef TActorPerceptionContainer FActorPerceptionContainer;

protected:
	UPROPERTY(EditDefaultsOnly, Instanced, Category = "AI Perception")
	TArray<TObjectPtr<UAISenseConfig>> SensesConfig;

	/** Indicated sense that takes precedence over other senses when determining sensed actor's location. 
	 *	Should be set to one of the senses configured in SensesConfig, or None. */
	UPROPERTY(EditDefaultsOnly, Category = "AI Perception")
	TSubclassOf<UAISense> DominantSense;
	
	FAISenseID DominantSenseID;

	UPROPERTY(Transient)
	TObjectPtr<AAIController> AIOwner;

	/** @todo this field is misnamed. It's an allow list. */
	FPerceptionChannelAllowList PerceptionFilter;

private:
	FPerceptionListenerID PerceptionListenerId;
	FActorPerceptionContainer PerceptualData;
		
protected:	
	struct FStimulusToProcess
	{
		TObjectKey<AActor> Source;
		FAIStimulus Stimulus;

		FStimulusToProcess(AActor* InSource, const FAIStimulus& InStimulus)
			: Source(InSource), Stimulus(InStimulus)
		{

		}
	};

	TArray<FStimulusToProcess> StimuliToProcess; 
	
	/** max age of stimulus to consider it "active" (e.g. target is visible) */
	TArray<float> MaxActiveAge;

private:

	/** Determines whether all knowledge of previously sensed actors will be removed or not when they become stale.
		That is, when they are no longer perceived and have exceeded the max age of the sense. */
	uint32 bForgetStaleActors : 1;

	uint32 bCleanedUp : 1;

public:

	AIMODULE_API virtual void PostInitProperties() override;
	AIMODULE_API virtual void BeginDestroy() override;
	AIMODULE_API virtual void OnRegister() override;
	AIMODULE_API virtual void OnUnregister() override;

	UFUNCTION()
	AIMODULE_API void OnOwnerEndPlay(AActor* Actor, EEndPlayReason::Type EndPlayReason);
	
	AIMODULE_API void GetLocationAndDirection(FVector& Location, FVector& Direction) const;
	AIMODULE_API const AActor* GetBodyActor() const;
	AIMODULE_API AActor* GetMutableBodyActor();

	inline FPerceptionChannelAllowList GetPerceptionFilter() const { return PerceptionFilter; }

	AIMODULE_API FGenericTeamId GetTeamIdentifier() const;
	inline FPerceptionListenerID GetListenerId() const { return PerceptionListenerId; }

	AIMODULE_API FVector GetActorLocation(const AActor& Actor) const;
	inline const FActorPerceptionInfo* GetActorInfo(const AActor& Actor) const { return PerceptualData.Find(&Actor); }
	inline FActorPerceptionContainer::TIterator GetPerceptualDataIterator() { return FActorPerceptionContainer::TIterator(PerceptualData); }
	inline FActorPerceptionContainer::TConstIterator GetPerceptualDataConstIterator() const { return FActorPerceptionContainer::TConstIterator(PerceptualData); }

	AIMODULE_API virtual void GetHostileActors(TArray<AActor*>& OutActors) const;
	
	AIMODULE_API void GetHostileActorsBySense(TSubclassOf<UAISense> SenseToFilterBy, TArray<AActor*>& OutActors) const;

	/**	Retrieves all actors in PerceptualData matching the predicate.
	 *	@return whether dead data (invalid actors) have been found while iterating over PerceptualData
	 */
	AIMODULE_API bool GetFilteredActors(const TFunctionRef<bool(const FActorPerceptionInfo&)>& Predicate, TArray<AActor*>& OutActors) const;

	// @note Will stop on first age 0 stimulus
	AIMODULE_API const FActorPerceptionInfo* GetFreshestTrace(const FAISenseID Sense) const;
	
	AIMODULE_API void SetDominantSense(TSubclassOf<UAISense> InDominantSense);
	inline FAISenseID GetDominantSenseID() const { return DominantSenseID; }
	inline TSubclassOf<UAISense> GetDominantSense() const { return DominantSense; }
	AIMODULE_API UAISenseConfig* GetSenseConfig(const FAISenseID& SenseID);
	AIMODULE_API const UAISenseConfig* GetSenseConfig(const FAISenseID& SenseID) const;

	template<typename T, typename = std::enable_if_t<std::is_base_of_v<UAISenseConfig, T>>>
	T* GetSenseConfig() const
	{
		for (UAISenseConfig* SenseConfig : SensesConfig)
		{
			if (T* SenseConfigType = Cast<T>(SenseConfig))
			{
				return SenseConfigType;
			}
		}
		return nullptr;
	}
	
	AIMODULE_API void ConfigureSense(UAISenseConfig& SenseConfig);

	typedef TArray<UAISenseConfig*>::TConstIterator TAISenseConfigConstIterator;
	AIMODULE_API TAISenseConfigConstIterator GetSensesConfigIterator() const;

	/** Notifies AIPerceptionSystem to update properties for this "stimuli listener" */
	UFUNCTION(BlueprintCallable, Category="AI|Perception")
	AIMODULE_API void RequestStimuliListenerUpdate();

	/** Allows toggling senses on and off */
	AIMODULE_API void UpdatePerceptionAllowList(const FAISenseID Channel, const bool bNewValue);

	UE_DEPRECATED(5.0, "Use UpdatePerceptionAllowList instead")
	void UpdatePerceptionWhitelist(const FAISenseID Channel, const bool bNewValue)
	{
		UpdatePerceptionAllowList(Channel, bNewValue);
	}
	

	AIMODULE_API void RegisterStimulus(AActor* Source, const FAIStimulus& Stimulus);
	AIMODULE_API void ProcessStimuli();
	/** Returns true if, as result of stimuli aging, this listener needs an update (like if some stimuli expired) */
	AIMODULE_API bool AgeStimuli(const float ConstPerceptionAgingRate);
	AIMODULE_API void ForgetActor(AActor* ActorToForget);

	/** basically cleans up PerceptualData, resulting in loss of all previous perception */
	UFUNCTION(BlueprintCallable, Category = "AI|Perception")
	AIMODULE_API void ForgetAll();

	AIMODULE_API float GetYoungestStimulusAge(const AActor& Source) const;
	AIMODULE_API bool HasAnyActiveStimulus(const AActor& Source) const;
	AIMODULE_API bool HasAnyCurrentStimulus(const AActor& Source) const;
	AIMODULE_API bool HasActiveStimulus(const AActor& Source, const FAISenseID Sense) const;

#if WITH_GAMEPLAY_DEBUGGER_MENU
	AIMODULE_API virtual void DescribeSelfToGameplayDebugger(FGameplayDebuggerCategory* DebuggerCategory) const;
#endif // WITH_GAMEPLAY_DEBUGGER_MENU

#if ENABLE_VISUAL_LOG
	AIMODULE_API virtual void DescribeSelfToVisLog(FVisualLogEntry* Snapshot) const;
#endif // ENABLE_VISUAL_LOG

	//----------------------------------------------------------------------//
	// blueprint interface
	//----------------------------------------------------------------------//
	UFUNCTION(BlueprintCallable, Category = "AI|Perception")
	AIMODULE_API void GetPerceivedHostileActors(TArray<AActor*>& OutActors) const;

	UFUNCTION(BlueprintCallable, Category = "AI|Perception")
	AIMODULE_API void GetPerceivedHostileActorsBySense(const TSubclassOf<UAISense> SenseToUse, TArray<AActor*>& OutActors) const;

	/** If SenseToUse is none all actors currently perceived in any way will get fetched */
	UFUNCTION(BlueprintCallable, Category = "AI|Perception")
	AIMODULE_API void GetCurrentlyPerceivedActors(TSubclassOf<UAISense> SenseToUse, TArray<AActor*>& OutActors) const;

	/** If SenseToUse is none all actors ever perceived in any way (and not forgotten yet) will get fetched */
	UFUNCTION(BlueprintCallable, Category = "AI|Perception")
	AIMODULE_API void GetKnownPerceivedActors(TSubclassOf<UAISense> SenseToUse, TArray<AActor*>& OutActors) const;

	/** Retrieves whatever has been sensed about given actor */
	UFUNCTION(BlueprintCallable, Category = "AI|Perception")
	AIMODULE_API bool GetActorsPerception(AActor* Actor, FActorPerceptionBlueprintInfo& Info);

	/** Note that this works only if given sense has been already configured for
	 *	this component instance */
	UFUNCTION(BlueprintCallable, Category = "AI|Perception")
	AIMODULE_API void SetSenseEnabled(TSubclassOf<UAISense> SenseClass, const bool bEnable);

	/** Returns if a sense is active. Note that this works only if given sense has been
	*	already configured for this component instance */
	UFUNCTION(BlueprintCallable, Category = "AI|Perception")
	AIMODULE_API bool IsSenseEnabled(TSubclassOf<UAISense> SenseClass) const;

	//////////////////////////////////////////////////////////////////////////
	// Might want to move these to special "BP_AIPerceptionComponent"
	//////////////////////////////////////////////////////////////////////////
	UPROPERTY(BlueprintAssignable)
	FPerceptionUpdatedDelegate OnPerceptionUpdated;

	/**
	 * Notifies all bound delegates that the perception info has been forgotten for a given target.
	 * The notification gets broadcast when all stimuli of a given target expire or the target was explicitly forgotten.
	 * @note The target forgetting functionality must be enabled through the AIPerceptionSystem.bForgetStaleActors flag from Project Settings.
	 *
	 * @param	SourceActor	Actor associated to the stimulus (can not be null)
	 * @param	Stimulus	Updated stimulus
	 * @see ForgetActor, ForgetAll
	 */
	UPROPERTY(BlueprintAssignable)
	FActorPerceptionForgetUpdatedDelegate OnTargetPerceptionForgotten;

	/**
	 * Notifies all bound objects that perception info has been updated for a given target.
	 * The notification is broadcast for any received stimulus or on change of state
	 * according to the stimulus configuration.
	 * 
	 * Note - This delegate will not be called if source actor is no longer valid 
	 * by the time a stimulus gets processed. 
	 * Use OnTargetPerceptionInfoUpdated providing a source id to handle those cases.
	 *
	 * @param	SourceActor	Actor associated to the stimulus (can not be null)
	 * @param	Stimulus	Updated stimulus
	 */
	UPROPERTY(BlueprintAssignable)
	FActorPerceptionUpdatedDelegate OnTargetPerceptionUpdated;

	/**
	 * Notifies all bound objects that perception info has been updated for a given target.
	 * The notification is broadcast for any received stimulus or on change of state
	 * according to the stimulus configuration.
	 *
	 * Note - This delegate will be called even if source actor is no longer valid 
	 * by the time a stimulus gets processed so it is better to use source id for bookkeeping.
	 *
	 * @param	UpdateInfo	Data structure providing information related to the updated perceptual data
	 */
	UPROPERTY(BlueprintAssignable)
	FActorPerceptionInfoUpdatedDelegate OnTargetPerceptionInfoUpdated;

protected:
	FActorPerceptionContainer& GetPerceptualData() { return PerceptualData; }
	const FActorPerceptionContainer& GetPerceptualData() const { return PerceptualData; }

	/** called to clean up on owner's end play or destruction */
	AIMODULE_API virtual void CleanUp();

	AIMODULE_API void RemoveDeadData();

	/** Updates the stimulus entry in StimulusStore, if NewStimulus is more recent or stronger */
	UE_DEPRECATED(5.6, "RefreshStimulus is deprecated. Use ConditionallyStoreSuccessfulStimulus() instead.")
	AIMODULE_API virtual void RefreshStimulus(FAIStimulus& StimulusStore, const FAIStimulus& NewStimulus) final;

	/**
	 * Updates the stimulus entry in StimulusStore, if NewStimulus is more recent or stronger
	 * @return whether the detected change in stimulus values should be considered a "perception change"
	 *		(and, in consequence, emit appropriate notifications) - defaults to `false`
	 */
	AIMODULE_API virtual bool ConditionallyStoreSuccessfulStimulus(FAIStimulus& StimulusStore, const FAIStimulus& NewStimulus);

	/** @note no need to call super implementation, it's there just for some validity checking */
	AIMODULE_API virtual void HandleExpiredStimulus(FAIStimulus& StimulusStore);
	
private:
	friend UAIPerceptionSystem;

	AIMODULE_API void RegisterSenseConfig(const UAISenseConfig& SenseConfig, UAIPerceptionSystem& AIPerceptionSys);
	void StoreListenerId(const FPerceptionListenerID InListenerId) { PerceptionListenerId = InListenerId; }
	AIMODULE_API void SetMaxStimulusAge(const FAISenseID SenseId, float MaxAge);
};

