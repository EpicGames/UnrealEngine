// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "AIPerceptionListenerInterface.generated.h"

class UAIPerceptionComponent;

UINTERFACE(MinimalAPI)
class UAIPerceptionListenerInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IAIPerceptionListenerInterface
{
	GENERATED_IINTERFACE_BODY()

	virtual UAIPerceptionComponent* GetPerceptionComponent() { return nullptr; }
};

