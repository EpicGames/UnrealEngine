// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Components/ActorComponent.h"
#include "Perception/AISense.h"

#include "AIPerceptionStimuliSourceComponent.generated.h"

/** Gives owning actor a way to auto-register as perception system's sense stimuli source */
UCLASS(ClassGroup = AI, HideCategories = (Activation, Collision), meta = (BlueprintSpawnableComponent), config = Game, MinimalAPI)
class UAIPerceptionStimuliSourceComponent : public UActorComponent
{
	GENERATED_BODY()

protected:
	UPROPERTY(EditAnywhere, Category = "AI Perception", BlueprintReadOnly, config)
	uint32 bAutoRegisterAsSource : 1;

	uint32 bSuccessfullyRegistered : 1;

	UPROPERTY(EditAnywhere, Category = "AI Perception", BlueprintReadOnly)
	TArray<TSubclassOf<UAISense> > RegisterAsSourceForSenses;

	AIMODULE_API virtual void OnRegister() override;
public:

	AIMODULE_API UAIPerceptionStimuliSourceComponent(const FObjectInitializer& ObjectInitializer);

#if WITH_EDITOR
	AIMODULE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	/** Registers owning actor as source of stimuli for senses specified in RegisterAsSourceForSenses. 
	 *	Note that you don't have to do it if bAutoRegisterAsSource == true */
	UFUNCTION(BlueprintCallable, Category = "AI|Perception")
	AIMODULE_API void RegisterWithPerceptionSystem();

	/** Registers owning actor as source for specified sense class */
	UFUNCTION(BlueprintCallable, Category = "AI|Perception")
	AIMODULE_API void RegisterForSense(TSubclassOf<UAISense> SenseClass);

	/** Unregister owning actor from being a source of sense stimuli */
	UFUNCTION(BlueprintCallable, Category = "AI|Perception")
	AIMODULE_API void UnregisterFromPerceptionSystem();

	/** Unregisters owning actor from sources list of a specified sense class */
	UFUNCTION(BlueprintCallable, Category = "AI|Perception")
	AIMODULE_API void UnregisterFromSense(TSubclassOf<UAISense> SenseClass);
};
