// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "Engine/World.h"
#include "Perception/AIPerceptionTypes.h"
#include "Perception/AISense.h"
#include "AISubsystem.h"
#include "AIPerceptionSystem.generated.h"

class FGameplayDebuggerCategory;
class UAIPerceptionComponent;
class UAISenseEvent;

AIMODULE_API DECLARE_LOG_CATEGORY_EXTERN(LogAIPerception, Warning, All);

class APawn;

/**
 *	AI Subsystem managing AI Perception through registered AI Senses between Listeners and Stimuli Sources
 */
UCLASS(ClassGroup=AI, config=Game, defaultconfig, MinimalAPI)
class UAIPerceptionSystem : public UAISubsystem
{
	GENERATED_BODY()
		
public:

	AIMODULE_API UAIPerceptionSystem(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
	
	// FTickableGameObject begin
	AIMODULE_API virtual void Tick(float DeltaTime) override;
	AIMODULE_API virtual TStatId GetStatId() const override;
	// FTickableGameObject end

protected:	
	AIPerception::FListenerMap ListenerContainer;

	UPROPERTY()
	TArray<TObjectPtr<UAISense>> Senses;

	UPROPERTY(config, EditAnywhere, Category = Perception)
	float PerceptionAgingRate;

	FActorEndPlaySignature::FDelegate StimuliSourceEndPlayDelegate;

	// not a UPROPERTY on purpose so that we have a control over when stuff gets removed from the map
	TMap<const AActor*, FPerceptionStimuliSource> RegisteredStimuliSources;

	uint32 bHandlePawnNotification : 1;

	struct FDelayedStimulus
	{
		double DeliveryTimestamp;
		FPerceptionListenerID ListenerId;
		TWeakObjectPtr<AActor> Instigator;
		FAIStimulus Stimulus;
	};

	TArray<FDelayedStimulus> DelayedStimuli;

	struct FPerceptionSourceRegistration
	{
		FAISenseID SenseID;
		TWeakObjectPtr<AActor> Source;

		FPerceptionSourceRegistration(FAISenseID InSenseID, AActor* SourceActor)
			: SenseID(InSenseID), Source(SourceActor)
		{}

		inline bool operator==(const FPerceptionSourceRegistration& Other) const
		{
			return SenseID == Other.SenseID && Source == Other.Source;
		}
	};
	TArray<FPerceptionSourceRegistration> SourcesToRegister;

public:

	inline bool IsSenseInstantiated(const FAISenseID& SenseID) const { return SenseID.IsValid() && Senses.IsValidIndex(SenseID) && Senses[SenseID] != nullptr; }

	/** Registers listener if not registered */
	AIMODULE_API void UpdateListener(UAIPerceptionComponent& Listener);
	AIMODULE_API void UnregisterListener(UAIPerceptionComponent& Listener);

	template<typename FEventClass, typename FSenseClass = typename FEventClass::FSenseClass>
	void OnEvent(const FEventClass& Event)
	{
		const FAISenseID SenseID = UAISense::GetSenseID<FSenseClass>();
		if (Senses.IsValidIndex(SenseID) && Senses[SenseID] != nullptr)
		{
			((FSenseClass*)Senses[SenseID])->RegisterEvent(Event);
		}
		// otherwise there's no one interested in this event, skip it.
	}

	template<typename FEventClass, typename FSenseClass = typename FEventClass::FSenseClass>
	void OnEventsBatch(const TArray<FEventClass>& Events)
	{
		if (Events.Num() > 0)
		{
			const FAISenseID SenseID = UAISense::GetSenseID<FSenseClass>();
			if (Senses.IsValidIndex(SenseID) && Senses[SenseID] != nullptr)
			{
				((FSenseClass*)Senses[SenseID])->RegisterEventsBatch(Events);
			}
		}
		// otherwise there's no one interested in this event, skip it.
	}

	template<typename FEventClass, typename FSenseClass = typename FEventClass::FSenseClass>
	static void OnEvent(UWorld* World, const FEventClass& Event)
	{
		UAIPerceptionSystem* PerceptionSys = GetCurrent(World);
		if (PerceptionSys != NULL)
		{
			PerceptionSys->OnEvent<FEventClass, FSenseClass>(Event);
		}
	}

	UFUNCTION(BlueprintCallable, Category = "AI|Perception")
	AIMODULE_API void ReportEvent(UAISenseEvent* PerceptionEvent);

	UFUNCTION(BlueprintCallable, Category = "AI|Perception", meta = (WorldContext="WorldContextObject"))
	static AIMODULE_API void ReportPerceptionEvent(UObject* WorldContextObject, UAISenseEvent* PerceptionEvent);

	/** Registers a source of given sense's stimuli */
	template<typename FSenseClass>
	void RegisterSource(AActor& SourceActor);

	/** Registers given actor as a source for all registered senses */
	AIMODULE_API void RegisterSource(AActor& SourceActor);

	AIMODULE_API void RegisterSourceForSenseClass(TSubclassOf<UAISense> Sense, AActor& Target);

	/** 
	 *	unregisters given actor from the list of active stimuli sources
	 *	@param Sense if null will result in removing SourceActor from all the senses
	 */
	AIMODULE_API void UnregisterSource(AActor& SourceActor, const TSubclassOf<UAISense> Sense = {});

	AIMODULE_API void OnListenerForgetsActor(const UAIPerceptionComponent& Listener, AActor& ActorToForget);
	AIMODULE_API void OnListenerForgetsAll(const UAIPerceptionComponent& Listener);
	AIMODULE_API void OnListenerConfigUpdated(FAISenseID SenseID, const UAIPerceptionComponent& Listener);

	AIMODULE_API void RegisterDelayedStimulus(FPerceptionListenerID ListenerId, float Delay, AActor* Instigator, const FAIStimulus& Stimulus);

	static AIMODULE_API UAIPerceptionSystem* GetCurrent(UObject* WorldContextObject);
	static AIMODULE_API UAIPerceptionSystem* GetCurrent(UWorld& World);

	static AIMODULE_API void MakeNoiseImpl(AActor* NoiseMaker, float Loudness, APawn* NoiseInstigator, const FVector& NoiseLocation, float MaxRange, FName Tag);

	UFUNCTION(BlueprintCallable, Category = "AI|Perception", meta = (WorldContext="WorldContextObject"))
	static AIMODULE_API bool RegisterPerceptionStimuliSource(UObject* WorldContextObject, TSubclassOf<UAISense> Sense, AActor* Target);

	AIMODULE_API FAISenseID RegisterSenseClass(TSubclassOf<UAISense> SenseClass);

	UFUNCTION(BlueprintCallable, Category = "AI|Perception", meta = (WorldContext="WorldContextObject"))
	static AIMODULE_API TSubclassOf<UAISense> GetSenseClassForStimulus(UObject* WorldContextObject, const FAIStimulus& Stimulus);

#if WITH_GAMEPLAY_DEBUGGER_MENU
	AIMODULE_API virtual void DescribeSelfToGameplayDebugger(FGameplayDebuggerCategory& DebuggerCategory) const;
#endif // WITH_GAMEPLAY_DEBUGGER_MENU

protected:
	
	UFUNCTION()
	AIMODULE_API void OnPerceptionStimuliSourceEndPlay(AActor* Actor, EEndPlayReason::Type EndPlayReason);
	
	/** requests registration of a given actor as a perception data source for specified sense */
	AIMODULE_API void RegisterSource(FAISenseID SenseID, AActor& SourceActor);

	/** iterates over all pawns and registeres them as a source for sense indicated by SenseID. Note that this will 
	 *  be performed only for senses that request that (see UAISense.bAutoRegisterAllPawnsAsSources).*/
	AIMODULE_API virtual void RegisterAllPawnsAsSourcesForSense(FAISenseID SenseID);

	enum EDelayedStimulusSorting 
	{
		RequiresSorting,
		NoNeedToSort,
	};
	/** sorts DelayedStimuli and delivers all the ones that are no longer "in the future"
	 *	@return true if any stimuli has become "current" stimuli (meaning being no longer in future) */
	AIMODULE_API bool DeliverDelayedStimuli(EDelayedStimulusSorting Sorting);
	AIMODULE_API void OnNewListener(const FPerceptionListener& NewListener);
	AIMODULE_API void OnListenerUpdate(const FPerceptionListener& UpdatedListener);
	AIMODULE_API void OnListenerRemoved(const FPerceptionListener& UpdatedListener);
	AIMODULE_API void PerformSourceRegistration();

	/** Returns true if aging resulted in tagging any of the listeners to process 
	 *	its stimuli (@see MarkForStimulusProcessing)*/
	AIMODULE_API bool AgeStimuli(const float Amount);

	friend class UAISense;
	inline AIPerception::FListenerMap& GetListenersMap() { return ListenerContainer; }

	friend class UAISystem;
	AIMODULE_API virtual void OnNewPawn(APawn& Pawn);
	AIMODULE_API virtual void StartPlay();

	/** Timestamp of the next stimuli aging */
	double NextStimuliAgingTick;
private:
	/** cached world's timestamp */
	double CurrentTime;
};

//////////////////////////////////////////////////////////////////////////
template<typename FSenseClass>
void UAIPerceptionSystem::RegisterSource(AActor& SourceActor)
{
	FAISenseID SenseID = UAISense::GetSenseID<FSenseClass>();
	if (IsSenseInstantiated(SenseID) == false)
	{
		RegisterSenseClass(FSenseClass::StaticClass());
		SenseID = UAISense::GetSenseID<FSenseClass>();
		check(SenseID.IsValid());
	}
	RegisterSource(SenseID, SourceActor);
}
