// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AITypes.h"
#include "GenericTeamAgentInterface.h"
#include "AIPerceptionTypes.generated.h"

class UAIPerceptionComponent;
class UAISense;

//////////////////////////////////////////////////////////////////////////
struct FAISenseCounter : FAIBasicCounter<uint8>
{};
typedef FAINamedID<FAISenseCounter> FAISenseID;

//////////////////////////////////////////////////////////////////////////
struct FPerceptionListenerCounter : FAIBasicCounter<uint32>
{};
typedef FAIGenericID<FPerceptionListenerCounter> FPerceptionListenerID;

//////////////////////////////////////////////////////////////////////////

UENUM()
enum class EAISenseNotifyType : uint8
{
	/** Continuous update whenever target is perceived. */
	OnEveryPerception,
	/** From "visible" to "not visible" or vice versa. */
	OnPerceptionChange,
};

struct FPerceptionChannelAllowList
{
	typedef int32 FFlagsContainer;

	FFlagsContainer AcceptedChannelsMask;

	// by default accept all
	FPerceptionChannelAllowList() : AcceptedChannelsMask()
	{}

	void Clear()
	{
		AcceptedChannelsMask = 0;
	}

	bool IsEmpty() const
	{
		return (AcceptedChannelsMask == 0);
	}

	inline FPerceptionChannelAllowList& FilterOutChannel(FAISenseID Channel)
	{
		AcceptedChannelsMask &= ~(1 << Channel);
		return *this;
	}

	inline FPerceptionChannelAllowList& AcceptChannel(FAISenseID Channel)
	{
		AcceptedChannelsMask |= (1 << Channel);
		return *this;
	}

	inline bool ShouldRespondToChannel(FAISenseID Channel) const
	{
		return (AcceptedChannelsMask & (1 << Channel)) != 0;
	}

	inline FPerceptionChannelAllowList& MergeFilterIn(const FPerceptionChannelAllowList& OtherFilter)
	{
		AcceptedChannelsMask |= OtherFilter.AcceptedChannelsMask;
		return *this;
	}

	inline FFlagsContainer GetAcceptedChannelsMask() const 
	{ 
		return AcceptedChannelsMask;
	}

	struct FConstIterator
	{
	private:
		FFlagsContainer RemainingChannelsToTest;
		const FPerceptionChannelAllowList& AllowList;
		int32 CurrentIndex;

	public:
		FConstIterator(const FPerceptionChannelAllowList& InAllowList)
			: RemainingChannelsToTest((FFlagsContainer)-1)
			, AllowList(InAllowList)
			, CurrentIndex(INDEX_NONE)
		{
			FindNextAcceptedChannel();
		}

		inline void FindNextAcceptedChannel()
		{
			const FFlagsContainer& Flags = AllowList.GetAcceptedChannelsMask();

			while ((RemainingChannelsToTest & Flags) != 0 && ((1 << ++CurrentIndex) | Flags) == 0)
			{
				RemainingChannelsToTest &= ~(1 << CurrentIndex);
			}
		}

		inline explicit operator bool() const
		{
			return (RemainingChannelsToTest & AllowList.GetAcceptedChannelsMask()) != 0;
		}

		inline int32 operator*() const
		{
			return CurrentIndex;
		}

		inline void operator++()
		{
			// mark "old" index as already used
			RemainingChannelsToTest &= ~(1 << CurrentIndex);
			FindNextAcceptedChannel();
		}
	};
};

USTRUCT(BlueprintType)
struct FAIStimulus
{
	GENERATED_USTRUCT_BODY()

	static AIMODULE_API const float NeverHappenedAge;

	enum FResult
	{
		SensingSucceeded,
		SensingFailed
	};

protected:
	UPROPERTY(BlueprintReadWrite, Category = "AI|Perception")
	float Age;

	UPROPERTY(BlueprintReadWrite, Category = "AI|Perception")
	float ExpirationAge;
public:
	UPROPERTY(BlueprintReadWrite, Category = "AI|Perception")
	float Strength;
	UPROPERTY(BlueprintReadWrite, Category = "AI|Perception")
	FVector StimulusLocation;
	UPROPERTY(BlueprintReadWrite, Category = "AI|Perception")
	FVector ReceiverLocation;
	UPROPERTY(BlueprintReadWrite, Category = "AI|Perception")
	FName Tag;

	FAISenseID Type;

protected:
	uint32 bWantsToNotifyOnlyOnValueChange : 1;

	UPROPERTY(BlueprintReadWrite, Category = "AI|Perception")
	uint32 bSuccessfullySensed:1; // currently used only for marking failed sight tests

	/** this means the stimulus was originally created with a "time limit" and this time has passed. 
	 *	Expiration also results in calling MarkNoLongerSensed */
	uint32 bExpired:1;	
	
public:
	
	/** this is the recommended constructor. Use others if you know what you're doing. */
	AIMODULE_API FAIStimulus(const UAISense& Sense, float StimulusStrength, const FVector& InStimulusLocation, const FVector& InReceiverLocation, FResult Result = SensingSucceeded, FName InStimulusTag = NAME_None);

	// default constructor
	FAIStimulus()
		: Age(NeverHappenedAge), ExpirationAge(NeverHappenedAge), Strength(-1.f), StimulusLocation(FAISystem::InvalidLocation)
		, ReceiverLocation(FAISystem::InvalidLocation), Tag(NAME_None), Type(FAISenseID::InvalidID()), bWantsToNotifyOnlyOnValueChange(false)
		, bSuccessfullySensed(false), bExpired(false)
	{}

	FAIStimulus& SetExpirationAge(float InExpirationAge) { ExpirationAge = InExpirationAge; return *this; }
	FAIStimulus& SetStimulusAge(float StimulusAge) { Age = StimulusAge; return *this; }
	FAIStimulus& SetWantsNotifyOnlyOnValueChange(bool InEnable) { bWantsToNotifyOnlyOnValueChange = InEnable; return *this; }
	
	inline float GetAge() const { return Strength > 0 ? Age : NeverHappenedAge; }
	/** @return false when this stimulus is no longer valid, when it is Expired */
	inline bool AgeStimulus(float ConstPerceptionAgingRate) 
	{ 
		Age += ConstPerceptionAgingRate; 
		return Age < ExpirationAge;
	}
	inline bool WasSuccessfullySensed() const { return bSuccessfullySensed; }
	inline bool IsExpired() const { return bExpired; }
	inline void MarkNoLongerSensed() { bSuccessfullySensed = false; }
	inline void MarkExpired() { bExpired = true; MarkNoLongerSensed(); }
	inline bool IsActive() const { return WasSuccessfullySensed() == true && IsValid(); }
	inline bool WantsToNotifyOnlyOnPerceptionChange() const { return bWantsToNotifyOnlyOnValueChange; }
	inline bool IsValid() const { return Type != FAISenseID::InvalidID() && GetAge() < NeverHappenedAge; }

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	AIMODULE_API FString GetDebugDescription() const;
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
};

USTRUCT(BlueprintType)
struct FAISenseAffiliationFilter
{
	GENERATED_USTRUCT_BODY()

	FAISenseAffiliationFilter() = default;
	
	FAISenseAffiliationFilter(bool bInDetectEnemies, bool bInDetectNeutrals, bool bInDetectFriendlies)
		: bDetectEnemies(bInDetectEnemies)
		, bDetectNeutrals(bInDetectNeutrals)
		, bDetectFriendlies(bInDetectFriendlies) {}

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Sense")
	uint32 bDetectEnemies : 1 = false;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Sense")
	uint32 bDetectNeutrals : 1 = false;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Sense")
	uint32 bDetectFriendlies : 1 = false;
	
	uint8 GetAsFlags() const { return IntCastChecked<uint8>((bDetectEnemies << ETeamAttitude::Hostile) | (bDetectNeutrals << ETeamAttitude::Neutral) | (bDetectFriendlies << ETeamAttitude::Friendly)); }
	inline bool ShouldDetectAll() const { return (bDetectEnemies && bDetectNeutrals && bDetectFriendlies); }

	static inline uint8 DetectAllFlags() { return (1 << ETeamAttitude::Hostile) | (1 << ETeamAttitude::Neutral) | (1 << ETeamAttitude::Friendly); }

	static bool ShouldSenseTeam(FGenericTeamId TeamA, FGenericTeamId TeamB, uint8 AffiliationFlags)
	{
		static const uint8 AllFlags = DetectAllFlags();
		return AffiliationFlags == AllFlags || ((1 << FGenericTeamId::GetAttitude(TeamA, TeamB)) & AffiliationFlags);
	}

	static bool ShouldSenseTeam(const IGenericTeamAgentInterface* TeamAgent, const AActor& TargetActor, uint8 AffiliationFlags)
	{
		static const uint8 AllFlags = DetectAllFlags();
		return AffiliationFlags == AllFlags 
			|| (TeamAgent == nullptr ? (AffiliationFlags & (1 << ETeamAttitude::Neutral)) : ((1 << TeamAgent->GetTeamAttitudeTowards(TargetActor)) & AffiliationFlags));
	}
};

/** Should contain only cached information common to all senses. Sense-specific data needs to be stored by senses themselves */
struct FPerceptionListener
{
	TWeakObjectPtr<UAIPerceptionComponent> Listener;

	FPerceptionChannelAllowList Filter;

	FVector CachedLocation;
	FVector CachedDirection;

	FGenericTeamId TeamIdentifier;

private:
	uint32 bHasStimulusToProcess : 1;

	FPerceptionListenerID ListenerID;

	AIMODULE_API FPerceptionListener();
public:
	AIMODULE_API FPerceptionListener(UAIPerceptionComponent& InListener);

	AIMODULE_API void UpdateListenerProperties(UAIPerceptionComponent& Listener);

	bool operator==(const UAIPerceptionComponent* Other) const { return Listener.Get() == Other; }
	bool operator==(const FPerceptionListener& Other) const { return Listener == Other.Listener; }

	AIMODULE_API void CacheLocation();

	AIMODULE_API void RegisterStimulus(AActor* Source, const FAIStimulus& Stimulus);

	inline bool HasAnyNewStimuli() const { return bHasStimulusToProcess; }
	AIMODULE_API void ProcessStimuli();

	inline bool HasSense(FAISenseID SenseID) const { return Filter.ShouldRespondToChannel(SenseID); }

	// used to remove "dead" listeners
	static AIMODULE_API const FPerceptionListener NullListener;

	inline FPerceptionListenerID GetListenerID() const { return ListenerID; }

	AIMODULE_API FName GetBodyActorName() const;
	AIMODULE_API uint32 GetBodyActorUniqueID() const;

	/** Returns pointer to the actor representing this listener's physical body */
	AIMODULE_API const AActor* GetBodyActor() const;

	AIMODULE_API const IGenericTeamAgentInterface* GetTeamAgent() const;

private:
	friend class UAIPerceptionSystem;
	inline void SetListenerID(FPerceptionListenerID InListenerID) { ListenerID = InListenerID; }
	inline void MarkForStimulusProcessing() { bHasStimulusToProcess = true; }
};

struct FPerceptionStimuliSource
{
	TWeakObjectPtr<AActor> SourceActor;
	FPerceptionChannelAllowList RelevantSenses;
};

namespace AIPerception
{
	typedef TMap<FPerceptionListenerID, FPerceptionListener> FListenerMap;
}
