// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "Perception/AIPerceptionTypes.h"
#include "AISense.generated.h"

class APawn;
class FGameplayDebuggerCategory;
class UAIPerceptionSystem;
class UAISenseEvent;

DECLARE_DELEGATE_OneParam(FOnPerceptionListenerUpdateDelegate, const FPerceptionListener&);

UCLASS(ClassGroup = AI, abstract, config = Engine, MinimalAPI)
class UAISense : public UObject
{
	GENERATED_UCLASS_BODY()

	static AIMODULE_API const float SuspendNextUpdate;

protected:
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "AI Perception", config)
	EAISenseNotifyType NotifyType;

	/** whether this sense is interested in getting notified about new Pawns being spawned 
	 *	this can be used for example for automated sense sources registration */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "AI Perception", config)
	uint32 bWantsNewPawnNotification : 1;

	/** If true all newly spawned pawns will get auto registered as source for this sense. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "AI Perception", config)
	uint32 bAutoRegisterAllPawnsAsSources : 1;

	/** this sense has some internal logic that requires it to be notified when 
	 *	a listener wants to forget an actor*/
	uint32 bNeedsForgettingNotification : 1;
	
private:
	UPROPERTY()
	TObjectPtr<UAIPerceptionSystem> PerceptionSystemInstance;

	/** then this count reaches 0 sense will be updated */
	float TimeUntilNextUpdate;

	FAISenseID SenseID;

protected:
	/**	Called when a new FPerceptionListener registers with AIPerceptionSystem */
	FOnPerceptionListenerUpdateDelegate OnNewListenerDelegate;

	/**	Called when a registered FPerceptionListener changes */
	FOnPerceptionListenerUpdateDelegate OnListenerUpdateDelegate;

	/**	Called when a FPerceptionListener is removed from AIPerceptionSystem */
	FOnPerceptionListenerUpdateDelegate OnListenerRemovedDelegate;
				
public:

	AIMODULE_API virtual UWorld* GetWorld() const override;

	/** use with caution! Needs to be called before any senses get instantiated or listeners registered. DOES NOT update any perceptions system instances */
	static AIMODULE_API void HardcodeSenseID(TSubclassOf<UAISense> SenseClass, FAISenseID HardcodedID);

	static FAISenseID GetSenseID(const TSubclassOf<UAISense> SenseClass) { return SenseClass ? ((const UAISense*)SenseClass->GetDefaultObject())->SenseID : FAISenseID::InvalidID(); }
	template<typename TSense>
	static FAISenseID GetSenseID() 
	{ 
		return GetDefault<TSense>()->GetSenseID();
	}
	inline FAISenseID GetSenseID() const { return SenseID; }

	inline bool WantsUpdateOnlyOnPerceptionValueChange() const { return (NotifyType == EAISenseNotifyType::OnPerceptionChange); }

	AIMODULE_API virtual void PostInitProperties() override;

	/** 
	 *	@return should this sense be ticked now
	 */
	bool ProgressTime(float DeltaSeconds)
	{
		TimeUntilNextUpdate -= DeltaSeconds;
		return TimeUntilNextUpdate <= 0.f;
	}

	void Tick()
	{
		if (TimeUntilNextUpdate <= 0.f)
		{
			TimeUntilNextUpdate = Update();
		}
	}

	//virtual void RegisterSources(TArray<AActor&> SourceActors) {}
	virtual void RegisterSource(AActor& SourceActors){}
	virtual void UnregisterSource(AActor& SourceActors){}

	AIMODULE_API virtual void RegisterWrappedEvent(UAISenseEvent& PerceptionEvent);
	AIMODULE_API virtual FAISenseID UpdateSenseID();

	bool NeedsNotificationOnForgetting() const { return bNeedsForgettingNotification; }
	virtual void OnListenerForgetsActor(const FPerceptionListener& Listener, AActor& ActorToForget) {}
	virtual void OnListenerForgetsAll(const FPerceptionListener& Listener) {}

	inline void OnNewListener(const FPerceptionListener& NewListener) { OnNewListenerDelegate.ExecuteIfBound(NewListener); }
	inline void OnListenerUpdate(const FPerceptionListener& UpdatedListener) { OnListenerUpdateDelegate.ExecuteIfBound(UpdatedListener); }
	inline void OnListenerRemoved(const FPerceptionListener& RemovedListener) { OnListenerRemovedDelegate.ExecuteIfBound(RemovedListener); }
	virtual void OnListenerConfigUpdated(const FPerceptionListener& UpdatedListener) { OnListenerUpdate(UpdatedListener); }

	bool WantsNewPawnNotification() const { return bWantsNewPawnNotification; }
	bool ShouldAutoRegisterAllPawnsAsSources() const { return bAutoRegisterAllPawnsAsSources; }

#if WITH_GAMEPLAY_DEBUGGER_MENU
	AIMODULE_API virtual void DescribeSelfToGameplayDebugger(const UAIPerceptionSystem& PerceptionSystem, FGameplayDebuggerCategory& DebuggerCategory) const;
#endif // WITH_GAMEPLAY_DEBUGGER_MENU

protected:
	friend UAIPerceptionSystem;
	/** gets called when perception system gets notified about new spawned pawn. 
	 *	@Note: do not call super implementation. It's used to detect when subclasses don't override it */
	AIMODULE_API virtual void OnNewPawn(APawn& NewPawn);

	/** @return time until next update */
	virtual float Update() { return FLT_MAX; }

	/** will result in updating as soon as possible */
	inline void RequestImmediateUpdate() { TimeUntilNextUpdate = 0.f; }

	/** will result in updating in specified number of seconds */
	inline void RequestUpdateInSeconds(float UpdateInSeconds) { TimeUntilNextUpdate = UpdateInSeconds; }

	inline UAIPerceptionSystem* GetPerceptionSystem() { return PerceptionSystemInstance; }

	AIMODULE_API void SetSenseID(FAISenseID Index);

	/** returning pointer rather then a reference to prevent users from
	 *	accidentally creating copies by creating non-reference local vars */
	AIMODULE_API AIPerception::FListenerMap* GetListeners();

	/** To be called only for BP-generated classes */
	AIMODULE_API void ForceSenseID(FAISenseID SenseID);
};
