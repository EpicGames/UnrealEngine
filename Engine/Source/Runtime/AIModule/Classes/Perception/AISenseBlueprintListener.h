// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/UserDefinedStruct.h"
#include "AISenseBlueprintListener.generated.h"

UCLASS(ClassGroup=AI, MinimalAPI)
class UAISenseBlueprintListener : public UUserDefinedStruct
{
	GENERATED_BODY()
};
