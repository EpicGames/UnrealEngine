// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "Perception/AIPerceptionTypes.h"
#include "Perception/AISense.h"
#include "AISenseConfig.generated.h"

class FGameplayDebuggerCategory;
class UAIPerceptionComponent;

UCLASS(ClassGroup = AI, abstract, EditInlineNew, config=Game, MinimalAPI)
class UAISenseConfig : public UObject
{
	GENERATED_BODY()

protected:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Sense, AdvancedDisplay)
	FColor DebugColor;

	/** 
	 *  Specifies age limit after stimuli generated by this sense become expired. 0 means "never"
	 *  Whether stimuli expiring affects given listener's memory of the target actor and triggers the Forget event is controlled by AIPerceptionSystem.bForgetStaleActors flag in Project Settings.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Sense, meta = (ClampMin = "0.0", UIMin = "0.0", Units="Seconds"))
	float MaxAge;

	/** determines whether given sense starts in an enabled state */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Sense)
	uint32 bStartsEnabled : 1;
	
	// name of sense
	mutable FString CachedSenseName;

public:
	AIMODULE_API UAISenseConfig(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	AIMODULE_API virtual TSubclassOf<UAISense> GetSenseImplementation() const;

	AIMODULE_API FAISenseID GetSenseID() const;
	AIMODULE_API FString GetSenseName() const;
	FColor GetDebugColor() const { return DebugColor; }
	void SetMaxAge(const float NewMaxAge) { MaxAge = NewMaxAge; }
	float GetMaxAge() const { return MaxAge == 0.f ? FAIStimulus::NeverHappenedAge : MaxAge; }
	
	UE_DEPRECATED(5.4, "Use GetStartsEnabled instead")
	bool IsEnabled() const { return bStartsEnabled; }

	bool GetStartsEnabled() const { return bStartsEnabled; }

	/** Changes whether the given sense starts off enabled.
	*	Note that calling the function after given sense config has already been registered won't have any effect */
	void SetStartsEnabled(bool bEnabled) { bStartsEnabled = bEnabled; }

#if WITH_GAMEPLAY_DEBUGGER_MENU
	AIMODULE_API virtual void DescribeSelfToGameplayDebugger(const UAIPerceptionComponent* PerceptionComponent, FGameplayDebuggerCategory* DebuggerCategory) const;
#endif // WITH_GAMEPLAY_DEBUGGER_MENU
};
