// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Perception/AISense.h"
#include "Perception/AISenseConfig.h"
#include "AISenseConfig_Blueprint.generated.h"

class UAISense_Blueprint;

UCLASS(Blueprintable, Abstract, hidedropdown, MinimalAPI)
class UAISenseConfig_Blueprint : public UAISenseConfig
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Sense", NoClear, config)
	TSubclassOf<UAISense_Blueprint> Implementation;

	AIMODULE_API virtual TSubclassOf<UAISense> GetSenseImplementation() const override;
};
