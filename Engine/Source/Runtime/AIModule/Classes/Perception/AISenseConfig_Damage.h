// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Perception/AISense.h"
#include "Perception/AISenseConfig.h"
#include "Perception/AISense_Damage.h"
#include "AISenseConfig_Damage.generated.h"

UCLASS(meta = (DisplayName = "AI Damage sense config"), MinimalAPI)
class UAISenseConfig_Damage : public UAISenseConfig
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Sense", NoClear, config)
	TSubclassOf<UAISense_Damage> Implementation;

	AIMODULE_API virtual TSubclassOf<UAISense> GetSenseImplementation() const override;
};
