// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Perception/AIPerceptionTypes.h"
#include "Perception/AISense.h"
#include "Perception/AISenseConfig.h"
#include "Perception/AISense_Hearing.h"
#include "AISenseConfig_Hearing.generated.h"

class FGameplayDebuggerCategory;
class UAIPerceptionComponent;

UCLASS(meta = (DisplayName = "AI Hearing config"), MinimalAPI)
class UAISenseConfig_Hearing : public UAISenseConfig
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Sense", NoClear, config)
	TSubclassOf<UAISense_Hearing> Implementation;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Sense", meta = (UIMin = 0.0, ClampMin = 0.0, Units="Centimeters"))
	float HearingRange;

	UE_DEPRECATED(5.2, "LoSHearingRange is deprecated. Use HearingRange instead.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Sense", meta = (UIMin = 0.0, ClampMin = 0.0, Units="Centimeters", EditCondition = "bUseLoSHearing"))
	float LoSHearingRange;

	UE_DEPRECATED(5.2, "bUseLoSHearing is deprecated.")
	UPROPERTY(EditAnywhere, Category = "Sense", meta = (InlineEditConditionToggle))
	uint32 bUseLoSHearing : 1;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Sense", config)
	FAISenseAffiliationFilter DetectionByAffiliation;

	AIMODULE_API virtual TSubclassOf<UAISense> GetSenseImplementation() const override;

#if WITH_GAMEPLAY_DEBUGGER_MENU
	AIMODULE_API virtual void DescribeSelfToGameplayDebugger(const UAIPerceptionComponent* PerceptionComponent, FGameplayDebuggerCategory* DebuggerCategory) const override;
#endif // WITH_GAMEPLAY_DEBUGGER_MENU
};
