// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Perception/AISense.h"
#include "Perception/AISenseConfig.h"
#include "AISenseConfig_Prediction.generated.h"

UCLASS(meta = (DisplayName = "AI Prediction sense config"), MinimalAPI)
class UAISenseConfig_Prediction : public UAISenseConfig
{
	GENERATED_UCLASS_BODY()
public:	
	AIMODULE_API virtual TSubclassOf<UAISense> GetSenseImplementation() const override;
};
