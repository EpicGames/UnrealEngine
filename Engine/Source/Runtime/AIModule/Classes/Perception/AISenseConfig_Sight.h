// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Perception/AIPerceptionTypes.h"
#include "Perception/AISense.h"
#include "Perception/AISenseConfig.h"
#include "Perception/AISense_Sight.h"
#include "AISenseConfig_Sight.generated.h"

class FGameplayDebuggerCategory;
class UAIPerceptionComponent;

UCLASS(meta = (DisplayName = "AI Sight config"), MinimalAPI)
class UAISenseConfig_Sight : public UAISenseConfig
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Sense", NoClear, config)
	TSubclassOf<UAISense_Sight> Implementation;

	/** Maximum sight distance to notice a target. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Sense", config, meta = (UIMin = 0.0, ClampMin = 0.0, Units="Centimeters"))
	float SightRadius;

	/** Maximum sight distance to see target that has been already seen. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Sense", config, meta = (UIMin = 0.0, ClampMin = 0.0, Units="Centimeters"))
	float LoseSightRadius;

	/** How far to the side AI can see, in degrees. Use SetPeripheralVisionAngle to change the value at runtime. 
	 *	The value represents the angle measured in relation to the forward vector, not the whole range. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Sense", config, meta=(UIMin = 0.0, ClampMin = 0.0, UIMax = 180.0, ClampMax = 180.0, DisplayName="Peripheral Vision Half Angle", Units="Degrees"))
	float PeripheralVisionAngleDegrees;

	/** */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Sense", config)
	FAISenseAffiliationFilter DetectionByAffiliation;

	/** If not an InvalidRange (which is the default), we will always be able to see the target that has already been seen if they are within this range of their last seen location. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Sense", config, meta = (Units="Centimeters"))
	float AutoSuccessRangeFromLastSeenLocation;

	/** Point of view move back distance for cone calculation. In conjunction with near clipping distance, this will act as a close by awareness and peripheral vision. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Sense", config, meta = (UIMin = 0.0, ClampMin = 0.0, Units="Centimeters"))
	float PointOfViewBackwardOffset;

	/** Near clipping distance, to be used with point of view backward offset. Will act as a close by awareness and peripheral vision */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Sense", config, meta = (UIMin = 0.0, ClampMin = 0.0, Units="Centimeters"))
	float NearClippingRadius;

	AIMODULE_API virtual TSubclassOf<UAISense> GetSenseImplementation() const override;

#if WITH_EDITOR
	AIMODULE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

#if WITH_GAMEPLAY_DEBUGGER_MENU
	AIMODULE_API virtual void DescribeSelfToGameplayDebugger(const UAIPerceptionComponent* PerceptionComponent, FGameplayDebuggerCategory* DebuggerCategory) const override;
#endif // WITH_GAMEPLAY_DEBUGGER_MENU
};
