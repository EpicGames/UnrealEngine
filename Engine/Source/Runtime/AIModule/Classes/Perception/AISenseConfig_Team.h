// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Perception/AISense.h"
#include "Perception/AISenseConfig.h"
#include "Perception/AISense_Team.h"
#include "AISenseConfig_Team.generated.h"

UCLASS(meta = (DisplayName = "AI Team sense config"), MinimalAPI)
class UAISenseConfig_Team : public UAISenseConfig
{
	GENERATED_UCLASS_BODY()
public:	
	virtual TSubclassOf<UAISense> GetSenseImplementation() const override { return UAISense_Team::StaticClass(); }
};
