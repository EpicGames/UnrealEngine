// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Perception/AISense.h"
#include "Perception/AISenseConfig.h"
#include "AISenseConfig_Touch.generated.h"

UCLASS(meta = (DisplayName = "AI Touch config"), MinimalAPI)
class UAISenseConfig_Touch : public UAISenseConfig
{
	GENERATED_UCLASS_BODY()
public:

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Sense", config)
	FAISenseAffiliationFilter DetectionByAffiliation = {true, true, true};
	
	AIMODULE_API virtual TSubclassOf<UAISense> GetSenseImplementation() const override;
};
