// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "EngineDefines.h"
#include "Perception/AIPerceptionTypes.h"
#include "AISenseEvent.generated.h"

UCLASS(ClassGroup = AI, abstract, EditInlineNew, config = Game, MinimalAPI)
class UAISenseEvent : public UObject
{
	GENERATED_BODY()

public:
	virtual FAISenseID GetSenseID() const PURE_VIRTUAL(UAISenseEvent::GetSenseID, return FAISenseID::InvalidID(););

#if ENABLE_VISUAL_LOG
	virtual void DrawToVLog(UObject& LogOwner) const {}
#else
	void DrawToVLog(UObject& LogOwner) const {}
#endif // ENABLE_VISUAL_LOG
};
