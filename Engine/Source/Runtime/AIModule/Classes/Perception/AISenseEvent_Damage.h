// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Perception/AISenseEvent.h"
#include "Perception/AISense_Damage.h"
#include "AISenseEvent_Damage.generated.h"

UCLASS(MinimalAPI)
class UAISenseEvent_Damage : public UAISenseEvent
{
	GENERATED_BODY()

public:	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sense")
	FAIDamageEvent Event;
	
	AIMODULE_API virtual FAISenseID GetSenseID() const override;

	inline FAIDamageEvent GetDamageEvent()
	{
		Event.Compile();
		return Event;
	}
};
