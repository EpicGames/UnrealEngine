// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Perception/AISense_Hearing.h"
#include "Perception/AISenseEvent.h"
#include "AISenseEvent_Hearing.generated.h"

UCLASS(MinimalAPI)
class UAISenseEvent_Hearing : public UAISenseEvent
{
	GENERATED_BODY()

protected:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sense")
	FAINoiseEvent Event;

public:
	AIMODULE_API UAISenseEvent_Hearing(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
	AIMODULE_API virtual FAISenseID GetSenseID() const override;
	
	inline FAINoiseEvent GetNoiseEvent()
	{
		Event.Compile();
		return Event;
	}
};
