// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Templates/SubclassOf.h"
#include "Perception/AISense.h"
#include "AISense_Blueprint.generated.h"

class APawn;
class UAIPerceptionComponent;
class UAISenseEvent;
class UUserDefinedStruct;

UCLASS(ClassGroup = AI, Abstract, Blueprintable, hidedropdown, MinimalAPI)
class UAISense_Blueprint : public UAISense
{
	GENERATED_BODY()

protected:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Sense")
	TSubclassOf<UUserDefinedStruct> ListenerDataType;

	UPROPERTY(BlueprintReadOnly, Category = "Sense")
	TArray<TObjectPtr<UAIPerceptionComponent>> ListenerContainer;

	UPROPERTY()
	TArray<TObjectPtr<UAISenseEvent>> UnprocessedEvents;

public:
	AIMODULE_API UAISense_Blueprint(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	//void RegisterEvent(const FAITeamStimulusEvent& Event);	

	/** returns requested amount of time to pass until next frame. 
	 *	Return 0 to get update every frame (WARNING: hits performance) */
	UFUNCTION(BlueprintImplementableEvent)
	AIMODULE_API float OnUpdate(const TArray<UAISenseEvent*>& EventsToProcess);

	/**
	 *	@param PerceptionComponent is ActorListener's AIPerceptionComponent instance
	 */
	UFUNCTION(BlueprintImplementableEvent)
	AIMODULE_API void OnListenerRegistered(AActor* ActorListener, UAIPerceptionComponent* PerceptionComponent);

	/**
	 *	@param PerceptionComponent is ActorListener's AIPerceptionComponent instance
	 */
	UFUNCTION(BlueprintImplementableEvent)
	AIMODULE_API void OnListenerUpdated(AActor* ActorListener, UAIPerceptionComponent* PerceptionComponent);

	/** called when a listener unregistered from this sense. Most often this is called due to actor's death
	 *	@param PerceptionComponent is ActorListener's AIPerceptionComponent instance
	 */
	UFUNCTION(BlueprintImplementableEvent)
	AIMODULE_API void OnListenerUnregistered(AActor* ActorListener, UAIPerceptionComponent* PerceptionComponent);
	
	UFUNCTION(BlueprintCallable, Category = "AI|Perception")
	AIMODULE_API void GetAllListenerActors(TArray<AActor*>& ListenerActors) const;

	UFUNCTION(BlueprintCallable, Category = "AI|Perception")
	AIMODULE_API void GetAllListenerComponents(TArray<UAIPerceptionComponent*>& ListenerComponents) const;

	/** called when sense's instance gets notified about new pawn that has just been spawned */
	UFUNCTION(BlueprintImplementableEvent, DisplayName="OnNewPawn", meta=(ScriptName="OnNewPawn"))
	AIMODULE_API void K2_OnNewPawn(APawn* NewPawn);

	AIMODULE_API virtual FAISenseID UpdateSenseID() override;
	AIMODULE_API virtual void RegisterWrappedEvent(UAISenseEvent& PerceptionEvent) override;

protected:
	AIMODULE_API virtual void OnNewPawn(APawn& NewPawn) override;
	AIMODULE_API virtual float Update() override;
	
	AIMODULE_API void OnNewListenerImpl(const FPerceptionListener& NewListener);
	AIMODULE_API void OnListenerUpdateImpl(const FPerceptionListener& UpdatedListener);
	AIMODULE_API void OnListenerRemovedImpl(const FPerceptionListener& UpdatedListener);

private:
	static TMap<FNameEntryId, FAISenseID> BPSenseToSenseID;
};
