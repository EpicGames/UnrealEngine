// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Perception/AISense.h"
#include "AISense_Damage.generated.h"

class IAIPerceptionListenerInterface;
class UAISenseEvent;

USTRUCT(BlueprintType)
struct FAIDamageEvent
{	
	GENERATED_USTRUCT_BODY()

	typedef class UAISense_Damage FSenseClass;

	/** Damage taken by DamagedActor.
	 *	@Note 0-damage events do not get ignored */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sense")
	float Amount;

	/** Event's "Location", or what will be later treated as the perceived location for this sense.
	 *	If not set, HitLocation will be used, if that is unset too DamagedActor's location */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sense")
	FVector Location;

	/** Event's additional spatial information
	 *	@TODO document */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sense")
	FVector HitLocation;
	
	/** Damaged actor */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sense")
	TObjectPtr<AActor> DamagedActor;

	/** Actor that instigated damage. Can be None */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sense")
	TObjectPtr<AActor> Instigator;

	/** Optional named identifier for the damage. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sense")
	FName Tag;
	
	AIMODULE_API FAIDamageEvent();
	AIMODULE_API FAIDamageEvent(AActor* InDamagedActor, AActor* InInstigator, float DamageAmount, const FVector& EventLocation, const FVector& InHitLocation = FAISystem::InvalidLocation, FName InTag = NAME_None);
	AIMODULE_API void Compile();

	bool IsValid() const
	{
		return DamagedActor != nullptr;
	}

	AIMODULE_API IAIPerceptionListenerInterface* GetDamagedActorAsPerceptionListener() const;
};

UCLASS(ClassGroup=AI, MinimalAPI)
class UAISense_Damage : public UAISense
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TArray<FAIDamageEvent> RegisteredEvents;

public:		
	AIMODULE_API void RegisterEvent(const FAIDamageEvent& Event);	
	AIMODULE_API virtual void RegisterWrappedEvent(UAISenseEvent& PerceptionEvent) override;

	/** EventLocation will be reported as Instigator's location at the moment of event happening*/
	UFUNCTION(BlueprintCallable, Category = "AI|Perception", meta = (WorldContext="WorldContextObject", AdvancedDisplay="HitLocation"))
	static AIMODULE_API void ReportDamageEvent(UObject* WorldContextObject, AActor* DamagedActor, AActor* Instigator, float DamageAmount, FVector EventLocation, FVector HitLocation, FName Tag = NAME_None);

protected:
	AIMODULE_API virtual float Update() override;
};
