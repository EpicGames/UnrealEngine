// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GenericTeamAgentInterface.h"
#include "Perception/AISense.h"
#include "AISense_Hearing.generated.h"

class UAISenseConfig_Hearing;
class UAISenseEvent;

USTRUCT(BlueprintType)
struct FAINoiseEvent
{	
	GENERATED_USTRUCT_BODY()

	typedef class UAISense_Hearing FSenseClass;

	float Age;

	/** if not set Instigator's location will be used */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sense")
	FVector NoiseLocation;

	/**
	 * Loudness modifier of the sound.
	 * If MaxRange is non-zero, this modifies the range (by multiplication).
	 * If there is no MaxRange, then if Square(DistanceToSound) <= Square(HearingRange * Loudness), the sound is heard, false otherwise.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sense", meta = (UIMin = 0, ClampMin = 0))
	float Loudness;

	/**
	 * Max range at which the sound can be heard. Multiplied by Loudness.
	 * A value of 0 indicates that there is no range limit, though listeners are still limited by their own hearing range.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sense", meta = (UIMin = 0, ClampMin = 0))
	float MaxRange;
	
	/**
	 * Actor triggering the sound.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sense")
	TObjectPtr<AActor> Instigator;

	/**
	 * Named identifier for the noise.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sense")
	FName Tag;

	FGenericTeamId TeamIdentifier;
		
	AIMODULE_API FAINoiseEvent();
	AIMODULE_API FAINoiseEvent(AActor* InInstigator, const FVector& InNoiseLocation, float InLoudness = 1.f, float InMaxRange = 0.f, FName Tag = NAME_None);

	/** Verifies and calculates derived data */
	AIMODULE_API void Compile();
};

UCLASS(ClassGroup=AI, Config=Game, MinimalAPI)
class UAISense_Hearing : public UAISense
{
	GENERATED_UCLASS_BODY()
		
protected:
	UPROPERTY()
	TArray<FAINoiseEvent> NoiseEvents;

	/** Defaults to 0 to have instant notification. Setting to > 0 will result in delaying 
	 *	when AI hears the sound based on the distance from the source */
	UPROPERTY(config)
	float SpeedOfSoundSq;

	struct FDigestedHearingProperties
	{
		float HearingRangeSq;
		uint8 AffiliationFlags;

		FDigestedHearingProperties(const UAISenseConfig_Hearing& SenseConfig);
		FDigestedHearingProperties();
	};
	TMap<FPerceptionListenerID, FDigestedHearingProperties> DigestedProperties;

public:	
	AIMODULE_API void RegisterEvent(const FAINoiseEvent& Event);	
	AIMODULE_API void RegisterEventsBatch(const TArray<FAINoiseEvent>& Events);

	AIMODULE_API virtual void PostInitProperties() override;

	// part of BP interface. Translates PerceptionEvent to FAINoiseEvent and call RegisterEvent(const FAINoiseEvent& Event)
	AIMODULE_API virtual void RegisterWrappedEvent(UAISenseEvent& PerceptionEvent) override;

	/**
	 * Report a noise event.
	 * 
	 * @param NoiseLocation Location of the noise.
	 * @param Loudness Loudness of the noise. If MaxRange is non-zero, modifies MaxRange, otherwise modifies the squared distance of the sensor's range.
	 * @param Instigator Actor that triggered the noise.
	 * @param MaxRange Max range at which the sound can be heard, multiplied by Loudness. Values <= 0 mean no limit (still limited by listener's range however).
	 * @param Tag Identifier for the event.
	 */
	UFUNCTION(BlueprintCallable, Category = "AI|Perception", meta = (WorldContext="WorldContextObject"))
	static AIMODULE_API void ReportNoiseEvent(UObject* WorldContextObject, FVector NoiseLocation, float Loudness = 1.f, AActor* Instigator = nullptr, float MaxRange = 0.f, FName Tag = NAME_None);

protected:
	AIMODULE_API virtual float Update() override;
	AIMODULE_API virtual void RegisterMakeNoiseDelegate();

	AIMODULE_API void OnNewListenerImpl(const FPerceptionListener& NewListener);
	AIMODULE_API void OnListenerUpdateImpl(const FPerceptionListener& UpdatedListener);
	AIMODULE_API void OnListenerRemovedImpl(const FPerceptionListener& RemovedListener);
};
