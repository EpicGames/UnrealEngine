// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Perception/AISense.h"
#include "AISense_Prediction.generated.h"

class AAIController;
class APawn;
class UAISense_Prediction;

USTRUCT()
struct FAIPredictionEvent
{	
	GENERATED_USTRUCT_BODY()

	typedef UAISense_Prediction FSenseClass;
	
	UPROPERTY()
	TObjectPtr<AActor> Requestor;

	UPROPERTY()
	TObjectPtr<AActor> PredictedActor;

	float TimeToPredict;
		
	FAIPredictionEvent() : Requestor(nullptr), PredictedActor(nullptr) {}
	
	FAIPredictionEvent(AActor* InRequestor, AActor* InPredictedActor, float PredictionTime)
		: Requestor(InRequestor), PredictedActor(InPredictedActor), TimeToPredict(PredictionTime)
	{
	}
};

UCLASS(ClassGroup=AI, MinimalAPI)
class UAISense_Prediction : public UAISense
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TArray<FAIPredictionEvent> RegisteredEvents;

public:		
	AIMODULE_API void RegisterEvent(const FAIPredictionEvent& Event);	

	/** Asks perception system to supply Requestor with PredictedActor's predicted location in PredictionTime seconds
	 *	Location is being predicted based on PredicterActor's current location and velocity */
	UFUNCTION(BlueprintCallable, Category = "AI|Perception")
	static AIMODULE_API void RequestControllerPredictionEvent(AAIController* Requestor, AActor* PredictedActor, float PredictionTime);

	/** Asks perception system to supply Requestor with PredictedActor's predicted location in PredictionTime seconds
	 *	Location is being predicted based on PredicterActor's current location and velocity */
	UFUNCTION(BlueprintCallable, Category = "AI|Perception")
	static AIMODULE_API void RequestPawnPredictionEvent(APawn* Requestor, AActor* PredictedActor, float PredictionTime);

protected:
	AIMODULE_API virtual float Update() override;
};
