// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GenericTeamAgentInterface.h"
#include "WorldCollision.h"
#include "Misc/MTTransactionallySafeAccessDetector.h"
#include "Perception/AISense.h"
#include "AISense_Sight.generated.h"

class IAISightTargetInterface;
class UAISense_Sight;
class UAISenseConfig_Sight;

namespace ESightPerceptionEventName
{
	enum Type
	{
		Undefined,
		GainedSight,
		LostSight
	};
}

USTRUCT()
struct FAISightEvent
{
	GENERATED_USTRUCT_BODY()

	typedef UAISense_Sight FSenseClass;

	float Age;
	ESightPerceptionEventName::Type EventType;	

	UPROPERTY()
	TObjectPtr<AActor> SeenActor;

	UPROPERTY()
	TObjectPtr<AActor> Observer;

	FAISightEvent() : SeenActor(nullptr), Observer(nullptr) {}

	FAISightEvent(AActor* InSeenActor, AActor* InObserver, ESightPerceptionEventName::Type InEventType)
		: Age(0.f), EventType(InEventType), SeenActor(InSeenActor), Observer(InObserver)
	{
	}
};

struct FAISightTarget
{
	typedef uint32 FTargetId;
	static AIMODULE_API const FTargetId InvalidTargetId;

	TWeakObjectPtr<AActor> Target;
	TWeakInterfacePtr<IAISightTargetInterface> WeakSightTargetInterface;
	FGenericTeamId TeamId;
	FTargetId TargetId;

	UE_DEPRECATED_FORGAME(5.5, "SightTargetInterface is deprecated. Use WeakSightTargetInterface instead.")
	IAISightTargetInterface* SightTargetInterface = nullptr;

	AIMODULE_API FAISightTarget(AActor* InTarget = NULL, FGenericTeamId InTeamId = FGenericTeamId::NoTeam);

	inline FVector GetLocationSimple() const
	{
		const AActor* TargetPtr = Target.Get();
		return TargetPtr ? TargetPtr->GetActorLocation() : FVector::ZeroVector;
	}

	inline const AActor* GetTargetActor() const { return Target.Get(); }
};

struct FAISightQuery
{
	FPerceptionListenerID ObserverId;
	FAISightTarget::FTargetId TargetId;

	float Score;
	float Importance;

	FVector LastSeenLocation;

	/** User data that can be used inside the IAISightTargetInterface::CanBeSeenFrom method to store a persistence state */ 
	mutable int32 UserData; 

	union
	{
		/**
		 * We can share the memory for these values because they aren't used at the same time :
		 * - The FrameInfo is used when the query is queued for an update at a later frame. It stores the last time the
		 *   query was processed so that we can prioritize it accordingly against the other queries
		 * - The TraceInfo is used when the query has requested a asynchronous trace and is waiting for the result.
		 *   The engine guarantees that we'll get the info at the next frame, but since we can have multiple queries that
		 *   are pending at the same time, we need to store some information to identify them when receiving the result callback
		 */

		struct  
		{
			uint64 bLastResult:1;
			uint64 LastProcessedFrameNumber:63;
		} FrameInfo;

		/**
		 * The 'FrameNumber' value can increase indefinitely while the 'Index' represents the number of queries that were
		 * already requested during this frame. So it shouldn't reach high values in the allocated 32 bits.
		 * Thanks to that we can reliable only use 31 bits for this value and thus have space to keep the bLastResult value
		 */
		struct
		{
			uint32 bLastResult:1;
			uint32 Index:31;
			uint32 FrameNumber;
		} TraceInfo;
	};

	FAISightQuery(FPerceptionListenerID ListenerId = FPerceptionListenerID::InvalidID(), FAISightTarget::FTargetId Target = FAISightTarget::InvalidTargetId)
		: ObserverId(ListenerId), TargetId(Target), Score(0), Importance(0), LastSeenLocation(FAISystem::InvalidLocation), UserData(0)
	{
		FrameInfo.bLastResult = false;
		FrameInfo.LastProcessedFrameNumber = GFrameCounter;
	}

	/**
	 * Note: This should only be called on queries that are queued up for later processing (in SightQueriesOutOfRange or SightQueriesOutOfRange)
	 */
	float GetAge() const
	{
		return (float)(GFrameCounter - FrameInfo.LastProcessedFrameNumber);
	}

	/**
	 * Note: This should only be called on queries that are queued up for later processing (in SightQueriesOutOfRange or SightQueriesOutOfRange)
	 */
	void RecalcScore()
	{
		Score = GetAge() + Importance;
	}

	void OnProcessed()
	{
		FrameInfo.LastProcessedFrameNumber = GFrameCounter;
	}

	void ForgetPreviousResult()
	{
		LastSeenLocation = FAISystem::InvalidLocation;
		SetLastResult(false);
	}

	bool GetLastResult() const
	{
		return FrameInfo.bLastResult;
	}

	void SetLastResult(const bool bValue)
	{
		FrameInfo.bLastResult = bValue;
	}

	/**
	* Note: This only be called for pending queries because it will erase the LastProcessedFrameNumber value
	*/
	void SetTraceInfo(const FTraceHandle& TraceHandle)
	{
		check((TraceHandle._Data.Index & (static_cast<uint32>(1) << 31)) == 0);
		TraceInfo.Index = TraceHandle._Data.Index;
		TraceInfo.FrameNumber = TraceHandle._Data.FrameNumber;
	}

	class FSortPredicate
	{
	public:
		FSortPredicate()
		{}

		bool operator()(const FAISightQuery& A, const FAISightQuery& B) const
		{
			return A.Score > B.Score;
		}
	};
};

struct FAISightQueryID
{
	FPerceptionListenerID ObserverId;
	FAISightTarget::FTargetId TargetId;

	FAISightQueryID(FPerceptionListenerID ListenerId = FPerceptionListenerID::InvalidID(), FAISightTarget::FTargetId Target = FAISightTarget::InvalidTargetId)
	: ObserverId(ListenerId), TargetId(Target)
	{
	}

	FAISightQueryID(const FAISightQuery& Query)
	: ObserverId(Query.ObserverId), TargetId(Query.TargetId)
	{
	}
};

DECLARE_DELEGATE_FiveParams(FOnPendingVisibilityQueryProcessedDelegate, const FAISightQueryID&, const bool, const float, const FVector&, const TOptional<int32>&);

UCLASS(ClassGroup=AI, config=Game, MinimalAPI)
class UAISense_Sight : public UAISense
{
	GENERATED_UCLASS_BODY()

public:
	struct FDigestedSightProperties
	{
		float PeripheralVisionAngleCos;
		float SightRadiusSq;
		float AutoSuccessRangeSqFromLastSeenLocation;
		float LoseSightRadiusSq;
		float PointOfViewBackwardOffset;
		float NearClippingRadiusSq;
		uint8 AffiliationFlags;

		FDigestedSightProperties();
		FDigestedSightProperties(const UAISenseConfig_Sight& SenseConfig);
	};

	enum class EVisibilityResult
	{
		Visible,
		NotVisible,
		Pending
	};

	typedef TMap<FAISightTarget::FTargetId, FAISightTarget> FTargetsContainer;
	FTargetsContainer ObservedTargets;
	TMap<FPerceptionListenerID, FDigestedSightProperties> DigestedProperties;

	/** The SightQueries are a n^2 problem and to reduce the sort time, they are now split between in range and out of range */
	/** Since the out of range queries only age as the distance component of the score is always 0, there is few need to sort them */
	/** In the majority of the cases most of the queries are out of range, so the sort time is greatly reduced as we only sort the in range queries */
	int32 NextOutOfRangeIndex = 0;
	bool bSightQueriesOutOfRangeDirty = true;
	TArray<FAISightQuery> SightQueriesOutOfRange;
	TArray<FAISightQuery> SightQueriesInRange;
	TArray<FAISightQuery> SightQueriesPending;

protected:
	UPROPERTY(EditDefaultsOnly, Category = "AI Perception", config)
	int32 MaxTracesPerTick;

	/** Maximum number of asynchronous traces that can be requested in a single update call*/
	UPROPERTY(EditDefaultsOnly, Category = "AI Perception", config)
	int32 MaxAsyncTracesPerTick;

	UPROPERTY(EditDefaultsOnly, Category = "AI Perception", config)
	int32 MinQueriesPerTimeSliceCheck;

	UPROPERTY(EditDefaultsOnly, Category = "AI Perception", config)
	double MaxTimeSlicePerTick;

	UPROPERTY(EditDefaultsOnly, Category = "AI Perception", config)
	float HighImportanceQueryDistanceThreshold;

	float HighImportanceDistanceSquare;

	UPROPERTY(EditDefaultsOnly, Category = "AI Perception", config)
	float MaxQueryImportance;

	UPROPERTY(EditDefaultsOnly, Category = "AI Perception", config)
	float SightLimitQueryImportance;

	/** Defines the amount of async trace queries to prevent based on the number of pending queries at the start of an update.
	 * 1 means that the async trace budget is slashed by the pending queries count
	 * 0 means that the async trace budget is not impacted by the pending queries
	 */
	UPROPERTY(EditDefaultsOnly, Category = "AI Perception", config)
	float PendingQueriesBudgetReductionRatio;

	/** Defines if we are allowed to use asynchronous trace queries when there is no IAISightTargetInterface for a Target */
	UPROPERTY(EditDefaultsOnly, Category = "AI Perception", config)
	bool bUseAsynchronousTraceForDefaultSightQueries;

	ECollisionChannel DefaultSightCollisionChannel;

	FOnPendingVisibilityQueryProcessedDelegate OnPendingCanBeSeenQueryProcessedDelegate;
	FTraceDelegate OnPendingTraceQueryProcessedDelegate;

	UE_MT_DECLARE_TS_RW_ACCESS_DETECTOR(QueriesListAccessDetector);

public:

	AIMODULE_API virtual void PostInitProperties() override;
	
	AIMODULE_API void RegisterEvent(const FAISightEvent& Event);	

	AIMODULE_API virtual void RegisterSource(AActor& SourceActors) override;
	AIMODULE_API virtual void UnregisterSource(AActor& SourceActor) override;
	
	AIMODULE_API virtual void OnListenerForgetsActor(const FPerceptionListener& Listener, AActor& ActorToForget) override;
	AIMODULE_API virtual void OnListenerForgetsAll(const FPerceptionListener& Listener) override;

#if WITH_GAMEPLAY_DEBUGGER_MENU
	AIMODULE_API virtual void DescribeSelfToGameplayDebugger(const UAIPerceptionSystem& PerceptionSystem, FGameplayDebuggerCategory& DebuggerCategory) const override;
#endif // WITH_GAMEPLAY_DEBUGGER_MENU

protected:
	AIMODULE_API virtual float Update() override;

	AIMODULE_API EVisibilityResult ComputeVisibility(UWorld* World, FAISightQuery& SightQuery, FPerceptionListener& Listener, const AActor* ListenerActor, FAISightTarget& Target, AActor* TargetActor, const FDigestedSightProperties& PropDigest, float& OutStimulusStrength, FVector& OutSeenLocation, int32& OutNumberOfLoSChecksPerformed, int32& OutNumberOfAsyncLosCheckRequested) const;
	AIMODULE_API virtual bool ShouldAutomaticallySeeTarget(const FDigestedSightProperties& PropDigest, FAISightQuery* SightQuery, FPerceptionListener& Listener, AActor* TargetActor, float& OutStimulusStrength) const;
	
	UE_DEPRECATED(5.3, "Please use the UpdateQueryVisibilityStatus version which takes an Actor& instead.")
	AIMODULE_API void UpdateQueryVisibilityStatus(FAISightQuery& SightQuery, FPerceptionListener& Listener, const bool bIsVisible, const FVector& SeenLocation, const float StimulusStrength, AActor* TargetActor, const FVector& TargetLocation) const;
	AIMODULE_API void UpdateQueryVisibilityStatus(FAISightQuery& SightQuery, FPerceptionListener& Listener, const bool bIsVisible, const FVector& SeenLocation, const float StimulusStrength, AActor& TargetActor, const FVector& TargetLocation) const;

	AIMODULE_API void OnPendingCanBeSeenQueryProcessed(const FAISightQueryID& QueryID, const bool bIsVisible, const float StimulusStrength, const FVector& SeenLocation, const TOptional<int32>& UserData);
	AIMODULE_API void OnPendingTraceQueryProcessed(const FTraceHandle& TraceHandle, FTraceDatum& TraceDatum);
	AIMODULE_API void OnPendingQueryProcessed(const int32 SightQueryIndex, const bool bIsVisible, const float StimulusStrength, const FVector& SeenLocation, const TOptional<int32>& UserData, const TOptional<AActor*> InTargetActor = NullOpt);

	AIMODULE_API void OnNewListenerImpl(const FPerceptionListener& NewListener);
	AIMODULE_API void OnListenerUpdateImpl(const FPerceptionListener& UpdatedListener);
	AIMODULE_API void OnListenerRemovedImpl(const FPerceptionListener& RemovedListener);
	AIMODULE_API virtual void OnListenerConfigUpdated(const FPerceptionListener& UpdatedListener) override;
	
	AIMODULE_API void GenerateQueriesForListener(const FPerceptionListener& Listener, const FDigestedSightProperties& PropertyDigest, const TFunction<void(FAISightQuery&)>& OnAddedFunc = nullptr);

	AIMODULE_API void RemoveAllQueriesByListener(const FPerceptionListener& Listener, const TFunction<void(const FAISightQuery&)>& OnRemoveFunc = nullptr);
	AIMODULE_API void RemoveAllQueriesToTarget(const FAISightTarget::FTargetId& TargetId, const TFunction<void(const FAISightQuery&)>& OnRemoveFunc = nullptr);
	/** RemoveAllQueriesToTarget version that need to already have a write access on QueriesListAccessDetector*/
	void RemoveAllQueriesToTarget_Internal(const FAISightTarget::FTargetId& TargetId, const TFunction<void(const FAISightQuery&)>& OnRemoveFunc = nullptr);

	/** returns information whether new LoS queries have been added */
	AIMODULE_API bool RegisterTarget(AActor& TargetActor, const TFunction<void(FAISightQuery&)>& OnAddedFunc = nullptr);

	AIMODULE_API float CalcQueryImportance(const FPerceptionListener& Listener, const FVector& TargetLocation, const float SightRadiusSq) const;
	AIMODULE_API bool RegisterNewQuery(const FPerceptionListener& Listener, const IGenericTeamAgentInterface* ListenersTeamAgent, const AActor& TargetActor, const FAISightTarget::FTargetId& TargetId, const FVector& TargetLocation, const FDigestedSightProperties& PropDigest, const TFunction<void(FAISightQuery&)>& OnAddedFunc);

protected:
	enum FQueriesOperationPostProcess
	{
		DontSort,
		Sort
	};
};
