// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GenericTeamAgentInterface.h"
#include "Perception/AISense.h"
#include "AISense_Team.generated.h"

class UAISense_Team;

USTRUCT()
struct FAITeamStimulusEvent
{	
	GENERATED_USTRUCT_BODY()

	typedef UAISense_Team FSenseClass;

	FVector LastKnowLocation;
private:
	FVector BroadcastLocation;
public:
	float RangeSq;
	float InformationAge;
	FGenericTeamId TeamIdentifier;
	float Strength;
private:
	UPROPERTY()
	TObjectPtr<AActor> Broadcaster;
public:
	UPROPERTY()
	TObjectPtr<AActor> Enemy;
		
	FAITeamStimulusEvent() : Broadcaster(nullptr), Enemy(nullptr) {}
	AIMODULE_API FAITeamStimulusEvent(AActor* InBroadcaster, AActor* InEnemy, const FVector& InLastKnowLocation, float EventRange, float PassedInfoAge = 0.f, float InStrength = 1.f);

	inline void CacheBroadcastLocation()
	{
		BroadcastLocation = Broadcaster ? Broadcaster->GetActorLocation() : FAISystem::InvalidLocation;
	}

	inline const FVector& GetBroadcastLocation() const 
	{
		return BroadcastLocation;
	}
};

UCLASS(ClassGroup=AI, MinimalAPI)
class UAISense_Team : public UAISense
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TArray<FAITeamStimulusEvent> RegisteredEvents;

public:		
	AIMODULE_API void RegisterEvent(const FAITeamStimulusEvent& Event);	

protected:
	AIMODULE_API virtual float Update() override;
};
