// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Perception/AISense.h"
#include "AISense_Touch.generated.h"

class IAIPerceptionListenerInterface;
class UAISense_Touch;
class UAISenseConfig_Touch;

USTRUCT()
struct FAITouchEvent
{	
	GENERATED_USTRUCT_BODY()

	typedef UAISense_Touch FSenseClass;

	FVector Location = FVector::ZeroVector;
	
	UPROPERTY()
	TObjectPtr<AActor> TouchReceiver;

	UPROPERTY()
	TObjectPtr<AActor> OtherActor;

	FGenericTeamId TeamIdentifier = FGenericTeamId::NoTeam;
		
	FAITouchEvent() = default;
	
	FAITouchEvent(AActor* InTouchReceiver, AActor* InOtherActor, const FVector& EventLocation)
		: Location(EventLocation), TouchReceiver(InTouchReceiver), OtherActor(InOtherActor)
	{
		TeamIdentifier = FGenericTeamId::GetTeamIdentifier(InOtherActor);
	}

	AIMODULE_API IAIPerceptionListenerInterface* GetTouchedActorAsPerceptionListener() const;
};

UCLASS(ClassGroup=AI, MinimalAPI)
class UAISense_Touch : public UAISense
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TArray<FAITouchEvent> RegisteredEvents;

public:		
	AIMODULE_API void RegisterEvent(const FAITouchEvent& Event);	

	UFUNCTION(BlueprintCallable, Category = "AI|Perception", meta = (WorldContext = "WorldContextObject"))
	static AIMODULE_API void ReportTouchEvent(UObject* WorldContextObject, AActor* TouchReceiver, AActor* OtherActor, FVector Location);

protected:
	
	struct FDigestedTouchProperties
	{
		uint8 AffiliationFlags;

		FDigestedTouchProperties(const UAISenseConfig_Touch& SenseConfig);
		FDigestedTouchProperties();
	};
	TMap<FPerceptionListenerID, FDigestedTouchProperties> DigestedProperties;

	
	AIMODULE_API virtual float Update() override;
	
	AIMODULE_API void OnNewListenerImpl(const FPerceptionListener& NewListener);
	AIMODULE_API void OnListenerUpdateImpl(const FPerceptionListener& UpdatedListener);
	AIMODULE_API void OnListenerRemovedImpl(const FPerceptionListener& RemovedListener);
};
