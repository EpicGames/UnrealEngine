// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AISense_Sight.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "AISightTargetInterface.generated.h"

class AActor;

UINTERFACE(MinimalAPI)
class UAISightTargetInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

struct FCanBeSeenFromContext
{
	/** The query identifier used by the delegate call to find the appropriate query */
	FAISightQueryID SightQueryID;
	/** The location of the observer */
	FVector ObserverLocation;
	/** The actor to ignore when doing test */
	const AActor* IgnoreActor = nullptr;
	/** If available, it is the previous visibility state */
	const bool* bWasVisible = nullptr;
};

class IAISightTargetInterface
{
	GENERATED_IINTERFACE_BODY()

	/**	
	 * The method needs to check whether the implementer is visible from given observer's location. 
	 * This version allows an asynchronous answer by returning Pending and using the provided Delegate when the information is computed
	 * @param Context								Struct containing all the information required to perform the query
	 * @param OutSeenLocation						The first visible target location
	 * @param OutNumberOfLoSChecksPerformed			The number of synchronous queries done. This is used to stop the perception tick sooner when too many LoS checks are done
	 * @param OutNumberOfAsyncLosCheckRequested		The number of asynchronous queries done. This is used to stop the perception tick sooner when too many asynchronous LoS checks are done
	 * @param OutSightStrength						The sight strength for how well the target is seen
	 * @param UserData								If available, it is a data passed between visibility tests for the users to store whatever they want
	 * @param Delegate								If available, the delegate to call later on, if the result needed asynchronous checks. Calling this delegate within the CanBeSeenFrom call will ensure
	 * @return										Undefined to indicate that we need to try another CanBeSeenFrom implementation, Visible/NotVisible if visible/not visible from the observer's location, Pending if the check is not finished
	 */
	virtual UAISense_Sight::EVisibilityResult CanBeSeenFrom(const FCanBeSeenFromContext& Context, FVector& OutSeenLocation, int32& OutNumberOfLoSChecksPerformed, int32& OutNumberOfAsyncLosCheckRequested, float& OutSightStrength, int32* UserData = nullptr, const FOnPendingVisibilityQueryProcessedDelegate* Delegate = nullptr)
	{
		OutNumberOfAsyncLosCheckRequested = 0;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const bool bCanBeSeenFrom = CanBeSeenFrom(Context.ObserverLocation, OutSeenLocation, OutNumberOfLoSChecksPerformed, OutSightStrength, Context.IgnoreActor, Context.bWasVisible, UserData);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return bCanBeSeenFrom ? UAISense_Sight::EVisibilityResult::Visible : UAISense_Sight::EVisibilityResult::NotVisible;
	}

	/**	
	 * The method needs to check whether the implementer is visible from given observer's location. 
	 * @param ObserverLocation	The location of the observer
	 * @param OutSeenLocation	The first visible target location
	 * @param OutSightStrength	The sight strength for how well the target is seen
	 * @param IgnoreActor		The actor to ignore when doing test
	 * @param bWasVisible		If available, it is the previous visibility state
	 * @param UserData			If available, it is a data passed between visibility tests for the users to store whatever they want
	 * @return	True if visible from the observer's location
	 */
	UE_DEPRECATED(5.2, "This function is deprecated. Use the new CanBeSeenFrom method signature")
	virtual bool CanBeSeenFrom(const FVector& ObserverLocation, FVector& OutSeenLocation, int32& NumberOfLoSChecksPerformed, float& OutSightStrength, const AActor* IgnoreActor = nullptr, const bool* bWasVisible = nullptr, int32* UserData = nullptr) const
	{ 
		NumberOfLoSChecksPerformed = 0;
		OutSightStrength = 0;
		return false; 
	}
};

