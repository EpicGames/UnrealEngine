// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Components/ActorComponent.h"
#include "PawnSensingComponent.generated.h"

class AActor;
class AController;
class APawn;
class UPawnNoiseEmitterComponent;

/**
 * SensingComponent encapsulates sensory (ie sight and hearing) settings and functionality for an Actor,
 * allowing the actor to see/hear Pawns in the world. It does nothing on network clients.
 */
class UE_DEPRECATED_FORGAME(5.5, "Pawn sensing is deprecated. AI Perception should be used instead.") UPawnSensingComponent;
UCLASS(ClassGroup=AI, HideCategories=(Activation, "Components|Activation", Collision), meta=(BlueprintSpawnableComponent), MinimalAPI)
class UPawnSensingComponent : public UActorComponent
{
	GENERATED_UCLASS_BODY()

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam( FSeePawnDelegate, APawn*, Pawn );
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams( FHearNoiseDelegate, APawn*, Instigator, const FVector&, Location, float, Volume);

	/** Max distance at which a makenoise(1.0) loudness sound can be heard, regardless of occlusion */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AI)
	float HearingThreshold;

	/** Max distance at which a makenoise(1.0) loudness sound can be heard if unoccluded (LOSHearingThreshold should be > HearingThreshold) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AI)
	float LOSHearingThreshold;

	/** Maximum sight distance. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AI)
	float SightRadius;

	/** Amount of time between pawn sensing updates. Use SetSensingInterval() to adjust this at runtime. A value <= 0 prevents any updates. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AI)
	float SensingInterval;

	// Max age of sounds we can hear. Should be greater than SensingInterval, or you might miss hearing some sounds!
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AI)
	float HearingMaxSoundAge;
	
public:

	/**
	 * Changes the SensingInterval.
	 * If we are currently waiting for an interval, this can either extend or shorten that interval.
	 * A value <= 0 prevents any updates.
	 */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category="AI|Components|PawnSensing")
	AIMODULE_API virtual void SetSensingInterval(const float NewSensingInterval);

	/** Enables or disables sensing updates. The timer is reset in either case. */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category="AI|Components|PawnSensing")
	AIMODULE_API virtual void SetSensingUpdatesEnabled(const bool bEnabled);

	/** Sets PeripheralVisionAngle. Calculates PeripheralVisionCosine from PeripheralVisionAngle */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category="AI|Components|PawnSensing")
	AIMODULE_API virtual void SetPeripheralVisionAngle(const float NewPeripheralVisionAngle);

	UFUNCTION(BlueprintCallable, Category="AI|Components|PawnSensing")
	float GetPeripheralVisionAngle() const;

	UFUNCTION(BlueprintCallable, Category="AI|Components|PawnSensing")
	float GetPeripheralVisionCosine() const;

	/** If true, component will perform sensing updates. At runtime change this using SetSensingUpdatesEnabled(). */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AI)
	uint32 bEnableSensingUpdates:1;

	/** If true, will only sense player-controlled pawns in the world. Default: true */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AI)
	uint32 bOnlySensePlayers:1;

	/** If true, we will perform visibility tests and will trigger notifications when a Pawn is visible. Default: true */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AI)
	uint32 bSeePawns:1;

	/**
	 * If true, we will perform audibility tests and will be notified when a Pawn makes a noise that can be heard. Default: true
	 * IMPORTANT NOTE: If we can see pawns (bSeePawns is true), and the pawn is visible, noise notifications are not triggered.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AI)
	uint32 bHearNoises:1;

	/** Is the given actor our owner? Used to ensure that we are not trying to sense our self / our owner. */
	AIMODULE_API virtual bool IsSensorActor(const AActor* Actor) const;

	/** Are we capable of sensing anything (and do we have any callbacks that care about sensing)? If so UpdateAISensing() will be called every sensing interval. */
	AIMODULE_API virtual bool CanSenseAnything() const;

	/** Returns true if we should check whether the given Pawn is visible (because we can see things, the Pawn is not hidden, and if the Pawn is a player and we only see players) */
	AIMODULE_API virtual bool ShouldCheckVisibilityOf(APawn* Pawn) const;

	/** 
	 * Chance of seeing other pawn decreases with increasing distance or angle in peripheral vision
	 * @param bMaySkipChecks if true allows checks to be sometimes skipped if target is far away (used by periodic automatic visibility checks)
	 * @return true if the specified pawn Other is potentially visible (within peripheral vision, etc.) - still need to do LineOfSightTo() check to see if actually visible.  
	 */
	AIMODULE_API virtual bool CouldSeePawn(const APawn* Other, bool bMaySkipChecks = false) const;

	/** Returns true if we should check whether we can hear the given Pawn (because we are able to hear, and the Pawn has the correct team relationship to us) */
	AIMODULE_API virtual bool ShouldCheckAudibilityOf(APawn* Pawn) const;

	/**
	 * Check line to other actor.
	 * @param Other is the actor whose visibility is being checked.
	 * @param ViewPoint is eye position visibility is being checked from.
	 * @return true if controller's pawn can see Other actor.
	 */
	AIMODULE_API virtual bool HasLineOfSightTo(const AActor* Other) const;

	/** Test whether the noise is loud enough and recent enough to care about.  bSourceWithinNoiseEmitter is true iff the 
	 * noise was made by the pawn itself or within close proximity (its collision volume).  Otherwise the noise was made
	 * at significant distance from the pawn.
	 */
	AIMODULE_API bool IsNoiseRelevant(const APawn& Pawn, const UPawnNoiseEmitterComponent& NoiseEmitterComponent, bool bSourceWithinNoiseEmitter) const;

	/** @Returns true if sensor can hear this noise. Only executed if the noise has been determined to be relevant (via IsNoiseRelevant) */
	AIMODULE_API virtual bool CanHear(const FVector& NoiseLoc, float Loudness, bool bFailedLOS) const;

	//~ Begin UActorComponent Interface.
	AIMODULE_API virtual void InitializeComponent() override;
	//~ End UActorComponent Interface.

	/** Get position where hearing/seeing occurs (i.e. ear/eye position).  If we ever need different positions for hearing/seeing, we'll deal with that then! */
	AIMODULE_API virtual FVector GetSensorLocation() const;

	/**  Get the rotation of this sensor. We need this for the sight component */
	AIMODULE_API virtual FRotator GetSensorRotation() const;

protected:

	/** See if there are interesting sounds and sights that we want to detect, and respond to them if so. */
	AIMODULE_API virtual void SensePawn(APawn& Pawn);

	/** Update function called on timer intervals. */
	AIMODULE_API virtual void OnTimer();

	/** Handle for efficient management of OnTimer timer */
	FTimerHandle TimerHandle_OnTimer;

	/** Modify the timer to fire in TimeDelay seconds. A value <= 0 disables the timer. */
	AIMODULE_API virtual void SetTimer(const float TimeDelay);

	/** Calls SensePawn on any Pawns that we are allowed to sense. */
	AIMODULE_API virtual void UpdateAISensing();

	AIMODULE_API AActor* GetSensorActor() const;	// Get the actor used as the actual sensor location is derived from this actor.
	AIMODULE_API AController* GetSensorController() const; // Get the controller of the sensor actor.

public:
	/** Delegate to execute when we see a Pawn. */
	UPROPERTY(BlueprintAssignable)
	FSeePawnDelegate OnSeePawn;

	/** Delegate to execute when we hear a noise from a Pawn's PawnNoiseEmitterComponent. */
	UPROPERTY(BlueprintAssignable)
	FHearNoiseDelegate OnHearNoise;

protected:

	// Broadcasts notification that our sensor sees a Pawn, using the OnSeePawn delegates.
	AIMODULE_API virtual void BroadcastOnSeePawn(APawn& Pawn);

	// Broadcasts notification that our sensor hears a noise made local to a Pawn's position, using the OnHearNoise delegates.
	AIMODULE_API virtual void BroadcastOnHearLocalNoise(APawn& Instigator, const FVector& Location, float Volume);

	// Broadcasts notification that our sensor hears a noise made remotely from a Pawn's position, using the OnHearNoise delegates.
	AIMODULE_API virtual void BroadcastOnHearRemoteNoise(APawn& Instigator, const FVector& Location, float Volume);

	/** How far to the side AI can see, in degrees. Use SetPeripheralVisionAngle to change the value at runtime. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AI)
	float PeripheralVisionAngle;

	/** Cosine of limits of peripheral vision. Computed from PeripheralVisionAngle. */
	UPROPERTY()
	float PeripheralVisionCosine;
};



inline float UPawnSensingComponent::GetPeripheralVisionAngle() const
{
	return PeripheralVisionAngle;
}

inline float UPawnSensingComponent::GetPeripheralVisionCosine() const
{
	return PeripheralVisionCosine;
}


