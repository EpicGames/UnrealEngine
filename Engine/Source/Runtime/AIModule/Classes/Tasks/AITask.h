// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameplayTask.h"
#include "UObject/Package.h"
#include "AITask.generated.h"

class AActor;
class AAIController;

UENUM()
enum class EAITaskPriority : uint8
{
	Lowest = 0,
	Low = 64, //FGameplayTasks::DefaultPriority / 2,
	AutonomousAI = 127, //FGameplayTasks::DefaultPriority,
	High = 192, //(1.5 * FGameplayTasks::DefaultPriority),
	Ultimate = 254,
};

UCLASS(Abstract, BlueprintType, MinimalAPI)
class UAITask : public UGameplayTask
{
	GENERATED_BODY()
protected:

	UPROPERTY(BlueprintReadOnly, Category="AI|Tasks")
	TObjectPtr<AAIController> OwnerController;

	AIMODULE_API virtual void Activate() override;

public:
	AIMODULE_API UAITask(const FObjectInitializer& ObjectInitializer);

	static AIMODULE_API AAIController* GetAIControllerForActor(AActor* Actor);
	AAIController* GetAIController() const { return OwnerController; };

	AIMODULE_API void InitAITask(AAIController& AIOwner, IGameplayTaskOwnerInterface& InTaskOwner, uint8 InPriority);
	AIMODULE_API void InitAITask(AAIController& AIOwner, IGameplayTaskOwnerInterface& InTaskOwner);

	/** effectively adds UAIResource_Logic to the set of Claimed resources */
	AIMODULE_API void RequestAILogicLocking();

	template <class T>
	static T* NewAITask(AAIController& AIOwner, IGameplayTaskOwnerInterface& InTaskOwner, FName InstanceName = FName())
	{
		return NewAITask<T>(*T::StaticClass(), AIOwner, InTaskOwner, InstanceName);
	}

	template <class T>
	static T* NewAITask(AAIController& AIOwner, IGameplayTaskOwnerInterface& InTaskOwner, EAITaskPriority InPriority, FName InstanceName = FName())
	{
		return NewAITask<T>(*T::StaticClass(), AIOwner, InTaskOwner, InPriority, InstanceName);
	}

	template <class T>
	static T* NewAITask(AAIController& AIOwner, FName InstanceName = FName())
	{
		return NewAITask<T>(*T::StaticClass(), AIOwner, AIOwner, InstanceName);
	}

	template <class T>
	static T* NewAITask(AAIController& AIOwner, EAITaskPriority InPriority, FName InstanceName = FName())
	{
		return NewAITask<T>(*T::StaticClass(), AIOwner, AIOwner, InPriority, InstanceName);
	}

	template <class T>
	static T* NewAITask(const UClass& Class, AAIController& AIOwner, IGameplayTaskOwnerInterface& InTaskOwner, FName InstanceName = FName())
	{
		T* TaskInstance = NewObject<T>(GetTransientPackage(), &Class);
		TaskInstance->InstanceName = InstanceName;
		TaskInstance->InitAITask(AIOwner, InTaskOwner);
		return TaskInstance;
	}

	template <class T>
	static T* NewAITask(const UClass& Class, AAIController& AIOwner, IGameplayTaskOwnerInterface& InTaskOwner, EAITaskPriority InPriority, FName InstanceName = FName())
	{
		T* TaskInstance = NewObject<T>(GetTransientPackage(), &Class);
		TaskInstance->InstanceName = InstanceName;
		TaskInstance->InitAITask(AIOwner, InTaskOwner, (uint8)InPriority);
		return TaskInstance;
	}
};
