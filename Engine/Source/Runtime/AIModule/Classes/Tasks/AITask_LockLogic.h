// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Tasks/AITask.h"
#include "AITask_LockLogic.generated.h"

/** Locks AI logic until removed by external trigger */
UCLASS(MinimalAPI)
class UAITask_LockLogic : public UAITask
{
	GENERATED_UCLASS_BODY()
};
