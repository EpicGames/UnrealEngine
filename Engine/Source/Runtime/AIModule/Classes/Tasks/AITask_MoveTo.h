// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "AITypes.h"
#include "Navigation/PathFollowingComponent.h"
#include "Tasks/AITask.h"
#include "AITask_MoveTo.generated.h"

class AAIController;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FMoveTaskCompletedSignature, TEnumAsByte<EPathFollowingResult::Type>, Result, AAIController*, AIController);
DECLARE_MULTICAST_DELEGATE_TwoParams(FMoveTaskFinishedSignature, TEnumAsByte<EPathFollowingResult::Type>, AAIController*);

UCLASS(MinimalAPI)
class UAITask_MoveTo : public UAITask
{
	GENERATED_BODY()

public:
	AIMODULE_API UAITask_MoveTo(const FObjectInitializer& ObjectInitializer);

	/** tries to start move request and handles retry timer */
	AIMODULE_API void ConditionalPerformMove();

	/** prepare move task for activation */
	AIMODULE_API void SetUp(AAIController* Controller, const FAIMoveRequest& InMoveRequest);

	EPathFollowingResult::Type GetMoveResult() const { return MoveResult; }
	bool WasMoveSuccessful() const { return MoveResult == EPathFollowingResult::Success; }
	bool WasMovePartial() const { return Path.IsValid() && Path->IsPartial(); }

	/**
	 * Move to a location or an actor
	 * @param ProjectGoalOnNavigation Try to move the goal to the navigation surface before requesting the move, fails if it can't.
	 * @param RequireNavigableEndLocation Set to No to allow pursuing the request even if no navigation surface is found at the goal location. 
	 */
	UFUNCTION(BlueprintCallable, Category = "AI|Tasks", meta = (AdvancedDisplay = "AcceptanceRadius,StopOnOverlap,AcceptPartialPath,bUsePathfinding,bUseContinuousGoalTracking,ProjectGoalOnNavigation,RequireNavigableEndLocation", DefaultToSelf = "Controller", BlueprintInternalUseOnly = "TRUE", DisplayName = "Move To Location or Actor"))
	static AIMODULE_API UAITask_MoveTo* AIMoveTo(AAIController* Controller, FVector GoalLocation, AActor* GoalActor = nullptr,
		float AcceptanceRadius = -1.f, EAIOptionFlag::Type StopOnOverlap = EAIOptionFlag::Default, EAIOptionFlag::Type AcceptPartialPath = EAIOptionFlag::Default,
		bool bUsePathfinding = true, bool bLockAILogic = true, bool bUseContinuousGoalTracking = false, EAIOptionFlag::Type ProjectGoalOnNavigation = EAIOptionFlag::Default,
		EAIOptionFlag::Type RequireNavigableEndLocation = EAIOptionFlag::Default);

	/** Allows custom move request tweaking. Note that all MoveRequest need to 
	 *	be performed before PerformMove is called. */
	FAIMoveRequest& GetMoveRequestRef() { return MoveRequest; }

	/** Switch task into continuous tracking mode: keep restarting move toward goal actor. Only pathfinding failure or external cancel will be able to stop this task. */
	AIMODULE_API void SetContinuousGoalTracking(bool bEnable);

	FMoveTaskFinishedSignature OnMoveTaskFinished;
protected:
	UPROPERTY(BlueprintAssignable)
	FGenericGameplayTaskDelegate OnRequestFailed;

	UPROPERTY(BlueprintAssignable)
	FMoveTaskCompletedSignature OnMoveFinished;

	/** parameters of move request */
	UPROPERTY()
	FAIMoveRequest MoveRequest;

	/** handle of path following's OnMoveFinished delegate */
	FDelegateHandle PathFinishDelegateHandle;

	/** handle of path's update event delegate */
	FDelegateHandle PathUpdateDelegateHandle;

	/** handle of active ConditionalPerformMove timer  */
	FTimerHandle MoveRetryTimerHandle;

	/** handle of active ConditionalUpdatePath timer */
	FTimerHandle PathRetryTimerHandle;

	/** request ID of path following's request */
	FAIRequestID MoveRequestID;

	/** currently followed path */
	FNavPathSharedPtr Path;

	TEnumAsByte<EPathFollowingResult::Type> MoveResult;
	uint8 bUseContinuousTracking : 1;

	AIMODULE_API virtual void Activate() override;
	AIMODULE_API virtual void OnDestroy(bool bOwnerFinished) override;

	AIMODULE_API virtual void Pause() override;
	AIMODULE_API virtual void Resume() override;

	/** finish task */
	AIMODULE_API void FinishMoveTask(EPathFollowingResult::Type InResult);

	/** stores path and starts observing its events */
	AIMODULE_API void SetObservedPath(FNavPathSharedPtr InPath);

	/** remove all delegates */
	AIMODULE_API virtual void ResetObservers();

	/** remove all timers */
	AIMODULE_API virtual void ResetTimers();

	/** tries to update invalidated path and handles retry timer */
	AIMODULE_API void ConditionalUpdatePath();

	/** start move request */
	AIMODULE_API virtual void PerformMove();

	/** event from followed path */
	AIMODULE_API virtual void OnPathEvent(FNavigationPath* InPath, ENavPathEvent::Type Event);

	/** event from path following */
	AIMODULE_API virtual void OnRequestFinished(FAIRequestID RequestID, const FPathFollowingResult& Result);
};
