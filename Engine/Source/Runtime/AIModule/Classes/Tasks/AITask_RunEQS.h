// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Tasks/AITask.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "AITask_RunEQS.generated.h"

class AAIController;
class UEnvQuery;

UCLASS(MinimalAPI)
class UAITask_RunEQS : public UAITask
{
	GENERATED_BODY()
		
public:
	AIMODULE_API UAITask_RunEQS(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	UFUNCTION(BlueprintCallable, Category = "AI|Tasks", meta = (DefaultToSelf = "Controller", BlueprintInternalUseOnly = "TRUE"))
	static AIMODULE_API UAITask_RunEQS* RunEQS(AAIController* Controller, UEnvQuery* QueryTemplate);

	void SetQueryTemplate(UEnvQuery& InQueryTemplate) { EQSRequest.QueryTemplate = &InQueryTemplate; }
	void SetNotificationDelegate(const FQueryFinishedSignature& InNotificationDelegate) { NotificationDelegate = InNotificationDelegate; }
		
protected:
	AIMODULE_API virtual void Activate() override;

	AIMODULE_API void OnEQSRequestFinished(TSharedPtr<FEnvQueryResult> Result);

	FEQSParametrizedQueryExecutionRequest EQSRequest;
	FQueryFinishedSignature EQSFinishedDelegate;
	FQueryFinishedSignature NotificationDelegate;
	TSharedPtr<FEnvQueryResult> QueryResult;
};
