// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "EngineDefines.h"
#include "VisualLogger/VisualLoggerTypes.h"
#include "VisualLoggerExtension.generated.h"

class AActor;
class UCanvas;
class UEQSRenderingComponent;
struct FLogEntryItem;

namespace EVisLogTags
{
	const FString TAG_EQS = "LogEQS";
}

#if ENABLE_VISUAL_LOG
struct FVisualLogDataBlock;
struct FLogEntryItem;
class UCanvas;

class FVisualLoggerExtension : public FVisualLogExtensionInterface
{
public:
	virtual void ResetData(IVisualLoggerEditorInterface* EdInterface) override;
	virtual void DrawData(IVisualLoggerEditorInterface* EdInterface, UCanvas* Canvas) override;
	virtual void OnItemsSelectionChanged(IVisualLoggerEditorInterface* EdInterface) override;
	virtual void OnLogLineSelectionChanged(IVisualLoggerEditorInterface* EdInterface, TSharedPtr<struct FLogEntryItem> SelectedItem, int64 UserData) override;

private:
	void DrawData(UWorld* InWorld, class UEQSRenderingComponent* EQSRenderingComponent, UCanvas* Canvas, AActor* HelperActor, const FName& TagName, const FVisualLogDataBlock& DataBlock, double Timestamp);
	void DisableEQSRendering(AActor* HelperActor);

protected:
	int32 SelectedEQSId = INDEX_NONE;
	float CurrentTimestamp = FLT_MIN;
	TArray<TWeakObjectPtr<class UEQSRenderingComponent> >	EQSRenderingComponents;
};
#endif //ENABLE_VISUAL_LOG

UCLASS(Abstract, MinimalAPI)
class UVisualLoggerExtension : public UObject
{
	GENERATED_BODY()
};
