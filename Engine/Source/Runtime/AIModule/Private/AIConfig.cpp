// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIConfig.h"

float FAIConfig::Navigation::FocalPointDistance = 500.0f;
