// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIController.h"
#include "CollisionQueryParams.h"
#include "NavigationSystem.h"
#include "Components/CapsuleComponent.h"
#include "GameFramework/PhysicsVolume.h"
#include "Engine/Canvas.h"
#include "BehaviorTree/BehaviorTreeComponent.h"
#include "Tasks/AITask.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Object.h"
#include "Perception/AIPerceptionComponent.h"
#include "VisualLogger/VisualLoggerTypes.h"
#include "VisualLogger/VisualLogger.h"
#include "GameplayTaskResource.h"
#include "AIResources.h"
#include "AIModuleLog.h"
#include "Kismet/GameplayStatics.h"
#include "DisplayDebugHelpers.h"
#include "BehaviorTree/BehaviorTree.h"
#include "GameplayTasksComponent.h"
#include "Tasks/GameplayTask_ClaimResource.h"
#include "NetworkingDistanceConstants.h"
#include "Navigation/PathFollowingComponent.h"
#include "NavFilters/NavigationQueryFilter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AIController)


//----------------------------------------------------------------------//
// AAIController
//----------------------------------------------------------------------//
bool AAIController::bAIIgnorePlayers = false;

DECLARE_CYCLE_STAT(TEXT("MoveTo"), STAT_MoveTo, STATGROUP_AI);

DEFINE_LOG_CATEGORY(LogAINavigation);

AAIController::AAIController(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSetControlRotationFromPawnOrientation = true;
	PathFollowingComponent = CreateOptionalDefaultSubobject<UPathFollowingComponent>(TEXT("PathFollowingComponent"));
	if (PathFollowingComponent)
	{
		PathFollowingComponent->OnRequestFinished.AddUObject(this, &AAIController::OnMoveCompleted);
	}

	bSkipExtraLOSChecks = true;
	bWantsPlayerState = false;
	TeamID = FGenericTeamId::NoTeam;

	bStartAILogicOnPossess = false;
	bStopAILogicOnUnposses = true;
}

void AAIController::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);

	UpdateControlRotation(DeltaTime);
}

void AAIController::PostInitializeComponents()
{
	Super::PostInitializeComponents();

	if (bWantsPlayerState && IsValidChecked(this) && (GetNetMode() != NM_Client))
	{
		InitPlayerState();
	}

	if (BrainComponent == nullptr)
	{
		BrainComponent = FindComponentByClass<UBrainComponent>();
	}
	if (Blackboard == nullptr)
	{
		Blackboard = FindComponentByClass<UBlackboardComponent>();
	}

#if ENABLE_VISUAL_LOG
	for (UActorComponent* Component : GetComponents())
	{
		if (Component)
		{
			REDIRECT_OBJECT_TO_VLOG(Component, this);
		}
	}
#endif // ENABLE_VISUAL_LOG
}

void AAIController::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	// cache PerceptionComponent if not already set
	// note that it's possible for an AI to not have a perception component at all
	if (!IsValid(PerceptionComponent))
	{
		PerceptionComponent = FindComponentByClass<UAIPerceptionComponent>();
	}
}

void AAIController::Reset()
{
	Super::Reset();

	if (PathFollowingComponent)
	{
		PathFollowingComponent->AbortMove(*this, FPathFollowingResultFlags::OwnerFinished | FPathFollowingResultFlags::ForcedScript);
	}
}

void AAIController::DisplayDebug(UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos)
{
	Super::DisplayDebug(Canvas, DebugDisplay, YL, YPos);

	if (DebugDisplay.IsDisplayOn(NAME_AI))
	{
		if (PathFollowingComponent)
		{
			PathFollowingComponent->DisplayDebug(Canvas, DebugDisplay, YL, YPos);
		}

		AActor* FocusActor = GetFocusActor();
		if (FocusActor)
		{
			FDisplayDebugManager& DisplayDebugManager = Canvas->DisplayDebugManager;
			DisplayDebugManager.DrawString(FString::Printf(TEXT("      Focus %s"), *FocusActor->GetName()));
		}
	}
}

#if ENABLE_VISUAL_LOG

void AAIController::GrabDebugSnapshot(FVisualLogEntry* Snapshot) const
{
	FVisualLogStatusCategory MyCategory;
	MyCategory.Category = TEXT("AI Controller");
	MyCategory.Add(TEXT("Pawn"), GetNameSafe(GetPawn()));
	AActor* FocusActor = GetFocusActor();
	MyCategory.Add(TEXT("Focus"), GetDebugName(FocusActor));

	if (FocusActor == nullptr)
	{
		MyCategory.Add(TEXT("Focus Location"), TEXT_AI_LOCATION(GetFocalPoint()));
	}
	Snapshot->Status.Add(MyCategory);

	if (GetPawn())
	{
		Snapshot->Location = GetPawn()->GetActorLocation();
	}

	if (PathFollowingComponent)
	{
		PathFollowingComponent->DescribeSelfToVisLog(Snapshot);
	}

	if (BrainComponent != nullptr)
	{
		BrainComponent->DescribeSelfToVisLog(Snapshot);
	}

	if (PerceptionComponent != nullptr)
	{
		PerceptionComponent->DescribeSelfToVisLog(Snapshot);
	}
}
#endif // ENABLE_VISUAL_LOG

void AAIController::SetFocalPoint(FVector NewFocus, EAIFocusPriority::Type InPriority)
{
	// clear out existing
	ClearFocus(InPriority);

	// now set new focus
	if (InPriority >= FocusInformation.Priorities.Num())
	{
		FocusInformation.Priorities.SetNum(InPriority + 1);
	}
	
	FFocusKnowledge::FFocusItem& FocusItem = FocusInformation.Priorities[InPriority];
	FocusItem.Position = NewFocus;
}

FVector AAIController::GetFocalPointForPriority(EAIFocusPriority::Type InPriority) const
{
	FVector Result = FAISystem::InvalidLocation;

	if (InPriority < FocusInformation.Priorities.Num())
	{
		const FFocusKnowledge::FFocusItem& FocusItem = FocusInformation.Priorities[InPriority];

		AActor* FocusActor = FocusItem.Actor.Get();
		if (FocusActor)
		{
			Result = GetFocalPointOnActor(FocusActor);
		}
		else
		{
			Result = FocusItem.Position;
		}
	}

	return Result;
}

FVector AAIController::GetFocalPoint() const
{
	FVector Result = FAISystem::InvalidLocation;

	// find focus with highest priority
	for (int32 Index = FocusInformation.Priorities.Num() - 1; Index >= 0; --Index)
	{
		const FFocusKnowledge::FFocusItem& FocusItem = FocusInformation.Priorities[Index];
		AActor* FocusActor = FocusItem.Actor.Get();
		if (FocusActor)
		{
			Result = GetFocalPointOnActor(FocusActor);
			break;
		}
		else if (FAISystem::IsValidLocation(FocusItem.Position))
		{
			Result = FocusItem.Position;
			break;
		}
	}

	return Result;
}

AActor* AAIController::GetFocusActor() const
{
	AActor* FocusActor = nullptr;
	for (int32 Index = FocusInformation.Priorities.Num() - 1; Index >= 0; --Index)
	{
		const FFocusKnowledge::FFocusItem& FocusItem = FocusInformation.Priorities[Index];
		FocusActor = FocusItem.Actor.Get();
		if (FocusActor)
		{
			break;
		}
		else if (FAISystem::IsValidLocation(FocusItem.Position))
		{
			break;
		}
	}

	return FocusActor;
}

FVector AAIController::GetFocalPointOnActor(const AActor *Actor) const
{
	return Actor != nullptr ? Actor->GetActorLocation() : FAISystem::InvalidLocation;
}

void AAIController::K2_SetFocus(AActor* NewFocus)
{
	SetFocus(NewFocus, EAIFocusPriority::Gameplay);
}

void AAIController::K2_SetFocalPoint(FVector NewFocus)
{
	SetFocalPoint(NewFocus, EAIFocusPriority::Gameplay);
}

void AAIController::K2_ClearFocus()
{
	ClearFocus(EAIFocusPriority::Gameplay);
}

void AAIController::SetFocus(AActor* NewFocus, EAIFocusPriority::Type InPriority)
{
	// clear out existing
	ClearFocus(InPriority);

	// now set new
	if (NewFocus)
	{
		if (InPriority >= FocusInformation.Priorities.Num())
		{
			FocusInformation.Priorities.SetNum(InPriority + 1);
		}
		FocusInformation.Priorities[InPriority].Actor = NewFocus;
	}
}

void AAIController::ClearFocus(EAIFocusPriority::Type InPriority)
{
	if (InPriority < FocusInformation.Priorities.Num())
	{
		FocusInformation.Priorities[InPriority].Actor = nullptr;
		FocusInformation.Priorities[InPriority].Position = FAISystem::InvalidLocation;
	}
}

void AAIController::SetPerceptionComponent(UAIPerceptionComponent& InPerceptionComponent)
{
	if (PerceptionComponent != nullptr)
	{
		UE_VLOG(this, LogAIPerception, Warning, TEXT("Setting perception component while AIController already has one!"));
	}
	PerceptionComponent = &InPerceptionComponent;
}

bool AAIController::LineOfSightTo(const AActor* Other, FVector ViewPoint, bool bAlternateChecks) const
{
	if (Other == nullptr)
	{
		return false;
	}

	if (ViewPoint.IsZero())
	{
		FRotator ViewRotation;
		GetActorEyesViewPoint(ViewPoint, ViewRotation);

		// if we still don't have a view point we simply fail
		if (ViewPoint.IsZero())
		{
			return false;
		}
	}

	FVector TargetLocation = Other->GetTargetLocation(GetPawn());

	FCollisionQueryParams CollisionParams(SCENE_QUERY_STAT(LineOfSight), true, this->GetPawn());
	CollisionParams.AddIgnoredActor(Other);

	bool bHit = GetWorld()->LineTraceTestByChannel(ViewPoint, TargetLocation, ECC_Visibility, CollisionParams);
	if (!bHit)
	{
		return true;
	}

	// if other isn't using a cylinder for collision and isn't a Pawn (which already requires an accurate cylinder for AI)
	// then don't go any further as it likely will not be tracing to the correct location
	const APawn * OtherPawn = Cast<const APawn>(Other);
	if (!OtherPawn && Cast<UCapsuleComponent>(Other->GetRootComponent()) == NULL)
	{
		return false;
	}

	const FVector OtherActorLocation = Other->GetActorLocation();
	const FVector::FReal DistSq = (OtherActorLocation - ViewPoint).SizeSquared();
	if (DistSq > FARSIGHTTHRESHOLDSQUARED)
	{
		return false;
	}

	if (!OtherPawn && (DistSq > NEARSIGHTTHRESHOLDSQUARED))
	{
		return false;
	}

	float OtherRadius, OtherHeight;
	Other->GetSimpleCollisionCylinder(OtherRadius, OtherHeight);

	if (!bAlternateChecks || !bLOSflag)
	{
		//try viewpoint to head
		bHit = GetWorld()->LineTraceTestByChannel(ViewPoint, OtherActorLocation + FVector(0.f, 0.f, OtherHeight), ECC_Visibility, CollisionParams);
		if (!bHit)
		{
			return true;
		}
	}

	if (!bSkipExtraLOSChecks && (!bAlternateChecks || bLOSflag))
	{
		// only check sides if width of other is significant compared to distance
		if (OtherRadius * OtherRadius / (OtherActorLocation - ViewPoint).SizeSquared() < 0.0001f)
		{
			return false;
		}
		//try checking sides - look at dist to four side points, and cull furthest and closest
		FVector Points[4];
		Points[0] = OtherActorLocation - FVector(OtherRadius, -1 * OtherRadius, 0);
		Points[1] = OtherActorLocation + FVector(OtherRadius, OtherRadius, 0);
		Points[2] = OtherActorLocation - FVector(OtherRadius, OtherRadius, 0);
		Points[3] = OtherActorLocation + FVector(OtherRadius, -1 * OtherRadius, 0);
		int32 IndexMin = 0;
		int32 IndexMax = 0;
		FVector::FReal CurrentMax = (Points[0] - ViewPoint).SizeSquared();
		FVector::FReal CurrentMin = CurrentMax;
		for (int32 PointIndex = 1; PointIndex<4; PointIndex++)
		{
			const FVector::FReal NextSize = (Points[PointIndex] - ViewPoint).SizeSquared();
			if (NextSize > CurrentMin)
			{
				CurrentMin = NextSize;
				IndexMax = PointIndex;
			}
			else if (NextSize < CurrentMax)
			{
				CurrentMax = NextSize;
				IndexMin = PointIndex;
			}
		}

		for (int32 PointIndex = 0; PointIndex<4; PointIndex++)
		{
			if ((PointIndex != IndexMin) && (PointIndex != IndexMax))
			{
				bHit = GetWorld()->LineTraceTestByChannel(ViewPoint, Points[PointIndex], ECC_Visibility, CollisionParams);
				if (!bHit)
				{
					return true;
				}
			}
		}
	}
	return false;
}

void AAIController::ActorsPerceptionUpdated(const TArray<AActor*>& UpdatedActors)
{

}

DEFINE_LOG_CATEGORY_STATIC(LogTestAI, All, All);

void AAIController::SetControlRotation(const FRotator& NewRotation)
{
	if (!IsValidControlRotation(NewRotation))
	{
		logOrEnsureNanError(TEXT("AAIController::SetControlRotation attempted to apply NaN-containing or NaN-causing rotation! (%s)"), *NewRotation.ToString());
		return;
	}

	if (!ControlRotation.Equals(NewRotation, 1e-3f))
	{
		ControlRotation = NewRotation;

		if (!bSkipRootComponentUpdate && RootComponent && RootComponent->IsUsingAbsoluteRotation())
		{
			RootComponent->SetWorldRotation(GetControlRotation());
		}
	}
}

void AAIController::UpdateControlRotation(float DeltaTime, bool bUpdatePawn)
{
	APawn* const MyPawn = GetPawn();
	if (MyPawn)
	{
		FRotator NewControlRotation = GetControlRotation();

		// Look toward focus
		const FVector FocalPoint = GetFocalPoint();
		if (FAISystem::IsValidLocation(FocalPoint))
		{
			NewControlRotation = (FocalPoint - MyPawn->GetPawnViewLocation()).Rotation();
		}
		else if (bSetControlRotationFromPawnOrientation)
		{
			NewControlRotation = MyPawn->GetActorRotation();
		}

		// Don't pitch view unless looking at another pawn
		if (NewControlRotation.Pitch != 0 && Cast<APawn>(GetFocusActor()) == nullptr)
		{
			NewControlRotation.Pitch = 0.f;
		}

		SetControlRotation(NewControlRotation);

		if (bUpdatePawn)
		{
			const FRotator CurrentPawnRotation = MyPawn->GetActorRotation();

			if (CurrentPawnRotation.Equals(NewControlRotation, 1e-3f) == false)
			{
				MyPawn->FaceRotation(NewControlRotation, DeltaTime);
			}
		}
	}
}


void AAIController::OnPossess(APawn* InPawn)
{
	// don't even try possessing pending-kill pawns
	if (InPawn != nullptr && !IsValid(InPawn))
	{
		return;
	}

	Super::OnPossess(InPawn);

	if (GetPawn() == nullptr || InPawn == nullptr)
	{
		return;
	}

	// not calling UpdateNavigationComponents() anymore. The PathFollowingComponent 
	// is now observing newly possessed pawns (via OnNewPawn)

	if (PathFollowingComponent)
	{
		PathFollowingComponent->Initialize();
	}

	if (bWantsPlayerState)
	{
		ChangeState(NAME_Playing);
	}

	// a Pawn controlled by AI _requires_ a GameplayTasksComponent, so if Pawn 
	// doesn't have one we need to create it
	if (CachedGameplayTasksComponent == nullptr)
	{
		UGameplayTasksComponent* GTComp = InPawn->FindComponentByClass<UGameplayTasksComponent>();
		if (GTComp == nullptr)
		{
			GTComp = NewObject<UGameplayTasksComponent>(InPawn, TEXT("GameplayTasksComponent"));
			GTComp->RegisterComponent();
		}
		CachedGameplayTasksComponent = GTComp;
	}

	if (CachedGameplayTasksComponent && !CachedGameplayTasksComponent->OnClaimedResourcesChange.Contains(this, GET_FUNCTION_NAME_CHECKED(AAIController, OnGameplayTaskResourcesClaimed)))
	{
		CachedGameplayTasksComponent->OnClaimedResourcesChange.AddDynamic(this, &AAIController::OnGameplayTaskResourcesClaimed);
	}

	if (Blackboard && Blackboard->GetBlackboardAsset())
	{
		InitializeBlackboard(*Blackboard, *Blackboard->GetBlackboardAsset());
	}

	if (bStartAILogicOnPossess && BrainComponent)
	{
		BrainComponent->StartLogic();
	}
}

void AAIController::OnUnPossess()
{
	const bool bGameplayTaskCleanupRequired = (CachedGameplayTasksComponent && (CachedGameplayTasksComponent->GetOwner() == GetPawn()));

	// End all tasks owned by the controller before calling OnUnPossess on the base class
	// since those tasks might require access to the pawn's associated controller to exit properly.
	// Also called before cleaning up the BrainComponent
	// since it might need to update its resource lock (see OnGameplayTaskResourcesClaimed)
	if (bGameplayTaskCleanupRequired)
	{
		CachedGameplayTasksComponent->EndAllResourceConsumingTasksOwnedBy(*this);
	}

	Super::OnUnPossess();

	if (PathFollowingComponent)
	{
		PathFollowingComponent->Cleanup();
	}

	if (bStopAILogicOnUnposses)
	{
		CleanupBrainComponent();
	}

	// Can now unregister callback and clear the cache
	if (bGameplayTaskCleanupRequired)
	{
		CachedGameplayTasksComponent->OnClaimedResourcesChange.RemoveDynamic(this, &AAIController::OnGameplayTaskResourcesClaimed);
		CachedGameplayTasksComponent = nullptr;
	}
}

void AAIController::SetPawn(APawn* InPawn)
{
	Super::SetPawn(InPawn);

	if (Blackboard)
	{
		const UBlackboardData* BBAsset = Blackboard->GetBlackboardAsset();
		if (BBAsset)
		{
			const FBlackboard::FKey SelfKey = BBAsset->GetKeyID(FBlackboard::KeySelf);
			if (SelfKey != FBlackboard::InvalidKey)
			{
				Blackboard->SetValue<UBlackboardKeyType_Object>(SelfKey, GetPawn());
			}
		}
	}
}

EPathFollowingRequestResult::Type AAIController::MoveToActor(AActor* Goal, float AcceptanceRadius, bool bStopOnOverlap, bool bUsePathfinding, bool bCanStrafe, TSubclassOf<UNavigationQueryFilter> FilterClass, bool bAllowPartialPaths)
{
	// abort active movement to keep only one request running
	if (PathFollowingComponent && PathFollowingComponent->GetStatus() != EPathFollowingStatus::Idle)
	{
		PathFollowingComponent->AbortMove(*this, FPathFollowingResultFlags::ForcedScript | FPathFollowingResultFlags::NewRequest
			, FAIRequestID::CurrentRequest, EPathFollowingVelocityMode::Keep);
	}

	FAIMoveRequest MoveReq(Goal);
	MoveReq.SetUsePathfinding(bUsePathfinding);
	MoveReq.SetAllowPartialPath(bAllowPartialPaths);
	MoveReq.SetNavigationFilter(*FilterClass ? FilterClass : DefaultNavigationFilterClass);
	MoveReq.SetAcceptanceRadius(AcceptanceRadius);
	MoveReq.SetReachTestIncludesAgentRadius(bStopOnOverlap);
	MoveReq.SetCanStrafe(bCanStrafe);

	return MoveTo(MoveReq);
}

EPathFollowingRequestResult::Type AAIController::MoveToLocation(const FVector& Dest, float AcceptanceRadius, bool bStopOnOverlap, bool bUsePathfinding, bool bProjectDestinationToNavigation, bool bCanStrafe, TSubclassOf<UNavigationQueryFilter> FilterClass, bool bAllowPartialPaths)
{
	// abort active movement to keep only one request running
	if (PathFollowingComponent && PathFollowingComponent->GetStatus() != EPathFollowingStatus::Idle)
	{
		PathFollowingComponent->AbortMove(*this, FPathFollowingResultFlags::ForcedScript | FPathFollowingResultFlags::NewRequest
			, FAIRequestID::CurrentRequest, EPathFollowingVelocityMode::Keep);
	}

	FAIMoveRequest MoveReq(Dest);
	MoveReq.SetUsePathfinding(bUsePathfinding);
	MoveReq.SetAllowPartialPath(bAllowPartialPaths);
	MoveReq.SetProjectGoalLocation(bProjectDestinationToNavigation);
	MoveReq.SetNavigationFilter(*FilterClass ? FilterClass : DefaultNavigationFilterClass);
	MoveReq.SetAcceptanceRadius(AcceptanceRadius);
	MoveReq.SetReachTestIncludesAgentRadius(bStopOnOverlap);
	MoveReq.SetCanStrafe(bCanStrafe);

	return MoveTo(MoveReq);
}

void AAIController::MergePaths(const FNavPathSharedPtr& InitialPath, FNavPathSharedPtr& InOutMergedPath) const
{
	if (!InitialPath.IsValid() || !InitialPath->IsValid())
	{
		UE_VLOG_UELOG(this, LogAINavigation, Error, TEXT("%hs: InitialPath is Invalid"), __FUNCTION__);
		return;
	}

	if (!InOutMergedPath.IsValid() || !InOutMergedPath->IsValid())
	{
		UE_VLOG_UELOG(this, LogAINavigation, Error, TEXT("%hs: InOutMergedPath is Invalid"), __FUNCTION__);
		return;
	}

	const TArray<FNavPathPoint>& InitialPathPoints = InitialPath->GetPathPoints();
	TArray<FNavPathPoint>& InOutPathPoints = InOutMergedPath->GetPathPoints();

	if (!InitialPathPoints.Last().Location.Equals(InOutPathPoints[0].Location))
	{
		UE_VLOG_UELOG(this, LogAINavigation, Error, TEXT("%hs: last %s and first %s points don't match."), __FUNCTION__, *InitialPathPoints.Last().Location.ToString(), *InOutPathPoints[0].Location.ToString());
		return;
	}

	// We don't want to keep path points that have already been traversed, so only merge the points starting from "CurrentPathIndex".
	const int32 StartingPointIndex = PathFollowingComponent ? PathFollowingComponent->GetCurrentPathIndex() : 0;
	if (StartingPointIndex < InitialPathPoints.Num())
	{
		InOutPathPoints.Insert(&InitialPathPoints[StartingPointIndex], InitialPathPoints.Num() - StartingPointIndex - 1, 0);
	}
}

FPathFollowingRequestResult AAIController::MoveTo(const FAIMoveRequest& MoveRequest, FNavPathSharedPtr* OutPath)
{
	// both MoveToActor and MoveToLocation can be called from blueprints/script and should keep only single movement request at the same time.
	// this function is entry point of all movement mechanics - do NOT abort in here, since movement may be handled by AITasks, which support stacking 

	SCOPE_CYCLE_COUNTER(STAT_MoveTo);
	UE_VLOG(this, LogAINavigation, Log, TEXT("MoveTo: %s"), *MoveRequest.ToString());

	FPathFollowingRequestResult ResultData;
	ResultData.Code = EPathFollowingRequestResult::Failed;

	if (MoveRequest.IsValid() == false)
	{
		UE_VLOG(this, LogAINavigation, Error, TEXT("MoveTo request failed due MoveRequest not being valid. Most probably desired Goal Actor not longer exists. MoveRequest: '%s'"), *MoveRequest.ToString());
		return ResultData;
	}

	if (PathFollowingComponent == nullptr)
	{
		UE_VLOG(this, LogAINavigation, Error, TEXT("MoveTo request failed due missing PathFollowingComponent"));
		return ResultData;
	}

	bool bCanRequestMove = true;
	bool bAlreadyAtGoal = false;
	
	if (!MoveRequest.IsMoveToActorRequest())
	{
		if (MoveRequest.GetGoalLocation().ContainsNaN() || FAISystem::IsValidLocation(MoveRequest.GetGoalLocation()) == false)
		{
			UE_VLOG(this, LogAINavigation, Error, TEXT("AAIController::MoveTo: Destination is not valid! Goal(%s)"), TEXT_AI_LOCATION(MoveRequest.GetGoalLocation()));
			bCanRequestMove = false;
		}

		// fail if projection to navigation is required but it failed
		if (bCanRequestMove && MoveRequest.IsProjectingGoal())
		{
			UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
			const FNavAgentProperties& AgentProps = GetNavAgentPropertiesRef();
			FNavLocation ProjectedLocation;

			if (NavSys && !NavSys->ProjectPointToNavigation(MoveRequest.GetGoalLocation(), ProjectedLocation, INVALID_NAVEXTENT, &AgentProps))
			{
				if (MoveRequest.IsUsingPathfinding())
				{
					UE_VLOG_LOCATION(this, LogAINavigation, Error, MoveRequest.GetGoalLocation(), 30.f, FColor::Red, TEXT("AAIController::MoveTo failed to project destination location to navmesh"));
				}
				else
				{
					UE_VLOG_LOCATION(this, LogAINavigation, Error, MoveRequest.GetGoalLocation(), 30.f, FColor::Red, TEXT("AAIController::MoveTo failed to project destination location to navmesh, path finding is disabled perhaps disable goal projection ?"));
				}

				bCanRequestMove = false;
			}

			MoveRequest.UpdateGoalLocation(ProjectedLocation.Location);
		}

		bAlreadyAtGoal = bCanRequestMove && PathFollowingComponent->HasReached(MoveRequest);
	}
	else 
	{
		bAlreadyAtGoal = bCanRequestMove && PathFollowingComponent->HasReached(MoveRequest);
	}

	if (bAlreadyAtGoal)
	{
		UE_VLOG(this, LogAINavigation, Log, TEXT("MoveTo: already at goal!"));
		ResultData.MoveId = PathFollowingComponent->RequestMoveWithImmediateFinish(EPathFollowingResult::Success);
		ResultData.Code = EPathFollowingRequestResult::AlreadyAtGoal;
	}
	else if (bCanRequestMove)
	{
		FPathFindingQuery PFQuery;

		bool bShouldMergePaths = false;
		FVector StartLocation = GetNavAgentLocation();
		if (MoveRequest.ShouldStartFromPreviousPath())
		{
			FNavPathSharedPtr CurrentPath = PathFollowingComponent->GetPath();
			if (CurrentPath.IsValid() && CurrentPath->IsValid() && CurrentPath->GetPathPoints().Num() > 0)
			{
				StartLocation = CurrentPath->GetPathPoints().Last();
				bShouldMergePaths = true;
			}
		}

		const bool bValidQuery = BuildPathfindingQuery(MoveRequest, StartLocation, PFQuery);
		if (bValidQuery)
		{
			FNavPathSharedPtr Path;
			FindPathForMoveRequest(MoveRequest, PFQuery, Path);

			if (bShouldMergePaths && Path.IsValid())
			{
				// Merge the newly generated path with the current one
				MergePaths(PathFollowingComponent->GetPath(), Path);
			}

			const FAIRequestID RequestID = Path.IsValid() ? RequestMove(MoveRequest, Path) : FAIRequestID::InvalidRequest;
			if (RequestID.IsValid())
			{
				bAllowStrafe = MoveRequest.CanStrafe();
				ResultData.MoveId = RequestID;
				ResultData.Code = EPathFollowingRequestResult::RequestSuccessful;

				if (OutPath)
				{
					*OutPath = Path;
				}
			}
		}
	}

	if (ResultData.Code == EPathFollowingRequestResult::Failed)
	{
		ResultData.MoveId = PathFollowingComponent->RequestMoveWithImmediateFinish(EPathFollowingResult::Invalid);
	}

	return ResultData;
}

FAIRequestID AAIController::RequestMove(const FAIMoveRequest& MoveRequest, FNavPathSharedPtr Path)
{
	uint32 RequestID = FAIRequestID::InvalidRequest;
	if (PathFollowingComponent)
	{
		RequestID = PathFollowingComponent->RequestMove(MoveRequest, Path);
	}

	return RequestID;
}

bool AAIController::PauseMove(FAIRequestID RequestToPause)
{
	if (PathFollowingComponent != NULL && RequestToPause.IsEquivalent(PathFollowingComponent->GetCurrentRequestId()))
	{
		PathFollowingComponent->PauseMove(RequestToPause, EPathFollowingVelocityMode::Reset);
		return true;
	}
	return false;
}

bool AAIController::ResumeMove(FAIRequestID RequestToResume)
{
	if (PathFollowingComponent != NULL && RequestToResume.IsEquivalent(PathFollowingComponent->GetCurrentRequestId()))
	{
		PathFollowingComponent->ResumeMove(RequestToResume);
		return true;
	}
	return false;
}

void AAIController::StopMovement()
{
	// @note FPathFollowingResultFlags::ForcedScript added to make AITask_MoveTo instances 
	// not ignore OnRequestFinished notify that's going to be sent out due to this call
	if (PathFollowingComponent)
	{
	    PathFollowingComponent->AbortMove(*this, FPathFollowingResultFlags::MovementStop | FPathFollowingResultFlags::ForcedScript);
	}
}

bool AAIController::ShouldPostponePathUpdates() const
{
	if (PathFollowingComponent && PathFollowingComponent->HasStartedNavLinkMove())
	{
		return true;
	}
	return Super::ShouldPostponePathUpdates();
}

bool AAIController::BuildPathfindingQuery(const FAIMoveRequest& MoveRequest, FPathFindingQuery& OutQuery) const
{
	return BuildPathfindingQuery(MoveRequest, GetNavAgentLocation(), OutQuery);
}

bool AAIController::BuildPathfindingQuery(const FAIMoveRequest& MoveRequest, const FVector& StartLocation, FPathFindingQuery& OutQuery) const
{
	bool bResult = false;

	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
	const ANavigationData* NavData = (NavSys == nullptr) ? nullptr :
		MoveRequest.IsUsingPathfinding() ? NavSys->GetNavDataForProps(GetNavAgentPropertiesRef(), GetNavAgentLocation()) :
		NavSys->GetAbstractNavData();

	if (NavData)
	{
		FVector GoalLocation = MoveRequest.GetGoalLocation();
		if (MoveRequest.IsMoveToActorRequest())
		{
			const INavAgentInterface* NavGoal = Cast<const INavAgentInterface>(MoveRequest.GetGoalActor());
			if (NavGoal)
			{
				const FVector Offset = NavGoal->GetMoveGoalOffset(this);
				GoalLocation = FQuatRotationTranslationMatrix(MoveRequest.GetGoalActor()->GetActorQuat(), NavGoal->GetNavAgentLocation()).TransformPosition(Offset);
			}
			else
			{
				GoalLocation = MoveRequest.GetGoalActor()->GetActorLocation();
			}
		}

		FSharedConstNavQueryFilter NavFilter = UNavigationQueryFilter::GetQueryFilter(*NavData, this, MoveRequest.GetNavigationFilter());
		OutQuery = FPathFindingQuery(*this, *NavData, StartLocation, GoalLocation, NavFilter);
		OutQuery.SetAllowPartialPaths(MoveRequest.IsUsingPartialPaths());
		OutQuery.SetRequireNavigableEndLocation(MoveRequest.IsNavigableEndLocationRequired());
		if (MoveRequest.IsApplyingCostLimitFromHeuristic())
		{
			const float HeuristicScale = NavFilter->GetHeuristicScale();
			OutQuery.CostLimit = FPathFindingQuery::ComputeCostLimitFromHeuristic(OutQuery.StartLocation, OutQuery.EndLocation, HeuristicScale, MoveRequest.GetCostLimitFactor(), MoveRequest.GetMinimumCostLimit()); 
		}

		if (PathFollowingComponent)
		{
			PathFollowingComponent->OnPathfindingQuery(OutQuery);
		}

		bResult = true;
	}
	else
	{
		if (NavSys == nullptr)
		{
			UE_VLOG(this, LogAINavigation, Warning, TEXT("Unable AAIController::BuildPathfindingQuery due to no NavigationSystem present. Note that even pathfinding-less movement requires presence of NavigationSystem."));
		}
		else 
		{
			UE_VLOG(this, LogAINavigation, Warning, TEXT("Unable to find NavigationData instance while calling AAIController::BuildPathfindingQuery"));
		}
	}

	return bResult;
}

void AAIController::FindPathForMoveRequest(const FAIMoveRequest& MoveRequest, FPathFindingQuery& Query, FNavPathSharedPtr& OutPath) const
{
	SCOPE_CYCLE_COUNTER(STAT_AI_Overall);

	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
	if (NavSys)
	{
		FPathFindingResult PathResult = NavSys->FindPathSync(Query);
		if (PathResult.Result != ENavigationQueryResult::Error)
		{
			if (PathResult.IsSuccessful() && PathResult.Path.IsValid())
			{
				if (MoveRequest.IsMoveToActorRequest())
				{
					PathResult.Path->SetGoalActorObservation(*MoveRequest.GetGoalActor(), 100.0f);
				}

				PathResult.Path->EnableRecalculationOnInvalidation(true);
				OutPath = PathResult.Path;
			}
		}
		else
		{
			UE_VLOG(this, LogAINavigation, Error, TEXT("Trying to find path to %s resulted in Error")
				, MoveRequest.IsMoveToActorRequest() ? *GetNameSafe(MoveRequest.GetGoalActor()) : *MoveRequest.GetGoalLocation().ToString());
			UE_VLOG_SEGMENT(this, LogAINavigation, Error, GetPawn() ? GetPawn()->GetActorLocation() : FAISystem::InvalidLocation
				, MoveRequest.GetGoalLocation(), FColor::Red, TEXT("Failed move to %s"), *GetNameSafe(MoveRequest.GetGoalActor()));
		}
	}
}

// DEPRECATED FUNCTION SUPPORT
bool AAIController::PreparePathfinding(const FAIMoveRequest& MoveRequest, FPathFindingQuery& Query)
{
	return BuildPathfindingQuery(MoveRequest, Query);
}

// DEPRECATED FUNCTION SUPPORT
FAIRequestID AAIController::RequestPathAndMove(const FAIMoveRequest& MoveRequest, FPathFindingQuery& Query)
{
	FAIRequestID MoveId = FAIRequestID::InvalidRequest;

	FNavPathSharedPtr FoundPath;
	FindPathForMoveRequest(MoveRequest, Query, FoundPath);

	if (FoundPath.IsValid())
	{
		MoveId = RequestMove(MoveRequest, FoundPath);
	}

	return MoveId;
}

EPathFollowingStatus::Type AAIController::GetMoveStatus() const
{
	return (PathFollowingComponent) ? PathFollowingComponent->GetStatus() : EPathFollowingStatus::Idle;
}

bool AAIController::HasPartialPath() const
{
	return (PathFollowingComponent != NULL) && (PathFollowingComponent->HasPartialPath());
}

bool AAIController::IsFollowingAPath() const
{
	return (PathFollowingComponent != nullptr) && (PathFollowingComponent->GetStatus() != EPathFollowingStatus::Idle);
}

IPathFollowingAgentInterface* AAIController::GetPathFollowingAgent() const
{
	return PathFollowingComponent;
}

FVector AAIController::GetImmediateMoveDestination() const
{
	return (PathFollowingComponent) ? PathFollowingComponent->GetCurrentTargetLocation() : FVector::ZeroVector;
}

void AAIController::SetMoveBlockDetection(bool bEnable)
{
	if (PathFollowingComponent)
	{
		PathFollowingComponent->SetBlockDetectionState(bEnable);
	}
}

void AAIController::OnMoveCompleted(FAIRequestID RequestID, const FPathFollowingResult& Result)
{
	ReceiveMoveCompleted.Broadcast(RequestID, Result.Code);
	OnMoveCompleted(RequestID, Result.Code);
}

void AAIController::OnMoveCompleted(FAIRequestID RequestID, EPathFollowingResult::Type Result)
{
	// deprecated
}

FAIRequestID AAIController::GetCurrentMoveRequestID() const
{
	return GetPathFollowingComponent() ? GetPathFollowingComponent()->GetCurrentRequestId() : FAIRequestID::InvalidRequest;
}

bool AAIController::RunBehaviorTree(UBehaviorTree* BTAsset)
{
	// @todo: find BrainComponent and see if it's BehaviorTreeComponent
	// Also check if BTAsset requires BlackBoardComponent, and if so 
	// check if BB type is accepted by BTAsset.
	// Spawn BehaviorTreeComponent if none present. 
	// Spawn BlackBoardComponent if none present, but fail if one is present but is not of compatible class
	if (BTAsset == NULL)
	{
		UE_VLOG(this, LogBehaviorTree, Warning, TEXT("RunBehaviorTree: Unable to run NULL behavior tree"));
		return false;
	}

	bool bSuccess = true;

	// see if need a blackboard component at all
	UBlackboardComponent* BlackboardComp = Blackboard;
	if (BTAsset->BlackboardAsset && (Blackboard == nullptr || Blackboard->IsCompatibleWith(BTAsset->BlackboardAsset) == false))
	{
		bSuccess = UseBlackboard(BTAsset->BlackboardAsset, BlackboardComp);
	}

	if (bSuccess)
	{
		UBehaviorTreeComponent* BTComp = Cast<UBehaviorTreeComponent>(BrainComponent);
		if (BTComp == NULL)
		{
			UE_VLOG(this, LogBehaviorTree, Log, TEXT("RunBehaviorTree: spawning BehaviorTreeComponent.."));

			BTComp = NewObject<UBehaviorTreeComponent>(this);
			BTComp->RegisterComponent();
			REDIRECT_OBJECT_TO_VLOG(BTComp, this);
		}
		
		// make sure BrainComponent points at the newly created BT component
		BrainComponent = BTComp;

		check(BTComp != NULL);
		BTComp->StartTree(*BTAsset, EBTExecutionMode::Looped);
	}

	return bSuccess;
}

void AAIController::CleanupBrainComponent()
{
	if (BrainComponent)
	{
		BrainComponent->Cleanup();
	}
}

void AAIController::ClaimTaskResource(TSubclassOf<UGameplayTaskResource> ResourceClass)
{
	if (CachedGameplayTasksComponent)
	{
		const uint8 ResourceID = UGameplayTaskResource::GetResourceID(ResourceClass);
		if (ScriptClaimedResources.HasID(ResourceID) == false)
		{
			ScriptClaimedResources.AddID(ResourceID);

			UE_VLOG(this, LogGameplayTasks, Log, TEXT("ClaimTaskResource %s"), *GetNameSafe(*ResourceClass));
			
			IGameplayTaskOwnerInterface& AsTaskOwner = *this;
			UGameplayTask_ClaimResource* ResourceTask = UGameplayTask_ClaimResource::ClaimResource(AsTaskOwner, ResourceClass, uint8(EAITaskPriority::High), ResourceClass->GetFName());
			if (ResourceTask)
			{
				CachedGameplayTasksComponent->AddTaskReadyForActivation(*ResourceTask);
			}
			UE_CVLOG(ResourceTask == nullptr, this, LogGameplayTasks, Warning, TEXT("ClaimTaskResource failed to create UGameplayTask_ClaimResource instance"));
		}
	}
}

void AAIController::UnclaimTaskResource(TSubclassOf<UGameplayTaskResource> ResourceClass)
{
	if (CachedGameplayTasksComponent)
	{
		const uint8 ResourceID = UGameplayTaskResource::GetResourceID(ResourceClass);
		if (ScriptClaimedResources.HasID(ResourceID) == true)
		{
			ScriptClaimedResources.RemoveID(ResourceID);
			
			UE_VLOG(this, LogGameplayTasks, Log, TEXT("UnclaimTaskResource %s"), *GetNameSafe(*ResourceClass));

			UGameplayTask* ResourceTask = CachedGameplayTasksComponent->FindResourceConsumingTaskByName(ResourceClass->GetFName());
			if (ResourceTask)
			{
				ResourceTask->EndTask();
			}
			UE_CVLOG(ResourceTask == nullptr, this, LogGameplayTasks, Warning, TEXT("UnclaimTaskResource failed to find UGameplayTask_ClaimResource instance"));
		}
	}
}

bool AAIController::InitializeBlackboard(UBlackboardComponent& BlackboardComp, UBlackboardData& BlackboardAsset)
{
	check(BlackboardComp.GetOwner() == this);

	if (BlackboardComp.InitializeBlackboard(BlackboardAsset))
	{
		// find the "self" key and set it to our pawn
		const FBlackboard::FKey SelfKey = BlackboardAsset.GetKeyID(FBlackboard::KeySelf);
		if (SelfKey != FBlackboard::InvalidKey)
		{
			BlackboardComp.SetValue<UBlackboardKeyType_Object>(SelfKey, GetPawn());
		}

		OnUsingBlackBoard(&BlackboardComp, &BlackboardAsset);
		return true;
	}
	return false;
}

bool AAIController::UseBlackboard(UBlackboardData* BlackboardAsset, UBlackboardComponent*& BlackboardComponent)
{
	if (BlackboardAsset == nullptr)
	{
		UE_VLOG(this, LogBehaviorTree, Log, TEXT("UseBlackboard: trying to use NULL Blackboard asset. Ignoring"));
		return false;
	}

	bool bSuccess = true;
	Blackboard = FindComponentByClass<UBlackboardComponent>();

	if (Blackboard == nullptr)
	{
		Blackboard = NewObject<UBlackboardComponent>(this, TEXT("BlackboardComponent"));
		REDIRECT_OBJECT_TO_VLOG(Blackboard, this);
		if (Blackboard != nullptr)
		{
			bSuccess = InitializeBlackboard(*Blackboard, *BlackboardAsset);
			Blackboard->RegisterComponent();
		}
	}
	else if (Blackboard->GetBlackboardAsset() == nullptr)
	{
		bSuccess = InitializeBlackboard(*Blackboard, *BlackboardAsset);
	}
	else if (Blackboard->GetBlackboardAsset() != BlackboardAsset)
	{
		// @todo this behavior should be opt-out-able.
		UE_VLOG(this, LogBehaviorTree, Log, TEXT("UseBlackboard: requested blackboard %s while already has %s instantiated. Forcing new BB.")
			, *GetNameSafe(BlackboardAsset), *GetNameSafe(Blackboard->GetBlackboardAsset()));
		bSuccess = InitializeBlackboard(*Blackboard, *BlackboardAsset);
	}

	BlackboardComponent = Blackboard;

	return bSuccess;
}

bool AAIController::ShouldSyncBlackboardWith(const UBlackboardComponent& OtherBlackboardComponent) const 
{ 
	return Blackboard != nullptr
		&& Blackboard->GetBlackboardAsset() != nullptr
		&& OtherBlackboardComponent.GetBlackboardAsset() != nullptr
		&& Blackboard->GetBlackboardAsset()->IsRelatedTo(*OtherBlackboardComponent.GetBlackboardAsset());
}

bool AAIController::SuggestTossVelocity(FVector& OutTossVelocity, FVector Start, FVector End, float TossSpeed, bool bPreferHighArc, float CollisionRadius, bool bOnlyTraceUp)
{
	// pawn's physics volume gets 2nd priority
	APhysicsVolume const* const PhysicsVolume = GetPawn() ? GetPawn()->GetPhysicsVolume() : nullptr;
	float const GravityOverride = PhysicsVolume ? PhysicsVolume->GetGravityZ() : 0.f;
	ESuggestProjVelocityTraceOption::Type const TraceOption = bOnlyTraceUp ? ESuggestProjVelocityTraceOption::OnlyTraceWhileAscending : ESuggestProjVelocityTraceOption::TraceFullPath;

	UGameplayStatics::FSuggestProjectileVelocityParameters VelocityParams = UGameplayStatics::FSuggestProjectileVelocityParameters(this, Start, End, TossSpeed);
	VelocityParams.bFavorHighArc = bPreferHighArc;
	VelocityParams.CollisionRadius = CollisionRadius;
	VelocityParams.OverrideGravityZ = GravityOverride;
	VelocityParams.TraceOption = TraceOption;

	return UGameplayStatics::SuggestProjectileVelocity(VelocityParams, OutTossVelocity);
}

FString AAIController::GetDebugIcon() const
{
	if (BrainComponent == NULL || BrainComponent->IsRunning() == false)
	{
		return TEXT("/Engine/EngineResources/AICON-Red.AICON-Red");
	}

	return TEXT("/Engine/EngineResources/AICON-Green.AICON-Green");
}

void AAIController::OnGameplayTaskResourcesClaimed(FGameplayResourceSet NewlyClaimed, FGameplayResourceSet FreshlyReleased)
{
	if (BrainComponent)
	{
		const uint8 LogicID = UGameplayTaskResource::GetResourceID<UAIResource_Logic>();
		if (NewlyClaimed.HasID(LogicID))
		{
			BrainComponent->LockResource(EAIRequestPriority::Logic);
		}
		else if (FreshlyReleased.HasID(LogicID))
		{
			BrainComponent->ClearResourceLock(EAIRequestPriority::Logic);
		}
	}
}

void AAIController::SetPathFollowingComponent(UPathFollowingComponent* NewPFComponent)
{ 
	PathFollowingComponent = NewPFComponent; 
#if ENABLE_VISUAL_LOG
	if (NewPFComponent)
	{
		REDIRECT_OBJECT_TO_VLOG(NewPFComponent, this);
	}
#endif // ENABLE_VISUAL_LOG
}

//----------------------------------------------------------------------//
// IGenericTeamAgentInterface
//----------------------------------------------------------------------//
void AAIController::SetGenericTeamId(const FGenericTeamId& NewTeamID)
{
	if (TeamID != NewTeamID)
	{
		TeamID = NewTeamID;
		// @todo notify perception system that a controller changed team ID
	}
}

