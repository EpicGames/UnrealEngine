// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIHelpers.h"
#include "GameFramework/Actor.h"

namespace FAISystem
{
	FVector FindClosestLocation(const FVector& Origin, const TArray<FVector>& Locations)
	{
		FVector BestLocation = FAISystem::InvalidLocation;
		FVector::FReal BestDistanceSq = TNumericLimits<FVector::FReal>::Max();

		for (const FVector& Candidate : Locations)
		{
			const FVector::FReal CurrentDistanceSq = FVector::DistSquared(Origin, Candidate);
			if (CurrentDistanceSq < BestDistanceSq)
			{
				BestDistanceSq = CurrentDistanceSq;
				BestLocation = Candidate;
			}
		}

		return BestLocation;
	}

	//----------------------------------------------------------------------//
	// CheckIsTargetInSightCone
	//                     F
	//                   *****  
	//              *             *
	//          *                     *
	//       *                           *
	//     *                               *
	//   *                                   * 
	//    \                                 /
	//     \                               /
	//      \                             /
	//       \             X             /
	//        \                         /
	//         \          ***          /
	//          \     *    N    *     /
	//           \ *               * /
	//            N                 N
	//            
	//           
	//           
	//           
	//
	// 
	//                     B 
	//
	// X = StartLocation
	// B = Backward offset
	// N = Near Clipping Radius (from the StartLocation adjusted by Backward offset)
	// F = Far Clipping Radius (from the StartLocation adjusted by Backward offset)
	//----------------------------------------------------------------------//
	bool CheckIsTargetInSightCone(const FVector& StartLocation, const FVector& ConeDirectionNormal, float PeripheralVisionAngleCos,
		float ConeDirectionBackwardOffset, float NearClippingRadiusSq, float const FarClippingRadiusSq, const FVector& TargetLocation)
	{
		const FVector BaseLocation = FMath::IsNearlyZero(ConeDirectionBackwardOffset) ? StartLocation : StartLocation - ConeDirectionNormal * ConeDirectionBackwardOffset;
		const FVector ActorToTarget = TargetLocation - BaseLocation;
		const FVector::FReal DistToTargetSq = ActorToTarget.SizeSquared();
		if (DistToTargetSq <= FarClippingRadiusSq && DistToTargetSq >= NearClippingRadiusSq)
		{
			// Will return true if squared distance to Target is smaller than SMALL_NUMBER
			if (DistToTargetSq < SMALL_NUMBER)
			{
				return true;
			}
			
			// Calculate the normal here instead of calling GetUnsafeNormal as we already have the DistToTargetSq (optim)
			const FVector DirectionToTargetNormal = ActorToTarget * FMath::InvSqrt(DistToTargetSq);

			return FVector::DotProduct(DirectionToTargetNormal, ConeDirectionNormal) > PeripheralVisionAngleCos;
		}

		return false;
	}
}

namespace UE::AI
{

TOptional<float> GetYawFromVector(const FVector& Vector)
{
	const FVector2D Vector2D(Vector);
	const FVector::FReal SizeSquared = Vector2D.SizeSquared();
	if (SizeSquared <= SMALL_NUMBER)
	{
		return TOptional<float>();
	}

	const FVector2D ForwardVector(1., 0.);
	const FVector2D NormVector2D = Vector2D * FMath::InvSqrt(SizeSquared);
	const FVector::FReal DotVal = ForwardVector | NormVector2D;
	const FVector::FReal ClampedDot = FMath::Clamp(DotVal, -1., 1.);
	const FVector::FReal Direction = ForwardVector ^ NormVector2D;
	const FVector::FReal Sign = Direction < 0. ? -1. : 1.;
	const FVector::FReal Yaw = Sign * FMath::Acos(ClampedDot);
	return static_cast<float>(Yaw);
}

TOptional<float> GetYawFromRotator(const FRotator& Rotator)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GetYawFromVector(Rotator.Vector());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

TOptional<float> GetYawFromQuaternion(const FQuat& Quaternion)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GetYawFromVector(Quaternion.Vector());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void GetActorClassDefaultComponents(const TSubclassOf<AActor>& ActorClass, TArray<UActorComponent*>& OutComponents, const TSubclassOf<UActorComponent>& InComponentClass)
{
	OutComponents.Reset();
	AActor::ForEachComponentOfActorClassDefault(ActorClass, InComponentClass, [&OutComponents](const UActorComponent* TemplateComponent)
	{
		OutComponents.Add(const_cast<UActorComponent*>(TemplateComponent));
		return true;
	});
}
} // UE::AI