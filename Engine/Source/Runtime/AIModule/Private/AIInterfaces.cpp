// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "AIResourceInterface.h"
#include "GenericTeamAgentInterface.h"

UAIResourceInterface::UAIResourceInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

const FGenericTeamId FGenericTeamId::NoTeam(FGenericTeamId::NoTeamId);

FGenericTeamId FGenericTeamId::GetTeamIdentifier(const AActor* TeamMember)
{
	const IGenericTeamAgentInterface* TeamAgent = Cast<const IGenericTeamAgentInterface>(TeamMember);
	if (TeamAgent)
	{
		return TeamAgent->GetGenericTeamId();
	}

	return FGenericTeamId::NoTeam;
}

//----------------------------------------------------------------------//
// FGenericTeamId
//----------------------------------------------------------------------//
namespace
{
	ETeamAttitude::Type DefaultTeamAttitudeSolver(FGenericTeamId A, FGenericTeamId B)
	{
		return A != B ? ETeamAttitude::Hostile : ETeamAttitude::Friendly;
	}
}

FGenericTeamId::FAttitudeSolverFunction FGenericTeamId::AttitudeSolverImpl = &DefaultTeamAttitudeSolver;

ETeamAttitude::Type FGenericTeamId::GetAttitude(const AActor* A, const AActor* B)
{
	const IGenericTeamAgentInterface* TeamAgentA = Cast<const IGenericTeamAgentInterface>(A);

	return TeamAgentA == NULL || B == NULL ? ETeamAttitude::Neutral : TeamAgentA->GetTeamAttitudeTowards(*B);
}

void FGenericTeamId::SetAttitudeSolver(const FGenericTeamId::FAttitudeSolverFunction& Solver)
{
	AttitudeSolverImpl = Solver;
}

void FGenericTeamId::ResetAttitudeSolver()
{
	AttitudeSolverImpl = &DefaultTeamAttitudeSolver;
}

//----------------------------------------------------------------------//
// UGenericTeamAgentInterface
//----------------------------------------------------------------------//
UGenericTeamAgentInterface::UGenericTeamAgentInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
