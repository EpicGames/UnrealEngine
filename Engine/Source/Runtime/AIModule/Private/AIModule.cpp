// Copyright Epic Games, Inc. All Rights Reserved.
#include "AIModule.h"
#include "EngineDefines.h"
#include "Templates/SubclassOf.h"
#include "AISystem.h"
#include "VisualLogger/VisualLogger.h"
#include "GameFramework/WorldSettings.h"
#include "DefaultManagerInstanceTracker.h"

#if WITH_EDITOR
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#if ENABLE_VISUAL_LOG
	#include "VisualLoggerExtension.h"
#endif // ENABLE_VISUAL_LOG
#endif // WITH_EDITOR

#if WITH_GAMEPLAY_DEBUGGER_CORE
#include "GameplayDebugger.h"
#endif // WITH_GAMEPLAY_DEBUGGER_CORE

#if WITH_GAMEPLAY_DEBUGGER
#include "GameplayDebugger/GameplayDebuggerCategory_AI.h"
#include "GameplayDebugger/GameplayDebuggerCategory_BehaviorTree.h"
#include "GameplayDebugger/GameplayDebuggerCategory_EQS.h"
#include "GameplayDebugger/GameplayDebuggerCategory_Navmesh.h"
#include "GameplayDebugger/GameplayDebuggerCategory_Perception.h"
#include "GameplayDebugger/GameplayDebuggerCategory_PerceptionSystem.h"
#include "GameplayDebugger/GameplayDebuggerCategory_NavLocalGrid.h"
#endif // WITH_GAMEPLAY_DEBUGGER

#define LOCTEXT_NAMESPACE "AIModule"

DEFINE_LOG_CATEGORY_STATIC(LogAIModule, Log, All);
DEFINE_LOG_CATEGORY(LogManagerInstanceTracker);

class FAIModule : public IAIModule
{
	// Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	
	virtual UAISystemBase* CreateAISystemInstance(UWorld* World) override;
	// End IModuleInterface
#if WITH_EDITOR
	virtual EAssetTypeCategories::Type GetAIAssetCategoryBit() const override { return AIAssetCategoryBit; }
protected:
	EAssetTypeCategories::Type AIAssetCategoryBit;
#if ENABLE_VISUAL_LOG
	FVisualLoggerExtension	VisualLoggerExtension;
#endif // ENABLE_VISUAL_LOG
#endif
};

IMPLEMENT_MODULE(FAIModule, AIModule)

void FAIModule::StartupModule()
{ 
	FModuleManager::LoadModulePtr< IModuleInterface >("NavigationSystem");
#if WITH_EDITOR 
	FModuleManager::LoadModulePtr< IModuleInterface >("AITestSuite");

	if (GIsEditor)
	{
		// Register asset types
		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

		// register AI category so that AI assets can register to it
		AIAssetCategoryBit = AssetTools.RegisterAdvancedAssetCategory(FName("Artificial Intelligence"), LOCTEXT("AIAssetCategory", "Artificial Intelligence"));
	}
#endif // WITH_EDITOR 

#if WITH_EDITOR && ENABLE_VISUAL_LOG
	FVisualLogger::Get().RegisterExtension(*EVisLogTags::TAG_EQS, &VisualLoggerExtension);
#endif

#if WITH_GAMEPLAY_DEBUGGER
	IGameplayDebugger& GameplayDebuggerModule = IGameplayDebugger::Get();
	GameplayDebuggerModule.RegisterCategory("AI", IGameplayDebugger::FOnGetCategory::CreateStatic(&FGameplayDebuggerCategory_AI::MakeInstance), EGameplayDebuggerCategoryState::EnabledInGameAndSimulate, 1);
	GameplayDebuggerModule.RegisterCategory("BehaviorTree", IGameplayDebugger::FOnGetCategory::CreateStatic(&FGameplayDebuggerCategory_BehaviorTree::MakeInstance), EGameplayDebuggerCategoryState::EnabledInGame, 2);
	GameplayDebuggerModule.RegisterCategory("EQS", IGameplayDebugger::FOnGetCategory::CreateStatic(&FGameplayDebuggerCategory_EQS::MakeInstance));
	GameplayDebuggerModule.RegisterCategory("Navmesh", IGameplayDebugger::FOnGetCategory::CreateStatic(&FGameplayDebuggerCategory_Navmesh::MakeInstance), EGameplayDebuggerCategoryState::Disabled, 0);
	GameplayDebuggerModule.RegisterCategory("Perception", IGameplayDebugger::FOnGetCategory::CreateStatic(&FGameplayDebuggerCategory_Perception::MakeInstance));
	GameplayDebuggerModule.RegisterCategory("PerceptionSystem", IGameplayDebugger::FOnGetCategory::CreateStatic(&FGameplayDebuggerCategory_PerceptionSystem::MakeInstance));
	GameplayDebuggerModule.RegisterCategory("NavGrid", IGameplayDebugger::FOnGetCategory::CreateStatic(&FGameplayDebuggerCategory_NavLocalGrid::MakeInstance), EGameplayDebuggerCategoryState::Hidden);
	GameplayDebuggerModule.NotifyCategoriesChanged();
#endif // WITH_GAMEPLAY_DEBUGGER
}

void FAIModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
#if WITH_EDITOR && ENABLE_VISUAL_LOG
	FVisualLogger::Get().UnregisterExtension(*EVisLogTags::TAG_EQS, &VisualLoggerExtension);
#endif

#if WITH_GAMEPLAY_DEBUGGER
	if (IGameplayDebugger::IsAvailable())
	{
		IGameplayDebugger& GameplayDebuggerModule = IGameplayDebugger::Get();
		GameplayDebuggerModule.UnregisterCategory("AI");
		GameplayDebuggerModule.UnregisterCategory("BehaviorTree");
		GameplayDebuggerModule.UnregisterCategory("EQS");
		GameplayDebuggerModule.UnregisterCategory("Navmesh");
		GameplayDebuggerModule.UnregisterCategory("Perception");
		GameplayDebuggerModule.UnregisterCategory("NavGrid");
		GameplayDebuggerModule.NotifyCategoriesChanged();
	}
#endif // WITH_GAMEPLAY_DEBUGGER
}

UAISystemBase* FAIModule::CreateAISystemInstance(UWorld* World)
{
	UE_LOGF(LogAIModule, Verbose, "Creating AISystem for world %ls", *GetNameSafe(World));
	
	FSoftClassPath AISystemClassName = UAISystemBase::GetAISystemClassName();

	if (World)
	{
		AWorldSettings* WorldSettings = World->GetWorldSettings();
		if (WorldSettings)
		{
			AISystemClassName = WorldSettings->GetAISystemClassName();
		}
	}

	UAISystemBase* AISystemInstance = nullptr;
	if (AISystemClassName.IsValid())
	{
		TSubclassOf<UAISystemBase> AISystemClass = LoadClass<UAISystemBase>(nullptr, *AISystemClassName.ToString(), nullptr, LOAD_None, nullptr);

		AISystemInstance = AISystemClass ? NewObject<UAISystemBase>(World, AISystemClass) : nullptr;
		if (AISystemInstance == nullptr)
		{
			UE_LOGF(LogAIModule, Error, "Failed to create AISystem instance of class %ls!", *AISystemClassName.ToString());
		}
	}

	return AISystemInstance;
}

#undef LOCTEXT_NAMESPACE
