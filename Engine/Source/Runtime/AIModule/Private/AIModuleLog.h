// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

//----------------------------------------------------------------------//
// Engine headers
//----------------------------------------------------------------------//

#include "CoreMinimal.h"


//----------------------------------------------------------------------//
// AIModule headers
//----------------------------------------------------------------------//









DECLARE_LOG_CATEGORY_EXTERN(LogAINavigation, Warning, All);
DECLARE_LOG_CATEGORY_EXTERN(LogCrowdFollowing, Warning, All);
DECLARE_LOG_CATEGORY_EXTERN(LogAIModuleGameplayDebugger, Log, All);
