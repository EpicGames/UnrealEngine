// Copyright Epic Games, Inc. All Rights Reserved.
#include "AIResources.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AIResources)

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
FString UAIResource_Movement::GenerateDebugDescription() const
{
	return TEXT("Move");
}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

