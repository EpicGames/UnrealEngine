// Copyright Epic Games, Inc. All Rights Reserved.

#include "AISubsystem.h"
#include "AISystem.h"
#include "Misc/App.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AISubsystem)


DEFINE_LOG_CATEGORY_STATIC(LogAISub, Log, All);

UAISubsystem::UAISubsystem(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		AISystem = Cast<UAISystem>(GetOuter());

		// Some AISubsystems might have been created from a different Outer
		// so we fall back on the world to cache the parent AISystem
		if (AISystem == nullptr)
		{
			AISystem = UAISystem::GetCurrentSafe(GetWorld());
		}

		UE_CLOGF(AISystem == nullptr, LogAISub, Error, "%ls is an invalid outer for UAISubsystem instance %ls"
			, *GetName(), *GetNameSafe(GetOuter()));
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (AISystem == nullptr 
			&& GetOuter()
			&& (IsRunningCommandlet() 
				|| (GIsEditor && GetWorld() && GetWorld()->WorldType == EWorldType::Editor)))
		{
			// not calling MarkPackageDirty on this because it might be marked as transient 
			GetOuter()->MarkPackageDirty();
		}		
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	}
}

UWorld* UAISubsystem::GetWorld() const
{
	return GetWorldFast();
}

void UAISubsystem::BeginDestroy()
{
	Super::BeginDestroy();

	// Make sure the tick is completely disabled
	SetTickableTickType(ETickableTickType::Never);
}

ETickableTickType UAISubsystem::GetTickableTickType() const
{
	return (HasAnyFlags(RF_ClassDefaultObject) || AISystem == nullptr)
		? ETickableTickType::Never
		: ETickableTickType::Always;
}

TStatId UAISubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UAISubsystem, STATGROUP_Tickables);
}
