// Copyright Epic Games, Inc. All Rights Reserved.

#include "AISystem.h"
#include "Engine/GameInstance.h"
#include "Modules/ModuleManager.h"
#include "AIController.h"
#include "Perception/AIPerceptionSystem.h"
#include "BehaviorTree/BehaviorTreeManager.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "EnvironmentQuery/EnvQueryManager.h"
#include "GameFramework/PlayerController.h"
#include "HotSpots/AIHotSpotManager.h"
#include "BehaviorTree/BlackboardData.h"
#include "Navigation/NavLocalGridManager.h"
#include "Misc/CommandLine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AISystem)

DEFINE_STAT(STAT_AI_Overall);

FRandomStream UAISystem::RandomStream;

UAISystem::UAISystem(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// default values of AI config params
	AcceptanceRadius = 5.f;
	bFinishMoveOnGoalOverlap = true;
	bAcceptPartialPaths = true;
	bAllowStrafing = false;
	DefaultSightCollisionChannel = ECC_Visibility;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// game-wise config
		if (FParse::Param(FCommandLine::Get(), TEXT("FixedSeed")) == false)
		{
			// by default FRandomStream is initialized with 0
			// only if not configured with commandline we should 
			// initialize the random stream.
			const int32 Seed = ((int32)(FDateTime::Now().GetTicks() % (int64)MAX_int32));
			RandomStream.Initialize(Seed);
		}
	}
}

void UAISystem::BeginDestroy()
{
	CleanupWorld(true, true);
	Super::BeginDestroy();
}

void UAISystem::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		UWorld* WorldOuter = GetOuterWorld();

		BehaviorTreeManager = NewObject<UBehaviorTreeManager>(this);
		ensure(BehaviorTreeManager != nullptr);
		NavLocalGrids = NewObject<UNavLocalGridManager>(this);
		ensure(NavLocalGrids != nullptr);

		TSubclassOf<UAIHotSpotManager> HotSpotManagerClass = HotSpotManagerClassName.IsValid() ? LoadClass<UAIHotSpotManager>(NULL, *HotSpotManagerClassName.ToString(), NULL, LOAD_None, NULL) : nullptr;
		if (HotSpotManagerClass)
		{
			HotSpotManager = NewObject<UAIHotSpotManager>(this, HotSpotManagerClass, TEXT("HotSpotManager"));
		}

		TSubclassOf<UAIPerceptionSystem> PerceptionSystemClass = PerceptionSystemClassName.IsValid() ? LoadClass<UAIPerceptionSystem>(NULL, *PerceptionSystemClassName.ToString(), NULL, LOAD_None, NULL) : nullptr;
		if (PerceptionSystemClass)
		{
			PerceptionSystem = NewObject<UAIPerceptionSystem>(this, PerceptionSystemClass, TEXT("PerceptionSystem"));
		}

		TSubclassOf<UEnvQueryManager> EnvQueryManagerClass = EnvQueryManagerClassName.IsValid() ? LoadClass<UEnvQueryManager>(NULL, *EnvQueryManagerClassName.ToString(), NULL, LOAD_None, NULL) : UEnvQueryManager::StaticClass();
		if (EnvQueryManagerClass)
		{
			EnvironmentQueryManager = NewObject<UEnvQueryManager>(this, EnvQueryManagerClass, TEXT("EnvironmentQueryManager"));
		}
		ensure(EnvironmentQueryManager != nullptr);

		if (WorldOuter)
		{
			const FOnActorSpawned::FDelegate ActorSpawnedDelegate = FOnActorSpawned::FDelegate::CreateUObject(this, &UAISystem::OnActorSpawned);
			ActorSpawnedDelegateHandle = WorldOuter->AddOnActorSpawnedHandler(ActorSpawnedDelegate);
		}

		PawnBeginPlayDelegateHandle = APawn::OnPawnBeginPlay.AddUObject(this, &UAISystem::OnPawnBeginPlay);

		ConditionalLoadDebuggerPlugin();
	}
}

void UAISystem::StartPlay()
{
	Super::StartPlay();

	if (PerceptionSystem)
	{
		PerceptionSystem->StartPlay();
	}
}

void UAISystem::OnActorSpawned(AActor* SpawnedActor)
{
}

void UAISystem::OnPawnBeginPlay(APawn* Pawn)
{
	check(Pawn);

	if (PerceptionSystem == nullptr || PerceptionSystem->bHandlePawnNotification == false)
	{
		return;
	}

	const UWorld* const PawnWorld = Pawn->GetWorld();
	check(PawnWorld);

	if (PawnWorld == GetWorld())
	{
		PerceptionSystem->OnNewPawn(*Pawn);
	}
}

void UAISystem::InitializeActorsForPlay(bool bTimeGotReset)
{

}

void UAISystem::WorldOriginLocationChanged(FIntVector OldOriginLocation, FIntVector NewOriginLocation)
{

}

void UAISystem::CleanupWorld(bool bSessionEnded, bool bCleanupResources, UWorld* NewWorld)
{
	CleanupWorld(bSessionEnded, bCleanupResources);
}

void UAISystem::CleanupWorld(bool bSessionEnded, bool bCleanupResources)
{
	Super::CleanupWorld(bSessionEnded, bCleanupResources);
	
	if (bCleanupResources)
	{
		if (EnvironmentQueryManager)
		{
			EnvironmentQueryManager->OnWorldCleanup();
			EnvironmentQueryManager = nullptr;
		}
	}

	const UWorld* const WorldOuter = GetOuterWorld();
	if (WorldOuter)
	{
		WorldOuter->RemoveOnActorSpawnedHandler(ActorSpawnedDelegateHandle);
	}
	APawn::OnPawnBeginPlay.Remove(PawnBeginPlayDelegateHandle);
}

void UAISystem::AIIgnorePlayers()
{
	AAIController::ToggleAIIgnorePlayers();
}

void UAISystem::AILoggingVerbose()
{
	UWorld* OuterWorld = GetOuterWorld();
	if (OuterWorld && OuterWorld->GetGameInstance())
	{
		APlayerController* PC = OuterWorld->GetGameInstance()->GetFirstLocalPlayerController();
		if (PC)
		{
			PC->ConsoleCommand(TEXT("log lognavigation verbose | log logpathfollowing verbose | log LogCharacter verbose | log LogBehaviorTree verbose"));
		}
	}
}

void UAISystem::RunEQS(const FString& QueryName, UObject* Target)
{
#if !UE_BUILD_SHIPPING
	UWorld* OuterWorld = GetOuterWorld();
	if (OuterWorld == NULL || OuterWorld->GetGameInstance() == NULL)
	{
		return;
	}

	APlayerController* MyPC = OuterWorld->GetGameInstance()->GetFirstLocalPlayerController();
	UEnvQueryManager* EQS = GetEnvironmentQueryManager();

	if (Target && MyPC && EQS)
	{
		const UEnvQuery* QueryTemplate = EQS->FindQueryTemplate(QueryName);

		if (QueryTemplate)
		{
			EQS->RunInstantQuery(FEnvQueryRequest(QueryTemplate, Target), EEnvQueryRunMode::AllMatching);
		}
		else
		{
			MyPC->ClientMessage(FString::Printf(TEXT("Unable to fing query template \'%s\'"), *QueryName));
		}
	}
	else
	{
		MyPC->ClientMessage(TEXT("No debugging target"));
	}
#endif // !UE_BUILD_SHIPPING
}

UAISystem::FBlackboardDataToComponentsIterator::FBlackboardDataToComponentsIterator(FBlackboardDataToComponentsMap& InBlackboardDataToComponentsMap, UBlackboardData* BlackboardAsset)
	: CurrentIteratorIndex(0)
	, Iterators()
{
	// Reserve space for the weak pointers so that we don't invalidate references as we insert
	int32 NumBlackboardAssets = 0;
	for (UBlackboardData* BlackboardAssetIt = BlackboardAsset; BlackboardAssetIt; BlackboardAssetIt = BlackboardAssetIt->Parent)
	{
		++NumBlackboardAssets;
	}
	IteratorKeysForReference.Reserve(NumBlackboardAssets);

	while (BlackboardAsset)
	{
		// In 32-bit, map key iterators hold TWeakObjectPtrs by reference, not value,
		// so we need to retain a bunch of weakobjptrs in an array so there is something to reference.
		TWeakObjectPtr<UBlackboardData>& WeakBlackboardAssetRef = IteratorKeysForReference.Add_GetRef(BlackboardAsset);
		Iterators.Add(InBlackboardDataToComponentsMap.CreateConstKeyIterator(WeakBlackboardAssetRef));
		BlackboardAsset = BlackboardAsset->Parent;
	}
	TryMoveIteratorToParentBlackboard();
}

void UAISystem::RegisterBlackboardComponent(UBlackboardData& BlackboardData, UBlackboardComponent& BlackboardComp)
{
	// mismatch of register/unregister.
	ensure(BlackboardDataToComponentsMap.FindPair(&BlackboardData, &BlackboardComp) == nullptr);

	BlackboardDataToComponentsMap.Add(&BlackboardData, &BlackboardComp);
	if (BlackboardData.Parent)
	{
		RegisterBlackboardComponent(*BlackboardData.Parent, BlackboardComp);
	}
}

void UAISystem::UnregisterBlackboardComponent(UBlackboardData& BlackboardData, UBlackboardComponent& BlackboardComp)
{
	// this is actually possible, we can end up unregistering before UBlackboardComponent cached its BrainComponent
	// which currently is tied to the whole process. 
	// @todo remove this dependency
	ensure(BlackboardDataToComponentsMap.FindPair(&BlackboardData, &BlackboardComp) != nullptr);

	if (BlackboardData.Parent)
	{
		UnregisterBlackboardComponent(*BlackboardData.Parent, BlackboardComp);
	}
	BlackboardDataToComponentsMap.RemoveSingle(&BlackboardData, &BlackboardComp);

	// mismatch of Register/Unregister.
	check(BlackboardDataToComponentsMap.FindPair(&BlackboardData, &BlackboardComp) == nullptr);
}

UAISystem::FBlackboardDataToComponentsIterator UAISystem::CreateBlackboardDataToComponentsIterator(UBlackboardData& BlackboardAsset)
{
	return UAISystem::FBlackboardDataToComponentsIterator(BlackboardDataToComponentsMap, &BlackboardAsset);
}

void UAISystem::ConditionalLoadDebuggerPlugin()
{
#if defined(ENABLED_GAMEPLAY_DEBUGGER) && ENABLED_GAMEPLAY_DEBUGGER
	if (bEnableDebuggerPlugin)
	{
		LoadDebuggerPlugin();
	}
#endif
}

void UAISystem::LoadDebuggerPlugin()
{
	FModuleManager::LoadModulePtr< IModuleInterface >("GameplayDebugger");
}

