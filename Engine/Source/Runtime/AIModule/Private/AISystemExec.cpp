// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/CoreMisc.h"
#include "AISystem.h"
#if !UE_BUILD_SHIPPING
#include "BehaviorTree/BehaviorTreeManager.h"

struct FAISystemExec : public FSelfRegisteringExec
{
	FAISystemExec()
	{
	}

protected:
	// Begin FExec Interface
	virtual bool Exec_Dev(UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	// End FExec Interface
};

FAISystemExec AISystemExecInstance;

bool FAISystemExec::Exec_Dev(UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (Inworld == NULL)
	{
		return false;
	}

	UAISystem* AISys = UAISystem::GetCurrent(*Inworld);	
	bool bHandled = false;

	if (AISys != NULL)
	{
		if (FParse::Command(&Cmd, TEXT("AIIgnorePlayers")))
		{
			AISys->AIIgnorePlayers();
			bHandled = true;
		}
		else if (FParse::Command(&Cmd, TEXT("AILoggingVerbose")))
		{
			AISys->AILoggingVerbose();
			bHandled = true;
		}
		else if (FParse::Command(&Cmd, TEXT("DumpBTUsageStats")))
		{
			if (AISys->GetBehaviorTreeManager())
			{
				AISys->GetBehaviorTreeManager()->DumpUsageStats();
				bHandled = true;
			}
		}
	}

	return bHandled;
}

#endif // !UE_BUILD_SHIPPING
