// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITypes.h"
#include "GameFramework/Actor.h"
#include "UObject/Package.h"
#include "Navigation/PathFollowingComponent.h"
#include "NavFilters/NavigationQueryFilter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AITypes)

const FRotator FAISystem::InvalidRotation = FRotator(FLT_MAX);
const FQuat FAISystem::InvalidOrientation = FQuat(FLT_MAX, FLT_MAX, FLT_MAX, FLT_MAX);

//----------------------------------------------------------------------//
// FAIResourceLock
//----------------------------------------------------------------------//
FAIResourceLock::FAIResourceLock()
{
	Locks = 0;
	bUseResourceLockCount = false;
}

void FAIResourceLock::SetLock(EAIRequestPriority::Type LockPriority)
{
	if (bUseResourceLockCount)
	{
		const uint8 ResourceLockCountIndex = static_cast<uint8>(LockPriority);
		ResourceLockCount[ResourceLockCountIndex] += 1;
	}

	Locks |= (1 << LockPriority);
}

void FAIResourceLock::ClearLock(EAIRequestPriority::Type LockPriority)
{
	if (bUseResourceLockCount)
	{
		const uint8 ResourceLockCountIndex = static_cast<uint8>(LockPriority);

		ensure(ResourceLockCount[ResourceLockCountIndex] > 0);
		ResourceLockCount[ResourceLockCountIndex] -= 1;

		if (ResourceLockCount[ResourceLockCountIndex] == 0)
		{
			Locks &= ~(1 << LockPriority);
		}
	}
	else
	{
		Locks &= ~(1 << LockPriority);
	}
}

void FAIResourceLock::SetUseResourceLockCount(bool inUseResourceLockCount)
{
	bUseResourceLockCount = inUseResourceLockCount;
	ForceClearAllLocks();

}

void FAIResourceLock::ForceClearAllLocks()
{
	FMemory::Memzero(Locks);

	ResourceLockCount.Reset();
	if (bUseResourceLockCount)
	{
		ResourceLockCount.AddZeroed(sizeof(FAIResourceLock::FLockFlags) * 8);
	}
}

FString FAIResourceLock::GetLockPriorityName() const
{
	const static UEnum* SourceEnum = StaticEnum<EAIRequestPriority::Type>();

	FString LockNames;

	for (int32 LockLevel = 0; LockLevel < int32(EAIRequestPriority::MAX); ++LockLevel)
	{
		if (IsLocked() && IsLockedBy(EAIRequestPriority::Type(LockLevel)))
		{
			LockNames += FString::Printf(TEXT("%s, "), *SourceEnum->GetNameStringByValue(LockLevel));
		}
	}

	return LockNames;
}

//----------------------------------------------------------------------//
// FAIResources
//----------------------------------------------------------------------//
template<>
FAIResCounter FAINamedID<FAIResCounter>::Counter = FAIResCounter();

namespace FAIResources
{
	const FAIResourceID InvalidResource;
	const FAIResourceID Movement = FAIResourceID(TEXT("Movement"));
	const FAIResourceID Logic = FAIResourceID(TEXT("Logic"));
	const FAIResourceID Perception = FAIResourceID(TEXT("Perception"));

	TArray<FAIResourceID> ResourceIDs;

	void RegisterResource(const FAIResourceID& Resource)
	{
		if (FAIResourceID::GetSize() > static_cast<uint32>(FAIResources::ResourceIDs.Num()))
		{
			ResourceIDs.AddZeroed(FAIResourceID::GetSize() - FAIResources::ResourceIDs.Num());
		}
		ResourceIDs[Resource.Index] = Resource;
	}

	const FAIResourceID& GetResource(int32 ResourceIndex)
	{
		return ResourceIDs.IsValidIndex(ResourceIndex) ? ResourceIDs[ResourceIndex] : InvalidResource;
	}

	int32 GetResourcesCount()
	{ 
		return ResourceIDs.Num(); 
	}

	FString GetSetDescription(FAIResourcesSet ResourceSet)
	{
		if (ResourceSet.IsEmpty() == false)
		{
			FString Description;

			for (uint8 Index = 0; Index < uint8(ResourceIDs.Num()); ++Index)
			{
				if (ResourceSet.ContainsResourceIndex(Index))
				{
					Description += ResourceIDs[Index].Name.ToString();
					Description += TEXT(", ");
				}
			}

			return Description;
		}

		return TEXT("(empty)");
	}
}

static struct FAIResourceSetup
{
	FAIResourceSetup()
	{
		FAIResources::RegisterResource(FAIResources::Movement);
		FAIResources::RegisterResource(FAIResources::Logic);
		FAIResources::RegisterResource(FAIResources::Perception);
	}
} ResourceSetup;

//----------------------------------------------------------------------//
// FAIRequestID
//----------------------------------------------------------------------//
const FAIRequestID FAIRequestID::InvalidRequest(FAIRequestID::InvalidRequestID);
const FAIRequestID FAIRequestID::AnyRequest(FAIRequestID::AnyRequestID);
// same as AnyRequest on purpose. Just a readability thing
const FAIRequestID FAIRequestID::CurrentRequest(FAIRequestID::AnyRequestID);

//----------------------------------------------------------------------//
// FAIMoveRequest
//----------------------------------------------------------------------//

FAIMoveRequest::FAIMoveRequest() : 
	GoalActor(nullptr), GoalLocation(FAISystem::InvalidLocation), FilterClass(nullptr),
	bInitialized(false), bMoveToActor(false),
	bUsePathfinding(true), bAllowPartialPath(true), bRequireNavigableEndLocation(true), bApplyCostLimitFromHeuristic(false), bProjectGoalOnNavigation(true),
	bStartFromPreviousPath(false), bReachTestIncludesAgentRadius(true), bReachTestIncludesGoalRadius(true), bCanStrafe(false),
	AcceptanceRadius(UPathFollowingComponent::DefaultAcceptanceRadius), CostLimitFactor(FLT_MAX), MinimumCostLimit(0.f),
	UserFlags(0)
{
}

FAIMoveRequest::FAIMoveRequest(const AActor* InGoalActor) :
	GoalActor(const_cast<AActor*>(InGoalActor)), GoalLocation(FAISystem::InvalidLocation), FilterClass(nullptr),
	bInitialized(true), bMoveToActor(true),
	bUsePathfinding(true), bAllowPartialPath(true), bRequireNavigableEndLocation(true), bApplyCostLimitFromHeuristic(false), bProjectGoalOnNavigation(true),
	bStartFromPreviousPath(false), bReachTestIncludesAgentRadius(true), bReachTestIncludesGoalRadius(true), bCanStrafe(false),
	AcceptanceRadius(UPathFollowingComponent::DefaultAcceptanceRadius), CostLimitFactor(FLT_MAX), MinimumCostLimit(0.f),
	UserFlags(0)
{
}

FAIMoveRequest::FAIMoveRequest(const FVector& InGoalLocation) :
	GoalActor(nullptr), GoalLocation(InGoalLocation), FilterClass(nullptr),
	bInitialized(true), bMoveToActor(false),
	bUsePathfinding(true), bAllowPartialPath(true), bRequireNavigableEndLocation(true), bApplyCostLimitFromHeuristic(false), bProjectGoalOnNavigation(true),
	bStartFromPreviousPath(false), bReachTestIncludesAgentRadius(true), bReachTestIncludesGoalRadius(true), bCanStrafe(false),
	AcceptanceRadius(UPathFollowingComponent::DefaultAcceptanceRadius), CostLimitFactor(FLT_MAX), MinimumCostLimit(0.f),
	UserFlags(0)
{
}

void FAIMoveRequest::SetGoalActor(const AActor* InGoalActor)
{
	if (!bInitialized)
	{
		GoalActor = (AActor*)InGoalActor;
		bMoveToActor = true;
		bInitialized = true;
	}
}

void FAIMoveRequest::SetGoalLocation(const FVector& InGoalLocation)
{
	if (!bInitialized)
	{
		GoalLocation = InGoalLocation;
		bInitialized = true;
	}
}

FVector FAIMoveRequest::GetDestination() const
{
	return bMoveToActor ? (GoalActor.IsValid() ? GoalActor->GetActorLocation() : FAISystem::InvalidLocation) : GoalLocation;
}

bool FAIMoveRequest::UpdateGoalLocation(const FVector& NewLocation) const
{
	if (!bMoveToActor)
	{
		GoalLocation = NewLocation;
		return true;
	}

	return false;
}

FString FAIMoveRequest::ToString() const
{
	return FString::Printf(TEXT("%s(%s) Mode(%s) Filter(%s) AcceptanceRadius(%.1f%s) StartsFromPreviousPath(%s))"),
		bMoveToActor ? TEXT("Actor") : TEXT("Location"), bMoveToActor ? *GetNameSafe(GoalActor.Get()) : *GoalLocation.ToString(),
		bUsePathfinding ? (bAllowPartialPath ? TEXT("partial path") : TEXT("complete path")) : TEXT("direct"),
		*GetNameSafe(FilterClass),
		AcceptanceRadius, bReachTestIncludesAgentRadius || (bMoveToActor && bReachTestIncludesGoalRadius) ? TEXT(" + overlap") : TEXT(""),
		bStartFromPreviousPath ? TEXT("Yes") : TEXT("No")
		);
}

