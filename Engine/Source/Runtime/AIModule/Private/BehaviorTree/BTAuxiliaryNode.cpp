// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/BTAuxiliaryNode.h"
#include "BehaviorTree/BTCompositeNode.h"
#include "VisualLogger/VisualLogger.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTAuxiliaryNode)

UBTAuxiliaryNode::UBTAuxiliaryNode(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	bNotifyBecomeRelevant = false;
	bNotifyCeaseRelevant = false;
	bNotifyTick = false;
	bTickIntervals = false;
	ChildIndex = BTSpecialChild::OwnedByComposite;
}

void UBTAuxiliaryNode::WrappedOnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	if (bNotifyBecomeRelevant)
	{
		const UBTNode* NodeOb = HasInstance() ? GetNodeInstance(OwnerComp, NodeMemory) : this;
		if (NodeOb)
		{
			UE_VLOG(&OwnerComp, LogBehaviorTree, Verbose, TEXT("OnBecomeRelevant: %s"), *UBehaviorTreeTypes::DescribeNodeHelper(NodeOb));
			((UBTAuxiliaryNode*)NodeOb)->OnBecomeRelevant(OwnerComp, NodeMemory);
		}
	}
}

void UBTAuxiliaryNode::WrappedOnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	if (bNotifyCeaseRelevant)
	{
		const UBTNode* NodeOb = HasInstance() ? GetNodeInstance(OwnerComp, NodeMemory) : this;
		if (NodeOb)
		{
			UE_VLOG(&OwnerComp, LogBehaviorTree, Verbose, TEXT("OnCeaseRelevant: %s"), *UBehaviorTreeTypes::DescribeNodeHelper(NodeOb));
			((UBTAuxiliaryNode*)NodeOb)->OnCeaseRelevant(OwnerComp, NodeMemory);
		}
	}
}

bool UBTAuxiliaryNode::WrappedTickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds, float& NextNeededDeltaTime) const
{
	if (bNotifyTick || HasInstance())
	{
		const UBTAuxiliaryNode* NodeOb = HasInstance() ? static_cast<UBTAuxiliaryNode*>(GetNodeInstance(OwnerComp, NodeMemory)) : this;
		
		ensure(NodeOb);

		if (NodeOb != nullptr && NodeOb->bNotifyTick)
		{
			float UseDeltaTime = DeltaSeconds;
 
			if (NodeOb->bTickIntervals)
			{
				FBTAuxiliaryMemory* AuxMemory = GetSpecialNodeMemory<FBTAuxiliaryMemory>(NodeMemory);
				AuxMemory->NextTickRemainingTime -= DeltaSeconds;
				AuxMemory->AccumulatedDeltaTime += DeltaSeconds;

				const bool bTick = AuxMemory->NextTickRemainingTime <= 0.0f;
				if (bTick)
				{
				    UseDeltaTime = AuxMemory->AccumulatedDeltaTime;
					AuxMemory->AccumulatedDeltaTime = 0.0f;
					// Clamp next tick remaining time to 0 (i.e. tick on next frame) but node will usually set a new tick time in the call to TickNode
					AuxMemory->NextTickRemainingTime = 0.0f;

					UE_VLOG(OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT("Ticking aux node: %s"), *UBehaviorTreeTypes::DescribeNodeHelper(this));
				    const_cast<UBTAuxiliaryNode*>(NodeOb)->TickNode(OwnerComp, NodeMemory, UseDeltaTime);
				}

				if (AuxMemory->NextTickRemainingTime < NextNeededDeltaTime)
				{
					NextNeededDeltaTime = AuxMemory->NextTickRemainingTime;
				}
				return bTick;
			}
			else
			{
				UE_VLOG(OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT("Ticking aux node: %s"), *UBehaviorTreeTypes::DescribeNodeHelper(this));

				const_cast<UBTAuxiliaryNode*>(NodeOb)->TickNode(OwnerComp, NodeMemory, UseDeltaTime);
				NextNeededDeltaTime = 0.0f;
				return true;
			}

		}
	}
	return false;
}

void UBTAuxiliaryNode::SetNextTickTime(uint8* NodeMemory, float RemainingTime) const
{
	if (bTickIntervals)
	{
		FBTAuxiliaryMemory* AuxMemory = GetSpecialNodeMemory<FBTAuxiliaryMemory>(NodeMemory);
		AuxMemory->NextTickRemainingTime = RemainingTime;
	}
}

float UBTAuxiliaryNode::GetNextTickRemainingTime(uint8* NodeMemory) const
{
	if (bTickIntervals)
	{
		FBTAuxiliaryMemory* AuxMemory = GetSpecialNodeMemory<FBTAuxiliaryMemory>(NodeMemory);
		return FMath::Max(0.0f, AuxMemory->NextTickRemainingTime);
	}

	return 0.0f;
}

void UBTAuxiliaryNode::DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const
{
	Super::DescribeRuntimeValues(OwnerComp, NodeMemory, Verbosity, Values);

	if (Verbosity == EBTDescriptionVerbosity::Detailed && bTickIntervals)
	{
		FBTAuxiliaryMemory* AuxMemory = GetSpecialNodeMemory<FBTAuxiliaryMemory>(NodeMemory);
		Values.Add(FString::Printf(TEXT("next tick: %ss"), *FString::SanitizeFloat(AuxMemory->NextTickRemainingTime)));
	}
}

void UBTAuxiliaryNode::OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	// empty in base class
}

void UBTAuxiliaryNode::OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	// empty in base class
}

void UBTAuxiliaryNode::TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds)
{
	// empty in base class
}

uint16 UBTAuxiliaryNode::GetSpecialMemorySize() const
{
	return bTickIntervals ? sizeof(FBTAuxiliaryMemory) : Super::GetSpecialMemorySize();
}

void UBTAuxiliaryNode::InitializeParentLink(uint8 MyChildIndex)
{
	ChildIndex = MyChildIndex;
}

const UBTNode* UBTAuxiliaryNode::GetMyNode() const
{
	return (ChildIndex == BTSpecialChild::OwnedByComposite) ? GetParentNode() : (GetParentNode() ? GetParentNode()->GetChildNode(ChildIndex) : nullptr);
}

float UBTAuxiliaryNode::GetNextNeededDeltaTime(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	if (bNotifyTick || HasInstance())
	{
		const UBTAuxiliaryNode* NodeOb = HasInstance() ? static_cast<UBTAuxiliaryNode*>(GetNodeInstance(OwnerComp, NodeMemory)) : this;

		if (NodeOb != nullptr && NodeOb->bNotifyTick)
		{
			if (bTickIntervals)
			{
				FBTAuxiliaryMemory* AuxMemory = GetSpecialNodeMemory<FBTAuxiliaryMemory>(NodeMemory);
				return AuxMemory->NextTickRemainingTime;
			}
			else
			{
				return 0.0f;
			}
		}
	}
	return FLT_MAX;
}

//----------------------------------------------------------------------//
// DEPRECATED
//----------------------------------------------------------------------//

