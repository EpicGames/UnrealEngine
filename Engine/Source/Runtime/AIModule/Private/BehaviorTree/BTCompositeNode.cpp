// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/BTCompositeNode.h"
#include "GameFramework/Actor.h"
#include "VisualLogger/VisualLogger.h"
#include "BehaviorTree/BTTaskNode.h"
#include "BehaviorTree/BTDecorator.h"
#include "BehaviorTree/BTService.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTCompositeNode)

UBTCompositeNode::UBTCompositeNode(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "UnknownComposite";
	bApplyDecoratorScope = false;
	bUseChildExecutionNotify = false;
	bUseNodeActivationNotify = false;
	bUseNodeDeactivationNotify = false;
	bUseDecoratorsActivationCheck = false;
	bUseDecoratorsDeactivationCheck = false;
	bUseDecoratorsFailedActivationCheck = false;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UBTCompositeNode::~UBTCompositeNode()
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UBTCompositeNode::InitializeComposite(uint16 InLastExecutionIndex)
{
	LastExecutionIndex = InLastExecutionIndex;
}

int32 UBTCompositeNode::FindChildToExecute(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type& LastResult) const
{
	FBTCompositeMemory* NodeMemory = GetNodeMemory<FBTCompositeMemory>(SearchData);
	int32 RetIdx = BTSpecialChild::ReturnToParent;

	if (Children.Num())
	{
		int32 ChildIdx = GetNextChild(SearchData, NodeMemory->CurrentChild, LastResult);
		while (Children.IsValidIndex(ChildIdx) && !SearchData.bPostponeSearch)
		{
			// check decorators
			if (DoDecoratorsAllowExecution(SearchData.OwnerComp, SearchData.OwnerComp.ActiveInstanceIdx, ChildIdx))
			{
				OnChildActivation(SearchData, ChildIdx);
				RetIdx = ChildIdx;
				break;
			}
			else
			{
				LastResult = EBTNodeResult::Failed;

				const bool bCanNotify = !bUseDecoratorsFailedActivationCheck || CanNotifyDecoratorsOnFailedActivation(SearchData, ChildIdx, LastResult);
				if (bCanNotify)
				{
					NotifyDecoratorsOnFailedActivation(SearchData, ChildIdx, LastResult);
				}
			}

			ChildIdx = GetNextChild(SearchData, ChildIdx, LastResult);
		}
	}

	return RetIdx;
}

int32 UBTCompositeNode::GetChildIndex(FBehaviorTreeSearchData& SearchData, const UBTNode& ChildNode) const
{
	if (ChildNode.GetParentNode() != this)
	{
		FBTCompositeMemory* NodeMemory = GetNodeMemory<FBTCompositeMemory>(SearchData);
		return NodeMemory->CurrentChild;		
	}

	return GetChildIndex(ChildNode);
}

int32 UBTCompositeNode::GetChildIndex(const UBTNode& ChildNode) const
{
	for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ChildIndex++)
	{
		if (Children[ChildIndex].ChildComposite == &ChildNode ||
			Children[ChildIndex].ChildTask == &ChildNode)
		{
			return ChildIndex;
		}
	}

	return BTSpecialChild::ReturnToParent;
}

void UBTCompositeNode::OnChildActivation(FBehaviorTreeSearchData& SearchData, const UBTNode& ChildNode) const
{
	OnChildActivation(SearchData, GetChildIndex(SearchData, ChildNode));
}

void UBTCompositeNode::OnChildActivation(FBehaviorTreeSearchData& SearchData, int32 ChildIndex) const
{
	const FBTCompositeChild& ChildInfo = Children[ChildIndex];
	FBTCompositeMemory* NodeMemory = GetNodeMemory<FBTCompositeMemory>(SearchData);

	// pass to decorators before changing current child in node memory
	// so they can access previously executed one if needed
	const bool bCanNotify = !bUseDecoratorsActivationCheck || CanNotifyDecoratorsOnActivation(SearchData, ChildIndex);
	if (bCanNotify)
	{
		NotifyDecoratorsOnActivation(SearchData, ChildIndex);
	}

	// don't activate task services here, it's applied BEFORE aborting (e.g. abort lower pri decorator)
	// use UBehaviorTreeComponent::ExecuteTask instead

	// pass to child composite
	if (ChildInfo.ChildComposite)
	{
		ChildInfo.ChildComposite->OnNodeActivation(SearchData);
	}

	// update active node in current context: child node
	NodeMemory->CurrentChild = IntCastChecked<int8>(ChildIndex);
}

void UBTCompositeNode::OnChildDeactivation(FBehaviorTreeSearchData& SearchData, const UBTNode& ChildNode, EBTNodeResult::Type& NodeResult, const bool bRequestedFromValidInstance) const
{
	OnChildDeactivation(SearchData, GetChildIndex(SearchData, ChildNode), NodeResult, bRequestedFromValidInstance);
}

void UBTCompositeNode::OnChildDeactivation(FBehaviorTreeSearchData& SearchData, int32 ChildIndex, EBTNodeResult::Type& NodeResult, const bool bRequestedFromValidInstance) const
{
	const FBTCompositeChild& ChildInfo = Children[ChildIndex];

	// pass to task services
	if (ChildInfo.ChildTask)
	{
		for (int32 ServiceIndex = 0; ServiceIndex < ChildInfo.ChildTask->Services.Num(); ServiceIndex++)
		{
			SearchData.AddUniqueUpdate(FBehaviorTreeSearchUpdate(ChildInfo.ChildTask->Services[ServiceIndex], SearchData.OwnerComp.GetActiveInstanceIdx(), EBTNodeUpdateMode::Remove));
		}
	}
	// pass to child composite
	else if (ChildInfo.ChildComposite)
	{
		ChildInfo.ChildComposite->OnNodeDeactivation(SearchData, NodeResult);
	}

	// pass to decorators after composite is updated (so far only simple parallel uses it)
	// to have them working on correct result + they must be able to modify it if requested (e.g. force success)
	const bool bCanNotify = !bUseDecoratorsDeactivationCheck || CanNotifyDecoratorsOnDeactivation(SearchData, ChildIndex, NodeResult);
	if (bCanNotify)
	{
		NotifyDecoratorsOnDeactivation(SearchData, ChildIndex, NodeResult, bRequestedFromValidInstance);
	}
}

void UBTCompositeNode::OnNodeActivation(FBehaviorTreeSearchData& SearchData) const
{
	OnNodeRestart(SearchData);

	if (bUseNodeActivationNotify)
	{
		NotifyNodeActivation(SearchData);
	}

	for (int32 ServiceIndex = 0; ServiceIndex < Services.Num(); ServiceIndex++)
	{
		// add services when execution flow enters this composite
		SearchData.AddUniqueUpdate(FBehaviorTreeSearchUpdate(Services[ServiceIndex], SearchData.OwnerComp.GetActiveInstanceIdx(), EBTNodeUpdateMode::Add));

		// give services chance to perform initial tick before searching further
		Services[ServiceIndex]->NotifyParentActivation(SearchData);
	}
}

void UBTCompositeNode::OnNodeDeactivation(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type& NodeResult) const
{
	if (bUseNodeDeactivationNotify)
	{
		NotifyNodeDeactivation(SearchData, NodeResult);
	}

	// remove all services if execution flow leaves this composite
	for (int32 ServiceIndex = 0; ServiceIndex < Services.Num(); ServiceIndex++)
	{
		SearchData.AddUniqueUpdate(FBehaviorTreeSearchUpdate(Services[ServiceIndex], SearchData.OwnerComp.GetActiveInstanceIdx(), EBTNodeUpdateMode::Remove));
	}

	// optional: remove all decorators if execution flow leaves this composite
	if (bApplyDecoratorScope)
	{
		const uint16 InstanceIdx = SearchData.OwnerComp.GetActiveInstanceIdx();
		const FBTNodeIndex FromIndex(InstanceIdx, GetExecutionIndex());
		const FBTNodeIndex ToIndex(InstanceIdx, GetLastExecutionIndex());

		SearchData.OwnerComp.UnregisterAuxNodesInRange(FromIndex, ToIndex);

		// remove all pending updates "Add"
		for (int32 Idx = SearchData.PendingUpdates.Num() - 1; Idx >= 0; Idx--)
		{
			const FBehaviorTreeSearchUpdate& UpdateInfo = SearchData.PendingUpdates[Idx];
			if (UpdateInfo.Mode == EBTNodeUpdateMode::Add)
			{
				const uint16 UpdateNodeIdx = UpdateInfo.AuxNode ? UpdateInfo.AuxNode->GetExecutionIndex() : UpdateInfo.TaskNode->GetExecutionIndex();
				const FBTNodeIndex UpdateIdx(UpdateInfo.InstanceIndex, UpdateNodeIdx);

				if (FromIndex.TakesPriorityOver(UpdateIdx) && UpdateIdx.TakesPriorityOver(ToIndex))
				{
					UE_VLOG(SearchData.OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT("Search node update[canceled]: %s"),
						*UBehaviorTreeTypes::DescribeNodeHelper(UpdateInfo.AuxNode ? (UBTNode*)UpdateInfo.AuxNode : (UBTNode*)UpdateInfo.TaskNode));

					SearchData.PendingUpdates.RemoveAt(Idx);
				}
			}
		}
	}
}

void UBTCompositeNode::OnNodeRestart(FBehaviorTreeSearchData& SearchData) const
{
	FBTCompositeMemory* NodeMemory = GetNodeMemory<FBTCompositeMemory>(SearchData);
	NodeMemory->CurrentChild = BTSpecialChild::NotInitialized;
	NodeMemory->OverrideChild = BTSpecialChild::NotInitialized;
}

void UBTCompositeNode::NotifyDecoratorsOnActivation(FBehaviorTreeSearchData& SearchData, int32 ChildIdx) const
{
	const FBTCompositeChild& ChildInfo = Children[ChildIdx];
	for (int32 DecoratorIndex = 0; DecoratorIndex < ChildInfo.Decorators.Num(); DecoratorIndex++)
	{
		const UBTDecorator* DecoratorOb = ChildInfo.Decorators[DecoratorIndex];
		DecoratorOb->WrappedOnNodeActivation(SearchData);

		switch (DecoratorOb->GetFlowAbortMode())
		{
			case EBTFlowAbortMode::LowerPriority:
				SearchData.AddUniqueUpdate(FBehaviorTreeSearchUpdate(DecoratorOb, SearchData.OwnerComp.GetActiveInstanceIdx(), EBTNodeUpdateMode::Remove));
				break;

			case EBTFlowAbortMode::Self:
			case EBTFlowAbortMode::Both:
				SearchData.AddUniqueUpdate(FBehaviorTreeSearchUpdate(DecoratorOb, SearchData.OwnerComp.GetActiveInstanceIdx(), EBTNodeUpdateMode::Add));
				break;

			default:
				break;
		}
	}
}

void UBTCompositeNode::NotifyDecoratorsOnDeactivation(FBehaviorTreeSearchData& SearchData, int32 ChildIdx, EBTNodeResult::Type& NodeResult, const bool bRequestedFromValidInstance) const
{
	const FBTCompositeChild& ChildInfo = Children[ChildIdx];
	if (NodeResult == EBTNodeResult::Aborted)
	{
		// simple notify when aborting execution:
		// - search update will be collected separately (UBehaviorTreeComponent::UnregisterAuxNodesUpTo)
		// - can't modify result in OnNodeProcessed
		for (int32 DecoratorIndex = 0; DecoratorIndex < ChildInfo.Decorators.Num(); DecoratorIndex++)
		{
			const UBTDecorator* DecoratorOb = ChildInfo.Decorators[DecoratorIndex];
			DecoratorOb->WrappedOnNodeDeactivation(SearchData, NodeResult);
		}
	}
	else
	{
		// regular execution flow
		for (int32 DecoratorIndex = 0; DecoratorIndex < ChildInfo.Decorators.Num(); DecoratorIndex++)
		{
			const UBTDecorator* DecoratorOb = ChildInfo.Decorators[DecoratorIndex];
			DecoratorOb->WrappedOnNodeProcessed(SearchData, NodeResult);
			DecoratorOb->WrappedOnNodeDeactivation(SearchData, NodeResult);

			if (!bRequestedFromValidInstance)
			{
				UE_VLOG(SearchData.OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT("Removing decorator(%s) as request is not in active instance"), *UBehaviorTreeTypes::DescribeNodeHelper(this));
				SearchData.AddUniqueUpdate(FBehaviorTreeSearchUpdate(DecoratorOb, SearchData.OwnerComp.GetActiveInstanceIdx(), EBTNodeUpdateMode::Remove));
			}
			// leaving child branch: 
			else if (DecoratorOb->GetFlowAbortMode() == EBTFlowAbortMode::Self)
			{
				UE_VLOG(SearchData.OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT("Removing out of scope decorator(%s) because of self abort type"), *UBehaviorTreeTypes::DescribeNodeHelper(this));
				// - observers with mode "Self" are now out of scope, remove them
				SearchData.AddUniqueUpdate(FBehaviorTreeSearchUpdate(DecoratorOb, SearchData.OwnerComp.GetActiveInstanceIdx(), EBTNodeUpdateMode::Remove));
			}
			else if (DecoratorOb->GetFlowAbortMode() == EBTFlowAbortMode::LowerPriority)
			{
				UE_VLOG(SearchData.OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT("Reactivating decorator(%s) because of lower priority abort type"), *UBehaviorTreeTypes::DescribeNodeHelper(this));
				// - observers with mode "Lower Priority" will try to reactivate themselves ("Both" is not removed on node activation)
				SearchData.AddUniqueUpdate(FBehaviorTreeSearchUpdate(DecoratorOb, SearchData.OwnerComp.GetActiveInstanceIdx(), EBTNodeUpdateMode::Add));
			}
		}
	}
}

void UBTCompositeNode::NotifyDecoratorsOnFailedActivation(FBehaviorTreeSearchData& SearchData, int32 ChildIdx, EBTNodeResult::Type& NodeResult) const
{
	const FBTCompositeChild& ChildInfo = Children[ChildIdx];
	const uint16 ActiveInstanceIdx = SearchData.OwnerComp.GetActiveInstanceIdx();

	for (int32 DecoratorIndex = 0; DecoratorIndex < ChildInfo.Decorators.Num(); DecoratorIndex++)
	{
		const UBTDecorator* DecoratorOb = ChildInfo.Decorators[DecoratorIndex];
		DecoratorOb->WrappedOnNodeProcessed(SearchData, NodeResult);

		if (DecoratorOb->GetFlowAbortMode() == EBTFlowAbortMode::LowerPriority ||
			DecoratorOb->GetFlowAbortMode() == EBTFlowAbortMode::Both)
		{
			SearchData.AddUniqueUpdate(FBehaviorTreeSearchUpdate(DecoratorOb, ActiveInstanceIdx, EBTNodeUpdateMode::Add));
		}
	}

	SearchData.OwnerComp.StoreDebuggerSearchStep(GetChildNode(ChildIdx), ActiveInstanceIdx, NodeResult);
}

void UBTCompositeNode::NotifyChildExecution(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, int32 ChildIdx, EBTNodeResult::Type& NodeResult) const
{
}

void UBTCompositeNode::NotifyNodeActivation(FBehaviorTreeSearchData& SearchData) const
{
}

void UBTCompositeNode::NotifyNodeDeactivation(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type& NodeResult) const
{
}

void UBTCompositeNode::ConditionalNotifyChildExecution(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, const UBTNode& ChildNode, EBTNodeResult::Type& NodeResult) const
{
	if (bUseChildExecutionNotify)
	{
		for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ChildIndex++)
		{
			if (Children[ChildIndex].ChildComposite == &ChildNode || Children[ChildIndex].ChildTask == &ChildNode)
			{
				NotifyChildExecution(OwnerComp, NodeMemory, ChildIndex, NodeResult);
				break;
			}
		}
	}
}

static bool IsLogicOp(const FBTDecoratorLogic& Info)
{
	return (Info.Operation != EBTDecoratorLogic::Test) && (Info.Operation != EBTDecoratorLogic::Invalid);
}

static FString DescribeLogicOp(const TEnumAsByte<EBTDecoratorLogic::Type>& Op)
{
	static FString LogicDesc[] = { TEXT("Invalid"), TEXT("Test"), TEXT("AND"), TEXT("OR"), TEXT("NOT") };
	return LogicDesc[Op];
}

struct FOperationStackInfo
{
	uint16 NumLeft;
	TEnumAsByte<EBTDecoratorLogic::Type> Op;
	uint8 bHasForcedResult : 1;
	uint8 bForcedResult : 1;

	FOperationStackInfo() {}
	FOperationStackInfo(const FBTDecoratorLogic& DecoratorOp) :
		NumLeft(DecoratorOp.Number), Op(DecoratorOp.Operation), bHasForcedResult(0) {};
};

static bool UpdateOperationStack(const UBehaviorTreeComponent& OwnerComp, FString& Indent,
								 TArray<FOperationStackInfo>& Stack, bool bTestResult,
								 int32& FailedDecoratorIdx, int32& NodeDecoratorIdx, bool& bShouldStoreNodeIndex)
{
	if (Stack.Num() == 0)
	{
		return bTestResult;
	}

	FOperationStackInfo& CurrentOp = Stack.Last();
	CurrentOp.NumLeft--;

	if (CurrentOp.Op == EBTDecoratorLogic::And)
	{
		if (!CurrentOp.bHasForcedResult && !bTestResult)
		{
			CurrentOp.bHasForcedResult = true;
			CurrentOp.bForcedResult = bTestResult;
		}
	}
	else if (CurrentOp.Op == EBTDecoratorLogic::Or)
	{
		if (!CurrentOp.bHasForcedResult && bTestResult)
		{
			CurrentOp.bHasForcedResult = true;
			CurrentOp.bForcedResult = bTestResult;
		}	
	}
	else if (CurrentOp.Op == EBTDecoratorLogic::Not)
	{
		bTestResult = !bTestResult;
	}

	// update debugger while processing top level stack
	if (Stack.Num() == 1)
	{
		// reset node flag and grab next decorator index
		bShouldStoreNodeIndex = true;

		// store first failed node
		if (!bTestResult && FailedDecoratorIdx == INDEX_NONE)
		{
			FailedDecoratorIdx = NodeDecoratorIdx;
		}
	}

	if (CurrentOp.bHasForcedResult)
	{
		bTestResult = CurrentOp.bForcedResult;
	}

	if (CurrentOp.NumLeft == 0)
	{
		UE_VLOG(OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT("%s%s finished: %s"), *Indent,
			*DescribeLogicOp(CurrentOp.Op),
			bTestResult ? TEXT("allowed") : TEXT("forbidden"));
		Indent.LeftChopInline(2, EAllowShrinking::No);

		Stack.RemoveAt(Stack.Num() - 1);
		return UpdateOperationStack(OwnerComp, Indent, Stack, bTestResult, FailedDecoratorIdx, NodeDecoratorIdx, bShouldStoreNodeIndex);
	}

	return bTestResult;
}

bool UBTCompositeNode::DoDecoratorsAllowExecution(UBehaviorTreeComponent& OwnerComp, const int32 InstanceIdx, const int32 ChildIdx) const
{
	ensure(Children.IsValidIndex(ChildIdx));
	if (Children.IsValidIndex(ChildIdx) == false)
	{
		UE_VLOG(OwnerComp.GetOwner(), LogBehaviorTree, Error, TEXT("%s: DoDecoratorsAllowExecution called with ChildIdx = %d which is not a valid child index")
			, *UBehaviorTreeTypes::DescribeNodeHelper(this), ChildIdx);
		return false;
	}

	const FBTCompositeChild& ChildInfo = Children[ChildIdx];
	bool bResult = true;

	if (ChildInfo.Decorators.Num() == 0)
	{
		return bResult;
	}

	FBehaviorTreeInstance& MyInstance = OwnerComp.InstanceStack[InstanceIdx];

	const uint16 InstanceIdxUint16 = IntCastChecked<uint16>(InstanceIdx);

	if (ChildInfo.DecoratorOps.Num() == 0)
	{
		// simple check: all decorators must agree
		for (int32 DecoratorIndex = 0; DecoratorIndex < ChildInfo.Decorators.Num(); DecoratorIndex++)
		{
			const UBTDecorator* TestDecorator = ChildInfo.Decorators[DecoratorIndex];
			const bool bIsAllowed = TestDecorator ? TestDecorator->WrappedCanExecute(OwnerComp, TestDecorator->GetNodeMemory<uint8>(MyInstance)) : false;
			OwnerComp.StoreDebuggerSearchStep(TestDecorator, InstanceIdxUint16, bIsAllowed);

			const UBTNode* ChildNode = GetChildNode(ChildIdx);

			if (!bIsAllowed)
			{
				UE_VLOG(OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT("Child[%d] \"%s\" execution forbidden by %s"),
					ChildIdx, *UBehaviorTreeTypes::DescribeNodeHelper(ChildNode), *UBehaviorTreeTypes::DescribeNodeHelper(TestDecorator));

				bResult = false;
				break;
			}
			else
			{
				UE_VLOG(OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT("Child[%d] \"%s\" execution allowed by %s"),
					ChildIdx, *UBehaviorTreeTypes::DescribeNodeHelper(ChildNode), *UBehaviorTreeTypes::DescribeNodeHelper(TestDecorator));
			}
		}
	}
	else
	{
		// advanced check: follow decorator logic operations (composite decorator on child link)
		UE_VLOG(OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT("Child[%d] execution test with logic operations"), ChildIdx);

		TArray<FOperationStackInfo> OperationStack;
		FString Indent;

		// debugger data collection:
		// - get index of each decorator from main AND test, they will match graph nodes
		// - if first operator is not AND it means, that there's only single composite decorator on line
		// - while updating top level stack, grab index of first failed node

		int32 NodeDecoratorIdx = INDEX_NONE;
		int32 FailedDecoratorIdx = INDEX_NONE;
		bool bShouldStoreNodeIndex = true;

		for (int32 OperationIndex = 0; OperationIndex < ChildInfo.DecoratorOps.Num(); OperationIndex++)
		{
			const FBTDecoratorLogic& DecoratorOp = ChildInfo.DecoratorOps[OperationIndex];
			if (IsLogicOp(DecoratorOp))
			{
				OperationStack.Add(FOperationStackInfo(DecoratorOp));
				Indent += TEXT("  ");
				UE_VLOG(OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT("%spushed %s:%d"), *Indent,
					*DescribeLogicOp(DecoratorOp.Operation), DecoratorOp.Number);
			}
			else if (DecoratorOp.Operation == EBTDecoratorLogic::Test)
			{
				const bool bHasOverride = OperationStack.Num() ? OperationStack.Last().bHasForcedResult : false;
				const bool bCurrentOverride = OperationStack.Num() ? OperationStack.Last().bForcedResult : false;

				// debugger: store first decorator of graph node
				if (bShouldStoreNodeIndex)
				{
					bShouldStoreNodeIndex = false;
					NodeDecoratorIdx = DecoratorOp.Number;
				}

				UBTDecorator* TestDecorator = ChildInfo.Decorators[DecoratorOp.Number];
				const bool bIsAllowed = bHasOverride ? bCurrentOverride : TestDecorator->WrappedCanExecute(OwnerComp, TestDecorator->GetNodeMemory<uint8>(MyInstance));
				UE_VLOG(OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT("%s%s %s: %s"), *Indent,
					bHasOverride ? TEXT("skipping") : TEXT("testing"),
					*UBehaviorTreeTypes::DescribeNodeHelper(TestDecorator),
					bIsAllowed ? TEXT("allowed") : TEXT("forbidden"));

				bResult = UpdateOperationStack(OwnerComp, Indent, OperationStack, bIsAllowed, FailedDecoratorIdx, NodeDecoratorIdx, bShouldStoreNodeIndex);
				if (OperationStack.Num() == 0)
				{
					UE_VLOG(OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT("finished execution test: %s"),
						bResult ? TEXT("allowed") : TEXT("forbidden"));

					OwnerComp.StoreDebuggerSearchStep(ChildInfo.Decorators[FMath::Max(0, FailedDecoratorIdx)], InstanceIdxUint16, bResult);
					break;
				}
			}
		}
	}

	return bResult;
}

int32 UBTCompositeNode::GetMatchingChildIndex(int32 ActiveInstanceIdx, FBTNodeIndex& NodeIdx) const
{
	const int32 OutsideRange = BTSpecialChild::ReturnToParent;
	const int32 UnlimitedRange = Children.Num() - 1;

	// search ends at the same instance level: use execution index to determine branch containing node index
	if (ActiveInstanceIdx == NodeIdx.InstanceIndex)
	{
		// is composite even in range of search?
		if (GetExecutionIndex() > NodeIdx.ExecutionIndex)
		{
			return OutsideRange;
		}

		// find child outside range
		for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ChildIndex++)
		{
			const uint16 FirstIndexInBranch = GetChildExecutionIndex(ChildIndex, EBTChildIndex::FirstNode);
			if (FirstIndexInBranch > NodeIdx.ExecutionIndex)
			{
				return ChildIndex ? (ChildIndex - 1) : 0;
			}
		}

		return UnlimitedRange;
	}

	// search ends at higher level: allow every node
	// search ends at lower level: outside allowed range
	return (ActiveInstanceIdx > NodeIdx.InstanceIndex) ? UnlimitedRange : OutsideRange;
}

uint16 UBTCompositeNode::GetBranchExecutionIndex(uint16 NodeInBranchIdx) const
{
	uint16 PrevBranchStartIdx = GetExecutionIndex();
	for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ChildIndex++)
	{
		const uint16 BranchStartIdx = GetChildExecutionIndex(ChildIndex, EBTChildIndex::FirstNode);
		if (BranchStartIdx > NodeInBranchIdx)
		{
			break;
		}

		PrevBranchStartIdx = BranchStartIdx;
	}

	return PrevBranchStartIdx;
}

int32 UBTCompositeNode::GetNextChild(FBehaviorTreeSearchData& SearchData, int32 LastChildIdx, EBTNodeResult::Type LastResult) const
{
	FBTCompositeMemory* NodeMemory = GetNodeMemory<FBTCompositeMemory>(SearchData);
	int32 NextChildIndex = BTSpecialChild::ReturnToParent;
	uint16 ActiveInstanceIdx = SearchData.OwnerComp.GetActiveInstanceIdx();

	// newly activated node, search range not reached yet: select search branch for decorator test
	if (LastChildIdx == BTSpecialChild::NotInitialized && SearchData.SearchStart.IsSet() &&
		FBTNodeIndex(ActiveInstanceIdx, GetExecutionIndex()).TakesPriorityOver(SearchData.SearchStart))
	{
		NextChildIndex = GetMatchingChildIndex(ActiveInstanceIdx, SearchData.SearchStart);
	}
	else if (NodeMemory->OverrideChild != BTSpecialChild::NotInitialized && !SearchData.OwnerComp.IsRestartPending())
	{
		NextChildIndex = NodeMemory->OverrideChild;
		NodeMemory->OverrideChild = BTSpecialChild::NotInitialized;
	}
	// or use composite's logic
	else 
	{
		NextChildIndex = GetNextChildHandler(SearchData, LastChildIdx, LastResult);
	}

	return NextChildIndex;
}

void UBTCompositeNode::SetChildOverride(FBehaviorTreeSearchData& SearchData, int8 Index) const
{
	if (Children.IsValidIndex(Index) || Index == BTSpecialChild::ReturnToParent)
	{
		FBTCompositeMemory* MyMemory = GetNodeMemory<FBTCompositeMemory>(SearchData);
		MyMemory->OverrideChild = Index;
	}
}

void UBTCompositeNode::RequestDelayedExecution(UBehaviorTreeComponent& OwnerComp, EBTNodeResult::Type LastResult) const
{
	OwnerComp.RequestExecution(LastResult);
}

uint16 UBTCompositeNode::GetChildExecutionIndex(int32 Index, EBTChildIndex ChildMode) const
{
	const UBTNode* ChildNode = GetChildNode(Index);
	if (ChildNode)
	{
		int32 Offset = 0;

		// When getting execution index of the first node we need to consider auxiliary nodes
		if (ChildMode == EBTChildIndex::FirstNode)
		{
			Offset += Children[Index].Decorators.Num();

			// Task nodes may also have service nodes to consider
			if (UBTTaskNode* ChildTask = Children[Index].ChildTask)
			{
				Offset += ChildTask->Services.Num();
			}
		}

		return IntCastChecked<uint16>(ChildNode->GetExecutionIndex() - Offset);
	}

	return (LastExecutionIndex + 1);
}

bool UBTCompositeNode::CanPushSubtree(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, int32 ChildIdx) const
{
	return true;
}

bool UBTCompositeNode::CanNotifyDecoratorsOnActivation(FBehaviorTreeSearchData& SearchData, int32 ChildIdx) const
{
	return true;
}

bool UBTCompositeNode::CanNotifyDecoratorsOnDeactivation(FBehaviorTreeSearchData& SearchData, int32 ChildIdx, EBTNodeResult::Type& NodeResult) const
{
	return true;
}

bool UBTCompositeNode::CanNotifyDecoratorsOnFailedActivation(FBehaviorTreeSearchData& SearchData, int32 ChildIdx, EBTNodeResult::Type& NodeResult) const
{
	return true;
}

void UBTCompositeNode::DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const
{
	Super::DescribeRuntimeValues(OwnerComp, NodeMemory, Verbosity, Values);

	if (Verbosity == EBTDescriptionVerbosity::Detailed)
	{
		FBTCompositeMemory* MyMemory = (FBTCompositeMemory*)NodeMemory;
		Values.Add(FString::Printf(TEXT("current child: %d"), MyMemory->CurrentChild));
		Values.Add(FString::Printf(TEXT("override child: %d"), MyMemory->OverrideChild));
	}
}

#if WITH_EDITOR

bool UBTCompositeNode::CanAbortSelf() const
{
	return true;
}

bool UBTCompositeNode::CanAbortLowerPriority() const
{
	return true;
}

#endif

uint16 UBTCompositeNode::GetInstanceMemorySize() const
{
	return sizeof(FBTCompositeMemory);
}

void UBTCompositeNode::InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const
{
	InitializeNodeMemory<FBTCompositeMemory>(NodeMemory, InitType);
}

void UBTCompositeNode::CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const
{
	CleanupNodeMemory<FBTCompositeMemory>(NodeMemory, CleanupType);
}


