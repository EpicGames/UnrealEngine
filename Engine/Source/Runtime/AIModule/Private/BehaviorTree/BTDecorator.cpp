// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/BTDecorator.h"
#include "BehaviorTree/BTCompositeNode.h"
#include "GameFramework/Actor.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTDecorator)

UBTDecorator::UBTDecorator(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	FlowAbortMode = EBTFlowAbortMode::None;
	bAllowAbortNone = true;
	bAllowAbortLowerPri = true;
	bAllowAbortChildNodes = true;
	bNotifyActivation = false;
	bNotifyDeactivation = false;
	bNotifyProcessed = false;

	bShowInverseConditionDesc = true;
	bInverseCondition = false;
}

bool UBTDecorator::CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	return true;
}

void UBTDecorator::SetIsInversed(bool bShouldBeInversed)
{
	bInverseCondition = bShouldBeInversed;
}

void UBTDecorator::OnNodeActivation(FBehaviorTreeSearchData& SearchData)
{
}

void UBTDecorator::OnNodeDeactivation(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type NodeResult)
{
}

void UBTDecorator::OnNodeProcessed(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type& NodeResult)
{
}

bool UBTDecorator::WrappedCanExecute(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	const UBTDecorator* NodeOb = bCreateNodeInstance ? (const UBTDecorator*)GetNodeInstance(OwnerComp, NodeMemory) : this;
	return NodeOb ? (IsInversed() != NodeOb->CalculateRawConditionValue(OwnerComp, NodeMemory)) : false;
}

void UBTDecorator::WrappedOnNodeActivation(FBehaviorTreeSearchData& SearchData) const
{
	if (bNotifyActivation)
	{
		const UBTNode* NodeOb = bCreateNodeInstance ? GetNodeInstance(SearchData) : this;
		if (NodeOb)
		{
			((UBTDecorator*)NodeOb)->OnNodeActivation(SearchData);
		}		
	}
};

void UBTDecorator::WrappedOnNodeDeactivation(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type NodeResult) const
{
	if (bNotifyDeactivation)
	{
		const UBTNode* NodeOb = bCreateNodeInstance ? GetNodeInstance(SearchData) : this;
		if (NodeOb)
		{
			((UBTDecorator*)NodeOb)->OnNodeDeactivation(SearchData, NodeResult);
		}		
	}
}

void UBTDecorator::WrappedOnNodeProcessed(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type& NodeResult) const
{
	if (bNotifyProcessed)
	{
		const UBTNode* NodeOb = bCreateNodeInstance ? GetNodeInstance(SearchData) : this;
		if (NodeOb)
		{
			((UBTDecorator*)NodeOb)->OnNodeProcessed(SearchData, NodeResult);
		}		
	}
}

void UBTDecorator::ConditionalFlowAbort(UBehaviorTreeComponent& OwnerComp, EBTDecoratorAbortRequest RequestMode) const
{
	if (FlowAbortMode == EBTFlowAbortMode::None)
	{
		return;
	}

	const int32 InstanceIdx = OwnerComp.FindInstanceContainingNode(GetParentNode());
	if (InstanceIdx == INDEX_NONE)
	{
		return;
	}

	uint8* NodeMemory = OwnerComp.GetNodeMemory((UBTNode*)this, InstanceIdx);

	const bool bPass = WrappedCanExecute(OwnerComp, NodeMemory);
	const bool bAlwaysRequestWhenPassing = (RequestMode == EBTDecoratorAbortRequest::ConditionPassing);

	UE_VLOG(OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT("%s, ConditionalFlowAbort(%s) pass:%s"),
		*UBehaviorTreeTypes::DescribeNodeHelper(this),
		bAlwaysRequestWhenPassing ? TEXT("always when passing") : TEXT("on change"), 
		*LexToString(bPass));


	if (!bPass)
	{
		OwnerComp.RequestBranchDeactivation(*this);
	}
	else
	{
		OwnerComp.RequestBranchActivation(*this, bAlwaysRequestWhenPassing);
	}
}

FString UBTDecorator::GetStaticDescription() const
{
	FString FlowAbortDesc;
	if (FlowAbortMode != EBTFlowAbortMode::None)
	{
		FlowAbortDesc = FString::Printf(TEXT("aborts %s"), *UBehaviorTreeTypes::DescribeFlowAbortMode(FlowAbortMode).ToLower());
	}

	FString InversedDesc;
	if (bShowInverseConditionDesc && IsInversed())
	{
		InversedDesc = TEXT("inversed");
	}

	FString AdditionalDesc;
	if (FlowAbortDesc.Len() || InversedDesc.Len())
	{
		AdditionalDesc = FString::Printf(TEXT("( %s%s%s )\n"), *FlowAbortDesc, 
			(FlowAbortDesc.Len() > 0) && (InversedDesc.Len() > 0) ? TEXT(", ") : TEXT(""),
			*InversedDesc);
	}

	return FString::Printf(TEXT("%s%s"), *AdditionalDesc, *UBehaviorTreeTypes::GetShortTypeName(this));
}

bool UBTDecorator::IsFlowAbortModeValid() const
{
#if WITH_EDITOR
	if (GetParentNode() == NULL ||
		(GetParentNode()->CanAbortLowerPriority() == false && GetParentNode()->CanAbortSelf() == false))
	{
		return (FlowAbortMode == EBTFlowAbortMode::None);
	}

	if (GetParentNode()->CanAbortLowerPriority() == false)
	{
		return (FlowAbortMode == EBTFlowAbortMode::None || FlowAbortMode == EBTFlowAbortMode::Self);
	}

	if (GetParentNode()->CanAbortSelf() == false)
	{
		return (FlowAbortMode == EBTFlowAbortMode::None || FlowAbortMode == EBTFlowAbortMode::LowerPriority);
	}
#endif

	return true;
}

void UBTDecorator::UpdateFlowAbortMode()
{
#if WITH_EDITOR
	if (GetParentNode() == NULL)
	{
		FlowAbortMode = EBTFlowAbortMode::None;
		return;
	}

	if (GetParentNode()->CanAbortLowerPriority() == false)
	{
		if (FlowAbortMode == EBTFlowAbortMode::Both)
		{
			FlowAbortMode = GetParentNode()->CanAbortSelf() ? EBTFlowAbortMode::Self : EBTFlowAbortMode::None;
		}
		else if (FlowAbortMode == EBTFlowAbortMode::LowerPriority)
		{
			FlowAbortMode = EBTFlowAbortMode::None;
		}
	}

	if (GetParentNode()->CanAbortSelf() == false)
	{
		if (FlowAbortMode == EBTFlowAbortMode::Both)
		{
			FlowAbortMode = GetParentNode()->CanAbortLowerPriority() ? EBTFlowAbortMode::LowerPriority : EBTFlowAbortMode::None;
		}
		else if (FlowAbortMode == EBTFlowAbortMode::Self)
		{
			FlowAbortMode = EBTFlowAbortMode::None;
		}
	}
#endif
}


