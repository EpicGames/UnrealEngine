// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/BTFunctionLibrary.h"
#include "GameFramework/Actor.h"
#include "BehaviorTree/BehaviorTreeComponent.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Enum.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Object.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Vector.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Rotator.h"
#include "BehaviorTree/BTNode.h"
#include "BlueprintNodeHelpers.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Bool.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Class.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Float.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Int.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Name.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_String.h"
#include "Misc/RuntimeErrors.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTFunctionLibrary)

//----------------------------------------------------------------------//
// 
//----------------------------------------------------------------------//
namespace FBTNodeBPImplementationHelper
{
	int32 CheckEventImplementationVersion(FName GenericEventName, FName AIEventName, const UObject& Object, const UClass& StopAtClass)
	{
		const bool bGeneric = BlueprintNodeHelpers::HasBlueprintFunction(GenericEventName, Object, StopAtClass);
		const bool bAI = BlueprintNodeHelpers::HasBlueprintFunction(AIEventName, Object, StopAtClass);

		return (bGeneric ? Generic : NoImplementation) | (bAI ? AISpecific : NoImplementation);
	}
}

//----------------------------------------------------------------------//
// 
//----------------------------------------------------------------------//

UBehaviorTreeComponent* UBTFunctionLibrary::GetOwnerComponent(UBTNode* NodeOwner)
{
	ensureAsRuntimeWarning((NodeOwner != nullptr) && (NodeOwner->GetClass()->HasAnyClassFlags(CLASS_CompiledFromBlueprint)));

	UBehaviorTreeComponent* OwnerComp = NodeOwner ? Cast<UBehaviorTreeComponent>(NodeOwner->GetOuter()) : nullptr;
	ensureAsRuntimeWarning(OwnerComp != nullptr);

	return OwnerComp;
}

UBlackboardComponent* UBTFunctionLibrary::GetOwnersBlackboard(UBTNode* NodeOwner)
{
	UBehaviorTreeComponent* BTComponent = GetOwnerComponent(NodeOwner);
	if (ensureAsRuntimeWarning(BTComponent != nullptr))
	{
		return BTComponent->GetBlackboardComponent();
	}
	return nullptr;
}

UObject* UBTFunctionLibrary::GetBlackboardValueAsObject(UBTNode* NodeOwner, const FBlackboardKeySelector& Key)
{
	UBlackboardComponent* BlackboardComp = GetOwnersBlackboard(NodeOwner);
	return BlackboardComp ? BlackboardComp->GetValueAsObject(Key.SelectedKeyName) : nullptr;
}

AActor* UBTFunctionLibrary::GetBlackboardValueAsActor(UBTNode* NodeOwner, const FBlackboardKeySelector& Key)
{
	return Cast<AActor>(GetBlackboardValueAsObject(NodeOwner, Key));
}

UClass* UBTFunctionLibrary::GetBlackboardValueAsClass(UBTNode* NodeOwner, const FBlackboardKeySelector& Key)
{
	UBlackboardComponent* BlackboardComp = GetOwnersBlackboard(NodeOwner);
	return BlackboardComp ? BlackboardComp->GetValueAsClass(Key.SelectedKeyName) : nullptr;
}

uint8 UBTFunctionLibrary::GetBlackboardValueAsEnum(UBTNode* NodeOwner, const FBlackboardKeySelector& Key)
{
	UBlackboardComponent* BlackboardComp = GetOwnersBlackboard(NodeOwner);
	return BlackboardComp ? BlackboardComp->GetValueAsEnum(Key.SelectedKeyName) : 0;
}

int32 UBTFunctionLibrary::GetBlackboardValueAsInt(UBTNode* NodeOwner, const FBlackboardKeySelector& Key)
{
	UBlackboardComponent* BlackboardComp = GetOwnersBlackboard(NodeOwner);
	return BlackboardComp ? BlackboardComp->GetValueAsInt(Key.SelectedKeyName) : 0;
}

float UBTFunctionLibrary::GetBlackboardValueAsFloat(UBTNode* NodeOwner, const FBlackboardKeySelector& Key)
{
	UBlackboardComponent* BlackboardComp = GetOwnersBlackboard(NodeOwner);
	return BlackboardComp ? BlackboardComp->GetValueAsFloat(Key.SelectedKeyName) : 0.0f;
}

bool UBTFunctionLibrary::GetBlackboardValueAsBool(UBTNode* NodeOwner, const FBlackboardKeySelector& Key)
{
	UBlackboardComponent* BlackboardComp = GetOwnersBlackboard(NodeOwner);
	return BlackboardComp ? BlackboardComp->GetValueAsBool(Key.SelectedKeyName) : false;
}

FString UBTFunctionLibrary::GetBlackboardValueAsString(UBTNode* NodeOwner, const FBlackboardKeySelector& Key)
{
	UBlackboardComponent* BlackboardComp = GetOwnersBlackboard(NodeOwner);
	return BlackboardComp ? BlackboardComp->GetValueAsString(Key.SelectedKeyName) : FString();
}

FName UBTFunctionLibrary::GetBlackboardValueAsName(UBTNode* NodeOwner, const FBlackboardKeySelector& Key)
{
	UBlackboardComponent* BlackboardComp = GetOwnersBlackboard(NodeOwner);
	return BlackboardComp ? BlackboardComp->GetValueAsName(Key.SelectedKeyName) : NAME_None;
}

FVector UBTFunctionLibrary::GetBlackboardValueAsVector(UBTNode* NodeOwner, const FBlackboardKeySelector& Key)
{
	UBlackboardComponent* BlackboardComp = GetOwnersBlackboard(NodeOwner);
	return BlackboardComp ? BlackboardComp->GetValueAsVector(Key.SelectedKeyName) : FVector::ZeroVector;
}

FRotator UBTFunctionLibrary::GetBlackboardValueAsRotator(UBTNode* NodeOwner, const FBlackboardKeySelector& Key)
{
	UBlackboardComponent* BlackboardComp = GetOwnersBlackboard(NodeOwner);
	return BlackboardComp ? BlackboardComp->GetValueAsRotator(Key.SelectedKeyName) : FRotator::ZeroRotator;
}

void UBTFunctionLibrary::SetBlackboardValueAsObject(UBTNode* NodeOwner, const FBlackboardKeySelector& Key, UObject* Value)
{
	if (UBlackboardComponent* BlackboardComp = GetOwnersBlackboard(NodeOwner))
	{
		BlackboardComp->SetValue<UBlackboardKeyType_Object>(Key.SelectedKeyName, Value);
	}
}

void UBTFunctionLibrary::SetBlackboardValueAsClass(UBTNode* NodeOwner, const FBlackboardKeySelector& Key, UClass* Value)
{
	if (UBlackboardComponent* BlackboardComp = GetOwnersBlackboard(NodeOwner))
	{
		BlackboardComp->SetValue<UBlackboardKeyType_Class>(Key.SelectedKeyName, Value);
	}
}

void UBTFunctionLibrary::SetBlackboardValueAsEnum(UBTNode* NodeOwner, const FBlackboardKeySelector& Key, uint8 Value)
{
	if (UBlackboardComponent* BlackboardComp = GetOwnersBlackboard(NodeOwner))
	{
		BlackboardComp->SetValue<UBlackboardKeyType_Enum>(Key.SelectedKeyName, Value);
	}
}

void UBTFunctionLibrary::SetBlackboardValueAsInt(UBTNode* NodeOwner, const FBlackboardKeySelector& Key, int32 Value)
{
	if (UBlackboardComponent* BlackboardComp = GetOwnersBlackboard(NodeOwner))
	{
		BlackboardComp->SetValue<UBlackboardKeyType_Int>(Key.SelectedKeyName, Value);
	}
}

void UBTFunctionLibrary::SetBlackboardValueAsFloat(UBTNode* NodeOwner, const FBlackboardKeySelector& Key, float Value)
{
	if (UBlackboardComponent* BlackboardComp = GetOwnersBlackboard(NodeOwner))
	{
		BlackboardComp->SetValue<UBlackboardKeyType_Float>(Key.SelectedKeyName, Value);
	}
}

void UBTFunctionLibrary::SetBlackboardValueAsBool(UBTNode* NodeOwner, const FBlackboardKeySelector& Key, bool Value)
{
	if (UBlackboardComponent* BlackboardComp = GetOwnersBlackboard(NodeOwner))
	{
		BlackboardComp->SetValue<UBlackboardKeyType_Bool>(Key.SelectedKeyName, Value);
	}
}

void UBTFunctionLibrary::SetBlackboardValueAsString(UBTNode* NodeOwner, const FBlackboardKeySelector& Key, FString Value)
{
	if (UBlackboardComponent* BlackboardComp = GetOwnersBlackboard(NodeOwner))
	{
		BlackboardComp->SetValue<UBlackboardKeyType_String>(Key.SelectedKeyName, Value);
	}
}

void UBTFunctionLibrary::SetBlackboardValueAsName(UBTNode* NodeOwner, const FBlackboardKeySelector& Key, FName Value)
{
	if (UBlackboardComponent* BlackboardComp = GetOwnersBlackboard(NodeOwner))
	{
		BlackboardComp->SetValue<UBlackboardKeyType_Name>(Key.SelectedKeyName, Value);
	}
}

void UBTFunctionLibrary::SetBlackboardValueAsVector(UBTNode* NodeOwner, const FBlackboardKeySelector& Key, FVector Value)
{
	if (UBlackboardComponent* BlackboardComp = GetOwnersBlackboard(NodeOwner))
	{
		BlackboardComp->SetValue<UBlackboardKeyType_Vector>(Key.SelectedKeyName, Value);
	}
}

void UBTFunctionLibrary::ClearBlackboardValueAsVector(UBTNode* NodeOwner, const FBlackboardKeySelector& Key)
{
	if (UBlackboardComponent* BlackboardComp = GetOwnersBlackboard(NodeOwner))
	{
		BlackboardComp->ClearValue(Key.SelectedKeyName);
	}
}

void UBTFunctionLibrary::SetBlackboardValueAsRotator(UBTNode* NodeOwner, const FBlackboardKeySelector& Key, FRotator Value)
{
	if (UBlackboardComponent* BlackboardComp = GetOwnersBlackboard(NodeOwner))
	{
		BlackboardComp->SetValue<UBlackboardKeyType_Rotator>(Key.SelectedKeyName, Value);
	}
}

void UBTFunctionLibrary::ClearBlackboardValue(UBTNode* NodeOwner, const FBlackboardKeySelector& Key)
{
	if (UBlackboardComponent* BlackboardComp = GetOwnersBlackboard(NodeOwner))
	{
		BlackboardComp->ClearValue(Key.SelectedKeyName);
	}
}

void UBTFunctionLibrary::StartUsingExternalEvent(UBTNode* NodeOwner, AActor* OwningActor)
{
	// deprecated, not removed yet
}

void UBTFunctionLibrary::StopUsingExternalEvent(UBTNode* NodeOwner)
{
	// deprecated, not removed yet
}

