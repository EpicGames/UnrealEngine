// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/BTNode.h"
#include "AIController.h"
#include "BehaviorTree/BehaviorTree.h"
#include "BehaviorTree/BTCompositeNode.h"
#include "BehaviorTree/ValueOrBBKey.h"
#include "Engine/World.h"
#include "GameplayTasksComponent.h"
#include "Styling/AppStyle.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTNode)

//----------------------------------------------------------------------//
// UBTNode
//----------------------------------------------------------------------//

UBTNode::UBTNode(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	ParentNode = NULL;
	TreeAsset = NULL;
	ExecutionIndex = 0;
	MemoryOffset = 0;
	TreeDepth = 0;
	bCreateNodeInstance = false;
	bIsInstanced = false;
	bIsInjected = false;

#if USE_BEHAVIORTREE_DEBUGGER
	NextExecutionNode = NULL;
#endif
}

UWorld* UBTNode::GetWorld() const
{
	if (GetOuter() == nullptr)
	{
		return nullptr;
	}

	// Special case for behavior tree nodes in the editor
	if (Cast<UPackage>(GetOuter()) != nullptr)
	{
		// GetOuter should return a UPackage and its Outer is a UWorld
		return Cast<UWorld>(GetOuter()->GetOuter());
	}

	// In all other cases...
	return GetOuter()->GetWorld();
}

void UBTNode::InitializeNode(UBTCompositeNode* InParentNode, uint16 InExecutionIndex, uint16 InMemoryOffset, uint8 InTreeDepth)
{
	ParentNode = InParentNode;
	ExecutionIndex = InExecutionIndex;
	MemoryOffset = InMemoryOffset;
	TreeDepth = InTreeDepth;
}

void UBTNode::InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const
{
	// empty in base 
}

void UBTNode::CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const
{
	// empty in base 
}

void UBTNode::OnInstanceCreated(UBehaviorTreeComponent& OwnerComp)
{
	// empty in base class
}

void UBTNode::OnInstanceDestroyed(UBehaviorTreeComponent& OwnerComp)
{
	// empty in base class
}

void UBTNode::InitializeInSubtree(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, int32& NextInstancedIndex, EBTMemoryInit::Type InitType) const
{
	FBTInstancedNodeMemory* SpecialMemory = GetSpecialNodeMemory<FBTInstancedNodeMemory>(NodeMemory);
	if (SpecialMemory)
	{
		SpecialMemory->NodeIdx = INDEX_NONE;
	}

	if (bCreateNodeInstance)
	{
		// composite nodes can't be instanced!
		check(IsA(UBTCompositeNode::StaticClass()) == false);

		UBTNode* NodeInstance = OwnerComp.NodeInstances.IsValidIndex(NextInstancedIndex) ? ToRawPtr(OwnerComp.NodeInstances[NextInstancedIndex]) : NULL;
		if (NodeInstance == NULL)
		{
			NodeInstance = (UBTNode*)StaticDuplicateObject(this, &OwnerComp);
			NodeInstance->InitializeNode(GetParentNode(), GetExecutionIndex(), GetMemoryOffset(), GetTreeDepth());
			NodeInstance->bIsInstanced = true;

			OwnerComp.NodeInstances.Add(NodeInstance);
		}

		check(NodeInstance);
		check(SpecialMemory);

		SpecialMemory->NodeIdx = NextInstancedIndex;

		NodeInstance->SetOwner(OwnerComp.GetOwner());
		NodeInstance->InitializeMemory(OwnerComp, NodeMemory, InitType);
		check(TreeAsset);

		FScopedBTLoggingContext LogContext(NodeInstance);
		NodeInstance->InitializeFromAsset(*TreeAsset);
		NodeInstance->OnInstanceCreated(OwnerComp);
		NextInstancedIndex++;
	}
	else
	{
		InitializeMemory(OwnerComp, NodeMemory, InitType);
	}
}

void UBTNode::CleanupInSubtree(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const
{
	const UBTNode* NodeOb = bCreateNodeInstance ? GetNodeInstance(OwnerComp, NodeMemory) : this;
	if (NodeOb)
	{
		NodeOb->CleanupMemory(OwnerComp, NodeMemory, CleanupType);
	}
}

#if USE_BEHAVIORTREE_DEBUGGER
void UBTNode::InitializeExecutionOrder(UBTNode* NextNode)
{
	NextExecutionNode = NextNode;
}
#endif

void UBTNode::InitializeFromAsset(UBehaviorTree& Asset)
{
	TreeAsset = &Asset;
}

UBlackboardData* UBTNode::GetBlackboardAsset() const
{
	return TreeAsset ? TreeAsset->BlackboardAsset : NULL;
}

uint16 UBTNode::GetInstanceMemorySize() const
{
	return 0;
}

uint16 UBTNode::GetSpecialMemorySize() const
{
	return bCreateNodeInstance ? sizeof(FBTInstancedNodeMemory) : 0;
}

UBTNode* UBTNode::GetNodeInstance(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	FBTInstancedNodeMemory* MyMemory = GetSpecialNodeMemory<FBTInstancedNodeMemory>(NodeMemory);
	return MyMemory && OwnerComp.NodeInstances.IsValidIndex(MyMemory->NodeIdx) ?
		OwnerComp.NodeInstances[MyMemory->NodeIdx] : NULL;
}

UBTNode* UBTNode::GetNodeInstance(FBehaviorTreeSearchData& SearchData) const
{
	return GetNodeInstance(SearchData.OwnerComp, GetNodeMemory<uint8>(SearchData));
}

FString UBTNode::GetNodeName() const
{
	return NodeName.Len() ? NodeName : UBehaviorTreeTypes::GetShortTypeName(this);
}

FString UBTNode::GetRuntimeDescription(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity) const
{
	FString Description = NodeName.Len() ? FString::Printf(TEXT("%d. %s [%s]"), ExecutionIndex, *NodeName, *GetStaticDescription()) : GetStaticDescription();
	TArray<FString> RuntimeValues;

	const UBTNode* NodeOb = bCreateNodeInstance ? GetNodeInstance(OwnerComp, NodeMemory) : this;
	if (NodeOb)
	{
		NodeOb->DescribeRuntimeValues(OwnerComp, NodeMemory, Verbosity, RuntimeValues);
	}

	for (int32 ValueIndex = 0; ValueIndex < RuntimeValues.Num(); ValueIndex++)
	{
		Description += TEXT(", ");
		Description += RuntimeValues[ValueIndex];
	}

	return Description;
}

FString UBTNode::GetStaticDescription() const
{
	// short type name
	return UBehaviorTreeTypes::GetShortTypeName(this);
}

void UBTNode::DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const
{
	// nothing stored in memory for base class
}

#if WITH_EDITOR

const ISlateStyle& UBTNode::GetNodeIconStyleSet() const
{
	return FAppStyle::Get();
}

FName UBTNode::GetNodeIconName() const
{
	return NAME_None;
}

bool UBTNode::UsesBlueprint() const
{
	return false;
}

FString UBTNode::GetErrorMessage() const
{
	return FString();
}

void UBTNode::PreSave(FObjectPreSaveContext SaveContext)
{
	if (const UBlackboardData* Blackboard = GetBlackboardAsset())
	{
		for (TFieldIterator<FStructProperty> StructPropIt(GetClass()); StructPropIt; ++StructPropIt)
		{
			if (StructPropIt->Struct && StructPropIt->Struct->IsChildOf(FValueOrBlackboardKeyBase::StaticStruct()))
			{
				if (FValueOrBlackboardKeyBase* ValueOrBBKey = StructPropIt->ContainerPtrToValuePtr<FValueOrBlackboardKeyBase>(this))
				{
					ValueOrBBKey->PreSave(this, *Blackboard, StructPropIt->GetFName());
				}
			}
			else if (StructPropIt->Struct && StructPropIt->Struct->IsChildOf(FBlackboardKeySelector::StaticStruct()))
			{
				if (FBlackboardKeySelector* KeySelector = StructPropIt->ContainerPtrToValuePtr<FBlackboardKeySelector>(this))
				{
					KeySelector->PreSave(this, *Blackboard, StructPropIt->GetFName());
				}
			}
		}
	}

	Super::PreSave(SaveContext);
}

const UObject* UBTNode::GetAssociatedAsset(TOptional<FBehaviorTreeNodeDebugContext> DebugContext) const
{
	// By default we return the blueprint the node is derived from. If the node is not a blueprint it will return nullptr.
	return UBlueprint::GetBlueprintFromClass(GetClass());
}

#endif // WITH_EDITOR

UGameplayTasksComponent* UBTNode::GetGameplayTasksComponent(const UGameplayTask& Task) const
{
	const UAITask* AITask = Cast<const UAITask>(&Task);
	return (AITask && AITask->GetAIController()) ? AITask->GetAIController()->GetGameplayTasksComponent(Task) : Task.GetGameplayTasksComponent();
}

AActor* UBTNode::GetGameplayTaskOwner(const UGameplayTask* Task) const
{
	if (Task == nullptr)
	{
		if (IsInstanced())
		{
			const UBehaviorTreeComponent* BTComponent = Cast<const UBehaviorTreeComponent>(GetOuter());
			//not having BT component for an instanced BT node is invalid!
			check(BTComponent);
			return BTComponent->GetAIOwner();
		}
		else
		{
			UE_LOGF(LogBehaviorTree, Warning, "%ls: Unable to determine default GameplayTaskOwner!", *GetName());
			return nullptr;
		}
	}

	const UAITask* AITask = Cast<const UAITask>(Task);
	if (AITask)
	{
		return AITask->GetAIController();
	}

	const UGameplayTasksComponent* TasksComponent = Task->GetGameplayTasksComponent();
	return TasksComponent ? TasksComponent->GetGameplayTaskOwner(Task) : nullptr;
}

AActor* UBTNode::GetGameplayTaskAvatar(const UGameplayTask* Task) const
{
	if (Task == nullptr)
	{
		if (IsInstanced())
		{
			const UBehaviorTreeComponent* BTComponent = Cast<const UBehaviorTreeComponent>(GetOuter());
			//not having BT component for an instanced BT node is invalid!
			check(BTComponent);
			return BTComponent->GetAIOwner();
		}
		else
		{
			UE_LOGF(LogBehaviorTree, Warning, "%ls: Unable to determine default GameplayTaskAvatar!", *GetName());
			return nullptr;
		}
	}

	const UAITask* AITask = Cast<const UAITask>(Task);
	if (AITask)
	{
		return AITask->GetAIController() ? AITask->GetAIController()->GetPawn() : nullptr;
	}

	const UGameplayTasksComponent* TasksComponent = Task->GetGameplayTasksComponent();
	return TasksComponent ? TasksComponent->GetGameplayTaskAvatar(Task) : nullptr;
}

uint8 UBTNode::GetGameplayTaskDefaultPriority() const
{
	return static_cast<uint8>(EAITaskPriority::AutonomousAI);
}

void UBTNode::OnGameplayTaskInitialized(UGameplayTask& Task)
{
	const UAITask* AITask = Cast<const UAITask>(&Task);
	if (AITask && (AITask->GetAIController() == nullptr))
	{
		// this means that the task has either been created without specifying 
		// UAITAsk::OwnerController's value (like via BP's Construct Object node)
		// or it has been created in C++ with inappropriate function
		UE_LOGF(LogBehaviorTree, Error, "Missing AIController in AITask %ls", *AITask->GetName());
	}
}

UBehaviorTreeComponent* UBTNode::GetBTComponentForTask(UGameplayTask& Task) const
{
	UAITask* AITask = Cast<UAITask>(&Task);
	return (AITask && AITask->GetAIController()) ? Cast<UBehaviorTreeComponent>(AITask->GetAIController()->BrainComponent) : nullptr;
}

//----------------------------------------------------------------------//
// DEPRECATED
//----------------------------------------------------------------------//

