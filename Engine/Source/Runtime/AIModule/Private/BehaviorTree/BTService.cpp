// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/BTService.h"
#include "GameFramework/Actor.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTService)

UBTService::UBTService(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	bNotifyTick = true;
	bNotifyOnSearch = true;
	bTickIntervals = true;
#if WITH_EDITORONLY_DATA
	bCanTickOnSearchStartBeExposed = true;
#endif // WITH_EDITORONLY_DATA
	bCallTickOnSearchStart = false;
	bRestartTimerOnEachActivation = false;

	Interval = 0.5f;
	RandomDeviation = 0.1f;
}

void UBTService::TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds)
{
	ScheduleNextTick(OwnerComp, NodeMemory);
}

void UBTService::OnSearchStart(FBehaviorTreeSearchData& SearchData)
{
	// empty in base class
}

void UBTService::NotifyParentActivation(FBehaviorTreeSearchData& SearchData)
{
	if (bNotifyOnSearch || bNotifyTick)
	{
		UBTNode* NodeOb = bCreateNodeInstance ? GetNodeInstance(SearchData) : this;
		if (NodeOb)
		{
			UBTService* ServiceNodeOb = (UBTService*)NodeOb;
			uint8* NodeMemory = GetNodeMemory<uint8>(SearchData);

			if (bNotifyTick)
			{
				const float RemainingTime = bRestartTimerOnEachActivation ? 0.0f : GetNextTickRemainingTime(NodeMemory);
				if (RemainingTime <= 0.0f)
				{
					ServiceNodeOb->ScheduleNextTick(SearchData.OwnerComp, NodeMemory);
				}
			}

			if (bNotifyOnSearch)
			{
				UE_VLOG(SearchData.OwnerComp.GetOwner(), LogBehaviorTree, VeryVerbose, TEXT("OnSearchStart: %s"), *UBehaviorTreeTypes::DescribeNodeHelper(ServiceNodeOb));
				ServiceNodeOb->OnSearchStart(SearchData);
			}

			if (bCallTickOnSearchStart)
			{
				UE_VLOG(SearchData.OwnerComp.GetOwner(), LogBehaviorTree, VeryVerbose, TEXT("TickNode (bCallTickOnSearchStart): %s"), *UBehaviorTreeTypes::DescribeNodeHelper(ServiceNodeOb));
				ServiceNodeOb->TickNode(SearchData.OwnerComp, NodeMemory, 0.0f);
			}
		}
	}
}

FString UBTService::GetStaticTickIntervalDescription() const
{
	if (bNotifyTick)
	{
		FString IntervalDesc("frame");
		if (bTickIntervals)
		{
			IntervalDesc = (RandomDeviation > 0.0f)
				? FString::Printf(TEXT("%.2fs..%.2fs"), FMath::Max(0.0f, Interval - RandomDeviation), (Interval + RandomDeviation))
				: FString::Printf(TEXT("%.2fs"), Interval);
		} 

		return FString::Printf(TEXT("tick every %s"), *IntervalDesc);
	}

	return TEXT("never ticks");
}

FString UBTService::GetStaticServiceDescription() const
{
	return GetStaticTickIntervalDescription();
}

FString UBTService::GetStaticDescription() const
{
	return FString::Printf(TEXT("%s: %s"), *UBehaviorTreeTypes::GetShortTypeName(this), *GetStaticServiceDescription());
}

#if WITH_EDITOR

FName UBTService::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Service.Icon");
}

#endif // WITH_EDITOR

void UBTService::ScheduleNextTick(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	const float NextTickTime = FMath::FRandRange(FMath::Max(0.0f, Interval - RandomDeviation), (Interval + RandomDeviation));
	SetNextTickTime(NodeMemory, NextTickTime);
}

