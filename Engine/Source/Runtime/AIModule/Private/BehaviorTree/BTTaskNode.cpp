// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/BTTaskNode.h"
#include "AIController.h"
#include "VisualLogger/VisualLogger.h"
#include "GameplayTasksComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTTaskNode)

UBTTaskNode::UBTTaskNode(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	bNotifyTick = false;
	bNotifyTaskFinished = false;
	bIgnoreRestartSelf = false;
	bTickIntervals = false;
}

EBTNodeResult::Type UBTTaskNode::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	return EBTNodeResult::Succeeded;
}

EBTNodeResult::Type UBTTaskNode::AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	return EBTNodeResult::Aborted;
}

void UBTTaskNode::SetNextTickTime(uint8* NodeMemory, float RemainingTime) const
{
	if (bTickIntervals)
	{
		FBTTaskMemory* TaskMemory = GetSpecialNodeMemory<FBTTaskMemory>(NodeMemory);
		TaskMemory->NextTickRemainingTime = RemainingTime;
	}
}

EBTNodeResult::Type UBTTaskNode::WrappedExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	EBTNodeResult::Type Result = EBTNodeResult::Failed;

	if (const UBTNode* NodeOb = bCreateNodeInstance ? GetNodeInstance(OwnerComp, NodeMemory) : this)
	{
		Result = ((UBTTaskNode*)NodeOb)->ExecuteTask(OwnerComp, NodeMemory);

		// Now that the task was executed we need to adjust times when using tick intervals
		// since `WrappedTickTask` uses `DeltaSeconds` which is accumulated time + current frame DT.
		// We don't want to pass previously accumulated time to the new task to execute.
		if (bTickIntervals)
		{
			const float AccumulatedDeltaTime = OwnerComp.GetAccumulatedTickDeltaTime();
			FBTTaskMemory* TaskMemory = GetSpecialNodeMemory<FBTTaskMemory>(NodeMemory);

			// Add accumulated time to `NextTickRemainingTime` set by the task to compensate for the next decrement in `WrappedTickTask`
			TaskMemory->NextTickRemainingTime += AccumulatedDeltaTime;

			// Assign task accumulated DT to negative current accumulated time/ for the next increment in `WrappedTickTask`
			TaskMemory->AccumulatedDeltaTime = -AccumulatedDeltaTime;
		}
	}

	return Result;
}

EBTNodeResult::Type UBTTaskNode::WrappedAbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	UBTNode* NodeOb = const_cast<UBTNode*>(bCreateNodeInstance ? GetNodeInstance(OwnerComp, NodeMemory) : this);
	UBTTaskNode* TaskNodeOb = static_cast<UBTTaskNode*>(NodeOb);
	EBTNodeResult::Type Result = TaskNodeOb ? TaskNodeOb->AbortTask(OwnerComp, NodeMemory) : EBTNodeResult::Aborted;

	return Result;
}

bool UBTTaskNode::WrappedTickTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds, float& NextNeededDeltaTime) const
{
	if (bNotifyTick)
	{
		const UBTTaskNode* NodeOb = bCreateNodeInstance ? static_cast<UBTTaskNode*>(GetNodeInstance(OwnerComp, NodeMemory)) : this;
		if (NodeOb)
		{
			if (NodeOb->bTickIntervals)
			{
				FBTTaskMemory* TaskMemory = GetSpecialNodeMemory<FBTTaskMemory>(NodeMemory);
				TaskMemory->NextTickRemainingTime -= DeltaSeconds;
				TaskMemory->AccumulatedDeltaTime += DeltaSeconds;

				const bool bTick = TaskMemory->NextTickRemainingTime <= 0.0f;
				if (bTick)
				{
				    const float UseDeltaTime = TaskMemory->AccumulatedDeltaTime;
				    TaskMemory->AccumulatedDeltaTime = 0.0f;
					TaskMemory->NextTickRemainingTime = 0.0f;
    
					UE_VLOG(OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT("Ticking aux node: %s"), *UBehaviorTreeTypes::DescribeNodeHelper(this));

					const_cast<UBTTaskNode*>(NodeOb)->TickTask(OwnerComp, NodeMemory, UseDeltaTime);
				}

				if (TaskMemory->NextTickRemainingTime < NextNeededDeltaTime)
				{
					NextNeededDeltaTime = TaskMemory->NextTickRemainingTime;
				}

				return bTick;
			}
			else
			{
				const_cast<UBTTaskNode*>(NodeOb)->TickTask(OwnerComp, NodeMemory, DeltaSeconds);
				NextNeededDeltaTime = 0.0f;
				return true;
			}
		}
	}
	return false;
}

void UBTTaskNode::WrappedOnTaskFinished(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTNodeResult::Type TaskResult) const
{
	UBTNode* NodeOb = const_cast<UBTNode*>(bCreateNodeInstance ? GetNodeInstance(OwnerComp, NodeMemory) : this);

	if (NodeOb)
	{
		UBTTaskNode* TaskNodeOb = static_cast<UBTTaskNode*>(NodeOb);
		if (TaskNodeOb->bNotifyTaskFinished)
		{
			TaskNodeOb->OnTaskFinished(OwnerComp, NodeMemory, TaskResult);
		}

		if (TaskNodeOb->bOwnsGameplayTasks && OwnerComp.GetAIOwner())
		{
			UGameplayTasksComponent* GTComp = OwnerComp.GetAIOwner()->GetGameplayTasksComponent();
			if (GTComp)
			{
				GTComp->EndAllResourceConsumingTasksOwnedBy(*TaskNodeOb);
			}
		}
	}
}

void UBTTaskNode::ReceivedMessage(UBrainComponent* BrainComp, const FAIMessage& Message)
{
	UBehaviorTreeComponent* OwnerComp = static_cast<UBehaviorTreeComponent*>(BrainComp);
	check(OwnerComp);
	
	const int32 InstanceIdx = OwnerComp->FindInstanceContainingNode(this);
	if (OwnerComp->InstanceStack.IsValidIndex(InstanceIdx))
	{
		uint8* NodeMemory = GetNodeMemory<uint8>(OwnerComp->InstanceStack[InstanceIdx]);
		OnMessage(*OwnerComp, NodeMemory, Message.MessageName, Message.RequestID, Message.Status == FAIMessage::Success);
	}
	else
	{
		UE_VLOG(OwnerComp->GetOwner(), LogBehaviorTree, Warning, TEXT("UBTTaskNode::ReceivedMessage called while %s node no longer in active BT")
			, *GetNodeName());
	}
}

void UBTTaskNode::TickTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds)
{
	// empty in base class
}

void UBTTaskNode::OnTaskFinished(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTNodeResult::Type TaskResult)
{
	// empty in base class
}

void UBTTaskNode::OnMessage(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, FName Message, int32 RequestID, bool bSuccess)
{
	const EBTTaskStatus::Type Status = OwnerComp.GetTaskStatus(this);
	if (Status == EBTTaskStatus::Active)
	{
		FinishLatentTask(OwnerComp, bSuccess ? EBTNodeResult::Succeeded : EBTNodeResult::Failed);
	}
	else if (Status == EBTTaskStatus::Aborting)
	{
		FinishLatentAbort(OwnerComp);
	}
}

void UBTTaskNode::FinishLatentTask(UBehaviorTreeComponent& OwnerComp, EBTNodeResult::Type TaskResult) const
{
	// OnTaskFinished must receive valid template node
	UBTTaskNode* TemplateNode = (UBTTaskNode*)OwnerComp.FindTemplateNode(this);
	OwnerComp.OnTaskFinished(TemplateNode, TaskResult);
}

void UBTTaskNode::FinishLatentAbort(UBehaviorTreeComponent& OwnerComp) const
{
	// OnTaskFinished must receive valid template node
	UBTTaskNode* TemplateNode = (UBTTaskNode*)OwnerComp.FindTemplateNode(this);
	OwnerComp.OnTaskFinished(TemplateNode, EBTNodeResult::Aborted);
}

void UBTTaskNode::WaitForMessage(UBehaviorTreeComponent& OwnerComp, FName MessageType) const
{
	// messages delegates should be called on node instances (if they exists)
	OwnerComp.RegisterMessageObserver(this, MessageType);
}

void UBTTaskNode::WaitForMessage(UBehaviorTreeComponent& OwnerComp, FName MessageType, int32 RequestID) const
{
	// messages delegates should be called on node instances (if they exists)
	OwnerComp.RegisterMessageObserver(this, MessageType, RequestID);
}
	
void UBTTaskNode::StopWaitingForMessages(UBehaviorTreeComponent& OwnerComp) const
{
	// messages delegates should be called on node instances (if they exists)
	OwnerComp.UnregisterMessageObserversFrom(this);
}

uint16 UBTTaskNode::GetSpecialMemorySize() const
{
	return bTickIntervals ? sizeof(FBTTaskMemory) : Super::GetSpecialMemorySize();
}

#if WITH_EDITOR

FName UBTTaskNode::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Task.Icon");
}

#endif	// WITH_EDITOR

void UBTTaskNode::OnGameplayTaskDeactivated(UGameplayTask& Task)
{
	ensure(Task.GetTaskOwner() == this);

	UBehaviorTreeComponent* BTComp = GetBTComponentForTask(Task);
	if (BTComp)
	{
		// this is a super-default behavior. Specific task will surely like to 
		// handle this themselves, finishing with specific result
		const EBTTaskStatus::Type Status = BTComp->GetTaskStatus(this);
		FinishLatentTask(*BTComp, Status == EBTTaskStatus::Aborting ? EBTNodeResult::Aborted : EBTNodeResult::Succeeded);
	}
}

//----------------------------------------------------------------------//
// DEPRECATED
//----------------------------------------------------------------------//

