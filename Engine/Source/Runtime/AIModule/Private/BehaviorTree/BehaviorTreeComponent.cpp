// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/BehaviorTreeComponent.h"
#include "BehaviorTree/BTTaskNode.h"
#include "BehaviorTree/BTDecorator.h"
#include "BehaviorTree/BTService.h"
#include "VisualLogger/VisualLoggerTypes.h"
#include "VisualLogger/VisualLogger.h"
#include "BehaviorTree/BTCompositeNode.h"
#include "BehaviorTreeDelegates.h"
#include "BehaviorTree/BehaviorTree.h"
#include "BehaviorTree/BehaviorTreeManager.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "BehaviorTree/Tasks/BTTask_RunBehaviorDynamic.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CoreDelegates.h"
#include "Misc/ConfigCacheIni.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BehaviorTreeComponent)


#if USE_BEHAVIORTREE_DEBUGGER
int32 UBehaviorTreeComponent::ActiveDebuggerCounter = 0;
#endif

// Code for timing BT Search
static TAutoConsoleVariable<int32> CVarBTRecordFrameSearchTimes(TEXT("BehaviorTree.RecordFrameSearchTimes"), 0, TEXT("Record Search Times Per Frame For Perf Stats"));
#if !UE_BUILD_SHIPPING
bool UBehaviorTreeComponent::bAddedEndFrameCallback = false;
double UBehaviorTreeComponent::FrameSearchTime = 0.;
int32 UBehaviorTreeComponent::NumSearchTimeCalls = 0;
#endif

namespace UE::BehaviorTree
{
	// Dedicated value used to stop ticking the tree
	constexpr float DisableTick = FLT_MAX;

	// Note this is defaulted to off for now as it caused a further bug, there is a BT unit test that will fire if this
	// code is re-enabled, once that is addressed this can be re-enabled by default.
	static bool bApplyAuxNodesFromFailedSearches = false;
	static FAutoConsoleVariableRef CVarApplyAuxNodesFromFailedSearches(
		TEXT("BehaviorTree.ApplyAuxNodesFromFailedSearches"),
		bApplyAuxNodesFromFailedSearches,
		TEXT("Apply Aux Nodes From Failed Searches"));

	// Enables the ensure in ScheduleExecutionUpdate to validate that either the pending execution or execution node is valid
	static bool bEnsureOnScheduleExecutionUpdateWithInvalidExecutionRequest = true;
	static FAutoConsoleVariableRef CVarEnsureOnScheduleExecutionUpdateWithInvalidExecutionRequest(
		TEXT("BehaviorTree.EnsureOnScheduleExecutionUpdateWithInvalidExecutionRequest"), 
		bEnsureOnScheduleExecutionUpdateWithInvalidExecutionRequest,
		TEXT("Should we ensure when pending execution and execution node are invalid when scheduling execution update."));
}

//----------------------------------------------------------------------//
// UBehaviorTreeComponent
//----------------------------------------------------------------------//

UBehaviorTreeComponent::UBehaviorTreeComponent(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer)
	, SearchData(*this)
{
	ActiveInstanceIdx = 0;
	bLoopExecution = false;
	bWaitingForLatentAborts = false;
	bRequestedFlowUpdate = false;
	bAutoActivate = true;
	bWantsInitializeComponent = true; 
	bIsRunning = false;
	bIsPaused = false;
	SuspendedBranchActions = EBTBranchAction::None;

	// Adding hook for bespoke framepro BT timings for BR
#if !UE_BUILD_SHIPPING
	if (!bAddedEndFrameCallback)
	{
		bAddedEndFrameCallback = true;
		FCoreDelegates::OnEndFrame.AddStatic(&UBehaviorTreeComponent::EndFrame);
	}
#endif
}

UBehaviorTreeComponent::UBehaviorTreeComponent(FVTableHelper& Helper)
	: Super(Helper)
	, SearchData(*this)
{

}

void UBehaviorTreeComponent::UninitializeComponent()
{
	UBehaviorTreeManager* BTManager = UBehaviorTreeManager::GetCurrent(GetWorld());
	if (BTManager)
	{
		BTManager->RemoveActiveComponent(*this);
	}

	if ((SuspendedBranchActions & EBTBranchAction::UninitializeComponent) != EBTBranchAction::None)
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("UninitializeComponent queued up"));
		PendingBranchActionRequests.Emplace(nullptr, EBTBranchAction::UninitializeComponent);
	}
	else
	{
		RemoveAllInstances();
	}
	Super::UninitializeComponent();
}

void UBehaviorTreeComponent::RegisterComponentTickFunctions(bool bRegister)
{
	if (bRegister)
	{
		ScheduleNextTick(0.0f);
	}
	Super::RegisterComponentTickFunctions(bRegister);
}

void UBehaviorTreeComponent::SetComponentTickEnabled(bool bEnabled)
{
	bool bWasEnabled = IsComponentTickEnabled();
	Super::SetComponentTickEnabled(bEnabled);

	// If enabling the component, this acts like a new component to tick in the TickTaskManager
	// So act like the component was never ticked
	if (!bWasEnabled && IsComponentTickEnabled())
	{
		bTickedOnce = false;
		ScheduleNextTick(0.0f);
	}
}

void UBehaviorTreeComponent::StartLogic()
{
	UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("%s"), ANSI_TO_TCHAR(__FUNCTION__));

	if (TreeHasBeenStarted())
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("%s: Skipping, logic already started."), ANSI_TO_TCHAR(__FUNCTION__));
		return;
	}

	if (TreeStartInfo.IsSet() == false)
	{
		TreeStartInfo.Asset = DefaultBehaviorTreeAsset;
	}

	if (TreeStartInfo.IsSet())
	{
		TreeStartInfo.bPendingInitialize = true;
		ProcessPendingInitialize();
	}
	else
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("%s: Could not find BehaviorTree asset to run."), ANSI_TO_TCHAR(__FUNCTION__));
	}
}

void UBehaviorTreeComponent::RestartLogic()
{
	UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("%s"), ANSI_TO_TCHAR(__FUNCTION__));
	RestartTree();
}

void UBehaviorTreeComponent::StopLogic(const FString& Reason) 
{
	UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("Stopping BT, reason: \'%s\'"), *Reason);
	StopTree(EBTStopMode::Safe);
}

void UBehaviorTreeComponent::PauseLogic(const FString& Reason)
{
	UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("Execution updates: PAUSED (%s)"), *Reason);
	bIsPaused = true;

	if (BlackboardComp)
	{
		BlackboardComp->PauseObserverNotifications();
	}

	// Store a debugger execution step so debugger is aware we've been paused.
	StoreDebuggerExecutionStep(EBTExecutionSnap::Regular);
}

EAILogicResuming::Type UBehaviorTreeComponent::ResumeLogic(const FString& Reason)
{
	UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("Execution updates: RESUMED (%s)"), *Reason);
	const EAILogicResuming::Type SuperResumeResult = Super::ResumeLogic(Reason);
	if (!!bIsPaused)
	{
		bIsPaused = false;
		ScheduleNextTick(0.0f);

		if (SuperResumeResult == EAILogicResuming::Continue)
		{
			if (BlackboardComp)
			{
				// Resume the blackboard's observer notifications and send any queued notifications
				BlackboardComp->ResumeObserverNotifications(true);
			}

			const bool bOutOfNodesPending = PendingExecution.IsSet() && PendingExecution.bOutOfNodes;
			if (ExecutionRequest.ExecuteNode || bOutOfNodesPending)
			{
				ScheduleExecutionUpdate();
			}

			return EAILogicResuming::Continue;
		}
		else if (SuperResumeResult == EAILogicResuming::RestartedInstead)
		{
			if (BlackboardComp)
			{
				// Resume the blackboard's observer notifications but do not send any queued notifications
				BlackboardComp->ResumeObserverNotifications(false);
			}
		}
	}

	return SuperResumeResult;
}

bool UBehaviorTreeComponent::TreeHasBeenStarted() const
{
	return bIsRunning && InstanceStack.Num();
}

bool UBehaviorTreeComponent::IsRunning() const
{ 
	return bIsPaused == false && TreeHasBeenStarted() == true;
}

bool UBehaviorTreeComponent::IsPaused() const
{
	return bIsPaused;
}

void UBehaviorTreeComponent::StartTree(UBehaviorTree& Asset, EBTExecutionMode::Type ExecuteMode /*= EBTExecutionMode::Looped*/)
{
	// clear instance stack, start should always run new tree from root
	UBehaviorTree* CurrentRoot = GetRootTree();
	
	if (CurrentRoot == &Asset && TreeHasBeenStarted())
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("Skipping behavior start request - it's already running"));
		return;
	}
	else if (CurrentRoot)
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("Abandoning behavior %s to start new one (%s)"),
			*GetNameSafe(CurrentRoot), *Asset.GetName());
	}

	StopTree(EBTStopMode::Safe);

	TreeStartInfo.Asset = &Asset;
	TreeStartInfo.ExecuteMode = ExecuteMode;
	TreeStartInfo.bPendingInitialize = true;

	ProcessPendingInitialize();
}

void UBehaviorTreeComponent::ProcessPendingInitialize()
{
	if ((SuspendedBranchActions & EBTBranchAction::ProcessPendingInitialize) != EBTBranchAction::None)
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("ProcessPendingInitialize(%s) queued up"), *GetNameSafe(TreeStartInfo.Asset));
		PendingBranchActionRequests.Emplace(nullptr, EBTBranchAction::ProcessPendingInitialize);
		return;
	}

	StopTree(EBTStopMode::Safe);
	if (bWaitingForLatentAborts)
	{
		return;
	}

	// finish cleanup
	RemoveAllInstances();

	bLoopExecution = (TreeStartInfo.ExecuteMode == EBTExecutionMode::Looped);
	bIsRunning = true;

#if USE_BEHAVIORTREE_DEBUGGER
	DebuggerSteps.Reset();
#endif

	UBehaviorTreeManager* BTManager = UBehaviorTreeManager::GetCurrent(GetWorld());
	if (BTManager)
	{
		BTManager->AddActiveComponent(*this);
	}

	// push new instance
	const bool bPushed = PushInstance(*TreeStartInfo.Asset);
	TreeStartInfo.bPendingInitialize = false;
}

void UBehaviorTreeComponent::StopTree(EBTStopMode::Type StopMode)
{
	SCOPE_CYCLE_COUNTER(STAT_AI_BehaviorTree_StopTree);

	const bool bForcedStop = StopMode == EBTStopMode::Forced;
	if ((SuspendedBranchActions & EBTBranchAction::StopTree) != EBTBranchAction::None)
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("Stop tree(%s) queued up"), bForcedStop ? TEXT("Forced") : TEXT("Safe"));
		PendingBranchActionRequests.Emplace(bForcedStop ? EBTBranchAction::StopTree_Forced : EBTBranchAction::StopTree_Safe);
		return;
	}

	UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("StopTree %s, mode:%s"), *GetNameSafe(GetRootTree()), bForcedStop ? TEXT("Forced") : TEXT("Safe"));
	FBTSuspendBranchActionsScoped ScopedSuspend(*this, EBTBranchAction::All);
	if (!bRequestedStop)
	{
		bRequestedStop = true;

		for (int32 InstanceIndex = InstanceStack.Num() - 1; InstanceIndex >= 0; InstanceIndex--)
		{
			FBehaviorTreeInstance& InstanceInfo = InstanceStack[InstanceIndex];

			// notify active aux nodes
			{
				InstanceInfo.ExecuteOnEachAuxNode([&InstanceInfo, this](const UBTAuxiliaryNode& AuxNode)
				{
					uint8* NodeMemory = AuxNode.GetNodeMemory<uint8>(InstanceInfo);
					AuxNode.WrappedOnCeaseRelevant(*this, NodeMemory);
				});
			}
			InstanceInfo.ResetActiveAuxNodes();

			// notify active parallel tasks
			//
			// calling OnTaskFinished with result other than InProgress will unregister parallel task,
			// modifying array we're iterating on - iterator needs to be moved one step back in that case
			//
			InstanceInfo.ExecuteOnEachParallelTask([&InstanceInfo, this](const FBehaviorTreeParallelTask& ParallelTaskInfo, const int32 ParallelIndex)
				{
					if (ParallelTaskInfo.Status != EBTTaskStatus::Active)
					{
						return;
					}

					const UBTTaskNode* CachedTaskNode = ParallelTaskInfo.TaskNode;
					if (IsValid(CachedTaskNode) == false)
					{
						return;
					}

					// remove all message observers added by task execution, so they won't interfere with Abort call
					UnregisterMessageObserversFrom(CachedTaskNode);

					uint8* NodeMemory = CachedTaskNode->GetNodeMemory<uint8>(InstanceInfo);
					EBTNodeResult::Type NodeResult = CachedTaskNode->WrappedAbortTask(*this, NodeMemory);

					UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("Parallel task aborted: %s (%s)"),
						*UBehaviorTreeTypes::DescribeNodeHelper(CachedTaskNode),
						(NodeResult == EBTNodeResult::InProgress) ? TEXT("in progress") : TEXT("instant"));

					// mark as pending abort
					if (NodeResult == EBTNodeResult::InProgress)
					{
						const bool bIsValidForStatus = InstanceInfo.IsValidParallelTaskIndex(ParallelIndex) && (ParallelTaskInfo.TaskNode == CachedTaskNode);
						if (bIsValidForStatus)
						{
							InstanceInfo.MarkParallelTaskAsAbortingAt(ParallelIndex);
							bWaitingForLatentAborts = true;
						}
						else
						{
							UE_VLOG(GetOwner(), LogBehaviorTree, Warning, TEXT("Parallel task %s was unregistered before completing Abort state!"),
								*UBehaviorTreeTypes::DescribeNodeHelper(CachedTaskNode));
						}
					}

					OnTaskFinished(CachedTaskNode, NodeResult);
				});

			// notify active task
			if (InstanceInfo.ActiveNodeType == EBTActiveNode::ActiveTask)
			{
				const UBTTaskNode* TaskNode = Cast<const UBTTaskNode>(InstanceInfo.ActiveNode);
				check(TaskNode != NULL);

				// remove all observers before requesting abort
				UnregisterMessageObserversFrom(TaskNode);
				InstanceInfo.ActiveNodeType = EBTActiveNode::AbortingTask;

				UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("Abort task: %s"), *UBehaviorTreeTypes::DescribeNodeHelper(TaskNode));

				// abort task using current state of tree
				uint8* NodeMemory = TaskNode->GetNodeMemory<uint8>(InstanceInfo);
				EBTNodeResult::Type TaskResult = TaskNode->WrappedAbortTask(*this, NodeMemory);

				// pass task finished if wasn't already notified (FinishLatentAbort)
				if (InstanceInfo.ActiveNodeType == EBTActiveNode::AbortingTask)
				{
					OnTaskFinished(TaskNode, TaskResult);
				}
			}
		}
	}

	if (bWaitingForLatentAborts)
	{
		if (!bForcedStop)
		{
			UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("StopTree is waiting for aborting tasks to finish..."));
			return;
		}

		UE_VLOG(GetOwner(), LogBehaviorTree, Warning, TEXT("StopTree was forced while waiting for tasks to finish aborting!"));
	}

	// make sure that all nodes are getting deactivation notifies
	if (InstanceStack.Num())
	{
		int32 DeactivatedChildIndex = INDEX_NONE;
		EBTNodeResult::Type AbortedResult = EBTNodeResult::Aborted;
		DeactivateUpTo(InstanceStack[0].RootNode, 0, AbortedResult, DeactivatedChildIndex);
	}

	// clear current state, don't touch debugger data
	for (int32 InstanceIndex = 0; InstanceIndex < InstanceStack.Num(); InstanceIndex++)
	{
		FBehaviorTreeInstance& InstanceInfo = InstanceStack[InstanceIndex];
		InstanceStack[InstanceIndex].Cleanup(*this, EBTMemoryClear::Destroy);
	}

	InstanceStack.Reset();
	TaskMessageObservers.Reset();
	SearchData.Reset();
	ExecutionRequest = FBTNodeExecutionInfo();
	PendingExecution = FBTPendingExecutionInfo();
	ActiveInstanceIdx = 0;

	// make sure to allow new execution requests
	bRequestedFlowUpdate = false;
	bRequestedStop = false;
	bIsRunning = false;
	bWaitingForLatentAborts = false;

	// make sure to not process scheduled ticks
	ScheduleNextTick(UE::BehaviorTree::DisableTick);
}

void UBehaviorTreeComponent::RestartTree(EBTRestartMode RestartMode /*= EBTRestartMode::SkipReAddedNodes*/)
{
	UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("%s"), ANSI_TO_TCHAR(__FUNCTION__));
	
	if (!bIsRunning)
	{
		if (TreeStartInfo.IsSet())
		{
			TreeStartInfo.bPendingInitialize = true;
			ProcessPendingInitialize();
		}
		else
		{
			UE_VLOG(GetOwner(), LogBehaviorTree, Warning, TEXT("\tFailed to restart tree logic since it has never been started and it\'s not possible to say which BT asset to use."));
		}
	}
	else if (bRequestedStop)
	{
		TreeStartInfo.bPendingInitialize = true;
	}
	else if (InstanceStack.Num())
	{
		switch(RestartMode)
		{
			case EBTRestartMode::ForceReevaluateRootNode:
			{
				FBehaviorTreeInstance& TopInstance = InstanceStack[0];
				RequestExecution(TopInstance.RootNode, 0, TopInstance.RootNode, -1, EBTNodeResult::Aborted);
				break;
			}
			case EBTRestartMode::CompleteRestart:
			{
				TreeStartInfo.Asset = GetRootTree();
				TreeStartInfo.ExecuteMode = bLoopExecution ? EBTExecutionMode::Looped : EBTExecutionMode::SingleRun;
				TreeStartInfo.bPendingInitialize = true;
				ProcessPendingInitialize();

				StopTree(EBTStopMode::Safe);
				break;
			}
		}
	}
}

void UBehaviorTreeComponent::Cleanup()
{
	SCOPE_CYCLE_COUNTER(STAT_AI_BehaviorTree_Cleanup);

	if ((SuspendedBranchActions & EBTBranchAction::Cleanup) != EBTBranchAction::None)
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("Cleanup queued up"));
		PendingBranchActionRequests.Emplace(nullptr, EBTBranchAction::Cleanup);
		return;
	}

	StopTree(EBTStopMode::Forced);
	RemoveAllInstances();

	KnownInstances.Reset();
	InstanceStack.Reset();
	NodeInstances.Reset();
}

void UBehaviorTreeComponent::HandleMessage(const FAIMessage& Message)
{
	Super::HandleMessage(Message);
	ScheduleNextTick(0.0f);
}

void UBehaviorTreeComponent::OnTaskFinished(const UBTTaskNode* TaskNode, EBTNodeResult::Type TaskResult)
{
	if (TaskNode == NULL || InstanceStack.Num() == 0 || !IsValidChecked(this))
	{
		return;
	}

	UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("Task %s finished: %s"), 
		*UBehaviorTreeTypes::DescribeNodeHelper(TaskNode), *UBehaviorTreeTypes::DescribeNodeResult(TaskResult));

	// notify parent node
	UBTCompositeNode* ParentNode = TaskNode->GetParentNode();
	const int32 TaskInstanceIdx = FindInstanceContainingNode(TaskNode);
	if (!InstanceStack.IsValidIndex(TaskInstanceIdx))
	{
		return;
	}

	uint8* ParentMemory = ParentNode->GetNodeMemory<uint8>(InstanceStack[TaskInstanceIdx]);

	ParentNode->ConditionalNotifyChildExecution(*this, ParentMemory, *TaskNode, TaskResult);

	if (TaskResult != EBTNodeResult::InProgress)
	{
		StoreDebuggerSearchStep(TaskNode, IntCastChecked<uint16>(TaskInstanceIdx), TaskResult);

		// cleanup task observers
		UnregisterMessageObserversFrom(TaskNode);

		// notify task about it
		uint8* TaskMemory = TaskNode->GetNodeMemory<uint8>(InstanceStack[TaskInstanceIdx]);
		TaskNode->WrappedOnTaskFinished(*this, TaskMemory, TaskResult);

		// update execution when active task is finished
		if (InstanceStack.IsValidIndex(ActiveInstanceIdx) && InstanceStack[ActiveInstanceIdx].ActiveNode == TaskNode)
		{
			FBehaviorTreeInstance& ActiveInstance = InstanceStack[ActiveInstanceIdx];
			const bool bWasAborting = (ActiveInstance.ActiveNodeType == EBTActiveNode::AbortingTask);
			ActiveInstance.ActiveNodeType = EBTActiveNode::InactiveTask;

			// request execution from parent
			if (!bWasAborting)
			{
				RequestExecution(TaskResult);
			}
			// schedule an update to process a pending execution when task is done aborting
			else if (PendingExecution.IsSet())
			{
				ScheduleExecutionUpdate();
			}
		}
		else if (TaskResult == EBTNodeResult::Aborted && InstanceStack.IsValidIndex(TaskInstanceIdx) && InstanceStack[TaskInstanceIdx].ActiveNode == TaskNode)
		{
			// active instance may be already changed when getting back from AbortCurrentTask 
			// (e.g. new task is higher on stack)

			InstanceStack[TaskInstanceIdx].ActiveNodeType = EBTActiveNode::InactiveTask;
		}
	}

	TrackNewLatentAborts();

	if (TreeStartInfo.HasPendingInitialize())
	{
		ProcessPendingInitialize();
	}
}

void UBehaviorTreeComponent::OnTreeFinished()
{
	UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("Ran out of nodes to check, %s tree."),
		bLoopExecution ? TEXT("looping") : TEXT("stopping"));

	ActiveInstanceIdx = 0;
	StoreDebuggerExecutionStep(EBTExecutionSnap::OutOfNodes);

	if (bLoopExecution && InstanceStack.Num())
	{
		// it should be already deactivated (including root)
		// set active node to initial state: root activation
		FBehaviorTreeInstance& TopInstance = InstanceStack[0];
		TopInstance.ActiveNode = NULL;
		TopInstance.ActiveNodeType = EBTActiveNode::Composite;

		// make sure that all active aux nodes will be removed
		// root level services are being handled on applying search data
		UnregisterAuxNodesUpTo(FBTNodeIndex(0, 0));

		// result doesn't really matter, root node will be reset and start iterating child nodes from scratch
		// although it shouldn't be set to Aborted, as it has special meaning in RequestExecution (switch to higher priority)
		RequestExecution(TopInstance.RootNode, 0, TopInstance.RootNode, 0, EBTNodeResult::InProgress);
	}
	else
	{
		StopTree(EBTStopMode::Safe);
	}
}

bool UBehaviorTreeComponent::IsExecutingBranch(const UBTNode* Node, int32 ChildIndex) const
{
	const int32 TestInstanceIdx = FindInstanceContainingNode(Node);
	if (!InstanceStack.IsValidIndex(TestInstanceIdx) || InstanceStack[TestInstanceIdx].ActiveNode == NULL)
	{
		return false;
	}

	// is it active node or root of tree?
	const FBehaviorTreeInstance& TestInstance = InstanceStack[TestInstanceIdx];
	if (Node == TestInstance.RootNode || Node == TestInstance.ActiveNode)
	{
		return true;
	}

	// compare with index of next child
	const uint16 ActiveExecutionIndex = TestInstance.ActiveNode->GetExecutionIndex();
	const uint16 NextChildExecutionIndex = Node->GetParentNode()->GetChildExecutionIndex(ChildIndex + 1);
	return (ActiveExecutionIndex >= Node->GetExecutionIndex()) && (ActiveExecutionIndex < NextChildExecutionIndex);
}

bool UBehaviorTreeComponent::IsAuxNodeActive(const UBTAuxiliaryNode* AuxNode) const
{
	if (AuxNode == NULL)
	{
		return false;
	}

	const uint16 AuxExecutionIndex = AuxNode->GetExecutionIndex();
	for (int32 InstanceIndex = 0; InstanceIndex < InstanceStack.Num(); InstanceIndex++)
	{
		const FBehaviorTreeInstance& InstanceInfo = InstanceStack[InstanceIndex];
		for (const UBTAuxiliaryNode* TestAuxNode : InstanceInfo.GetActiveAuxNodes())
		{
			// check template version
			if (TestAuxNode == AuxNode)
			{
				return true;
			}

			// check instanced version
			CA_SUPPRESS(6011);
			if (AuxNode->IsInstanced() && TestAuxNode && TestAuxNode->GetExecutionIndex() == AuxExecutionIndex)
			{
				const uint8* NodeMemory = TestAuxNode->GetNodeMemory<uint8>(InstanceInfo);
				UBTNode* NodeInstance = TestAuxNode->GetNodeInstance(*this, (uint8*)NodeMemory);

				if (NodeInstance == AuxNode)
				{
					return true;
				}
			}
		}
	}

	return false;
}

bool UBehaviorTreeComponent::IsAuxNodeActive(const UBTAuxiliaryNode* AuxNodeTemplate, int32 InstanceIdx) const
{
	return InstanceStack.IsValidIndex(InstanceIdx) && InstanceStack[InstanceIdx].GetActiveAuxNodes().Contains(AuxNodeTemplate);
}

EBTTaskStatus::Type UBehaviorTreeComponent::GetTaskStatus(const UBTTaskNode* TaskNode) const
{
	EBTTaskStatus::Type Status = EBTTaskStatus::Inactive;
	const int32 InstanceIdx = FindInstanceContainingNode(TaskNode);

	if (InstanceStack.IsValidIndex(InstanceIdx))
	{
		const uint16 ExecutionIndex = TaskNode->GetExecutionIndex();
		const FBehaviorTreeInstance& InstanceInfo = InstanceStack[InstanceIdx];

		// always check parallel execution first, it takes priority over ActiveNodeType
		for (const FBehaviorTreeParallelTask& ParallelInfo : InstanceInfo.GetParallelTasks())
		{
			if (ParallelInfo.TaskNode == TaskNode ||
				(TaskNode->IsInstanced() && ParallelInfo.TaskNode && ParallelInfo.TaskNode->GetExecutionIndex() == ExecutionIndex))
			{
				Status = ParallelInfo.Status;
				break;
			}
		}

		if (Status == EBTTaskStatus::Inactive)
		{
			if (InstanceInfo.ActiveNode == TaskNode ||
				(TaskNode->IsInstanced() && InstanceInfo.ActiveNode && InstanceInfo.ActiveNode->GetExecutionIndex() == ExecutionIndex))
			{
				Status =
					(InstanceInfo.ActiveNodeType == EBTActiveNode::ActiveTask) ? EBTTaskStatus::Active :
					(InstanceInfo.ActiveNodeType == EBTActiveNode::AbortingTask) ? EBTTaskStatus::Aborting :
					EBTTaskStatus::Inactive;
			}
		}
	}

	return Status;
}

void UBehaviorTreeComponent::RequestUnregisterAuxNodesInBranch(const UBTCompositeNode* Node)
{
	if (!Node)
	{
		return;
	}

	if ((SuspendedBranchActions & EBTBranchAction::UnregisterAuxNodes) != EBTBranchAction::None)
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("%s, unregister aux nodes in branch queued up"), *UBehaviorTreeTypes::DescribeNodeHelper(Node));
		PendingBranchActionRequests.Emplace(Node, EBTBranchAction::UnregisterAuxNodes);
		return;
	}

	UnregisterAuxNodesInBranch(Node, true/*bApplyImmediately*/);
}

void UBehaviorTreeComponent::DeactivateBranch(const UBTDecorator& RequestedBy)
{
	if (IsExecutingBranch(&RequestedBy, RequestedBy.GetChildIndex()))
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("%s, Branch deactivation resulted in an evaluate branch"), *UBehaviorTreeTypes::DescribeNodeHelper(&RequestedBy));
		EvaluateBranch(RequestedBy);
	}
	else if (ensureMsgf(RequestedBy.GetParentNode() && RequestedBy.GetParentNode()->Children.IsValidIndex(RequestedBy.GetChildIndex()), 
				TEXT("The decorator %s does not have a parent or is not a valid child."), *UBehaviorTreeTypes::DescribeNodeHelper(&RequestedBy)))
	{
		const bool bAbortPending = IsAbortPending();
		if (bAbortPending)
		{
			// Branch that caused an abort and still waiting on a latent abort to complete might see its decorators changed again causing a call to DeactivateBranch.
			// In this particular case we explicitly request the parent composite node to reevaluate without specifying a child index (i.e. RequestedByChildIndex = -1) so it will reevaluate all children.
			UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("%s, Branch deactivation resulted in a reevaluation of the parent composite node because the abort was pending"), *UBehaviorTreeTypes::DescribeNodeHelper(&RequestedBy));
			const int32 InstanceIdx = FindInstanceContainingNode(&RequestedBy);
			RequestExecution(RequestedBy.GetParentNode(), InstanceIdx, &RequestedBy, -1, EBTNodeResult::Aborted);
		}

		UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("%s, Branch deactivation resulted in aux nodes unregistration"), *UBehaviorTreeTypes::DescribeNodeHelper(&RequestedBy));
		if (const UBTCompositeNode* BranchRoot = RequestedBy.GetParentNode()->Children[RequestedBy.GetChildIndex()].ChildComposite)
		{
			if ((SuspendedBranchActions & EBTBranchAction::UnregisterAuxNodes) != EBTBranchAction::None)
			{
				UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("%s, unregister aux nodes in branch queued up"), *UBehaviorTreeTypes::DescribeNodeHelper(&RequestedBy));
				PendingBranchActionRequests.Emplace(BranchRoot, EBTBranchAction::UnregisterAuxNodes);
			}
			else
			{
				UnregisterAuxNodesInBranch(BranchRoot, true/*bApplyImmediately*/);
			}
		}
	}
	else
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Error, TEXT("The decorator %s does not have a parent or is not a valid child."), *UBehaviorTreeTypes::DescribeNodeHelper(&RequestedBy));
	}
}

void UBehaviorTreeComponent::RequestBranchDeactivation(const UBTDecorator& RequestedBy)
{
	if ((SuspendedBranchActions & EBTBranchAction::DecoratorDeactivate) != EBTBranchAction::None)
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("%s, Branch deactivation queued up"), *UBehaviorTreeTypes::DescribeNodeHelper(&RequestedBy));
		PendingBranchActionRequests.Emplace(&RequestedBy, EBTBranchAction::DecoratorDeactivate);
		return;
	}

	DeactivateBranch(RequestedBy);
}

void UBehaviorTreeComponent::SuspendBranchActions(EBTBranchAction BranchActions)
{
	UE_VLOG(GetOwner(), LogBehaviorTree, VeryVerbose, TEXT("Suspending branch actions."));
	checkf(SuspendedBranchActions == EBTBranchAction::None, TEXT("This logic does not support re-entrance"));
	SuspendedBranchActions = BranchActions;
}

void UBehaviorTreeComponent::ResumeBranchActions()
{
	UE_VLOG(GetOwner(), LogBehaviorTree, VeryVerbose, TEXT("Resuming branch actions."));
	checkf(SuspendedBranchActions != EBTBranchAction::None, TEXT("Expecting SuspendBranchActions() be called before calling resume"));
	SuspendedBranchActions = EBTBranchAction::None;

	// Flushing any pending branch actions
	while (PendingBranchActionRequests.Num() > 0)
	{
		TArray<FBranchActionInfo> PendingBranchActionRequestsToProcess(MoveTemp(PendingBranchActionRequests));
		PendingBranchActionRequests.Reset();
		for (const FBranchActionInfo& Info : PendingBranchActionRequestsToProcess)
		{
			switch (Info.Action)
			{
				case EBTBranchAction::DecoratorEvaluate:
				{
					const UBTDecorator* RequestedBy = CastChecked<UBTDecorator>(Info.Node);

					// Since we have been queued up, decorator might have been removed from active nodes, need to make sure it is still there.
					if (!IsAuxNodeActive(RequestedBy))
					{
						UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("Request deactivation skipped because decorator(%s) is not active anymore"), *UBehaviorTreeTypes::DescribeNodeHelper(RequestedBy));
						break;
					}

					EvaluateBranch(*RequestedBy);
					break;
				}

				case EBTBranchAction::DecoratorActivate_IfNotExecuting:
				case EBTBranchAction::DecoratorActivate_EvenIfExecuting:
				{
					const UBTDecorator* RequestedBy = CastChecked<UBTDecorator>(Info.Node);

					// Since we have been queued up, decorator might have been removed from active nodes, need to make sure it is still there.
					if (!IsAuxNodeActive(RequestedBy))
					{
						UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("Request deactivation skipped because decorator(%s) is not active anymore"), *UBehaviorTreeTypes::DescribeNodeHelper(RequestedBy));
						break;
					}

					ActivateBranch(*RequestedBy, Info.Action == EBTBranchAction::DecoratorActivate_EvenIfExecuting /*bForceRequestEvenIfExecuting*/);
					break;
				}
				case EBTBranchAction::DecoratorDeactivate:
				{
					const UBTDecorator* RequestedBy = CastChecked<UBTDecorator>(Info.Node);

					// Since we have been queued up, decorator might have been removed from active nodes, need to make sure it is still there.
					if (!IsAuxNodeActive(RequestedBy))
					{
						UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("Request deactivation skipped because decorator(%s) is not active anymore"), *UBehaviorTreeTypes::DescribeNodeHelper(RequestedBy));
						break;
					}

					DeactivateBranch(*RequestedBy);
					break;
				}
				case EBTBranchAction::UnregisterAuxNodes:
				{
					const UBTCompositeNode* BranchRoot = CastChecked<UBTCompositeNode>(Info.Node);
					UnregisterAuxNodesInBranch(BranchRoot, true/*bApplyImmediately*/);
					break;
				}
				case EBTBranchAction::StopTree_Safe:
				case EBTBranchAction::StopTree_Forced:
				{
					StopTree(Info.Action == EBTBranchAction::StopTree_Forced ? EBTStopMode::Forced : EBTStopMode::Safe);
					break;
				}
				case EBTBranchAction::ActiveNodeEvaluate:
				{
					const UBTNode* ActiveNode = GetActiveNode();
					if (ActiveNode != Info.Node)
					{
						UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("Request evaluation skipped because node(%s) is not active anymore"), *UBehaviorTreeTypes::DescribeNodeHelper(ActiveNode), *UEnum::GetValueAsString(Info.ContinueWithResult));
						break;
					}

					EvaluateBranch(Info.ContinueWithResult);
					break;
				}
				case EBTBranchAction::SubTreeEvaluate:
				{
					const UBTCompositeNode* BranchRoot = CastChecked<UBTCompositeNode>(Info.Node);

					const UBTNode* RootNode = InstanceStack.Num() ? InstanceStack[ActiveInstanceIdx].RootNode : nullptr;
					if (RootNode != BranchRoot)
					{
						UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("Sub tree evaluation skipped because node(%s) is not active anymore"), *UBehaviorTreeTypes::DescribeNodeHelper(BranchRoot));
						break;
					}

					RequestExecution(BranchRoot, ActiveInstanceIdx, BranchRoot, 0, EBTNodeResult::InProgress);
					break;
				}
				case EBTBranchAction::ProcessPendingInitialize:
				{
					ProcessPendingInitialize();
					break;
				}
				case EBTBranchAction::Cleanup:
				{
					Cleanup();
					break;
				}
				case EBTBranchAction::UninitializeComponent:
				{
					// We do not call UninitializeComponent here because only part of the method was queued up and we cannot delay it nor call it twice.
					// All other actions in the method were performed synchronously.
					RemoveAllInstances();
                    break;
				}
			}
		}
	}
}

void UBehaviorTreeComponent::RequestBranchEvaluation(const UBTDecorator& RequestedBy)
{
	if ((SuspendedBranchActions & EBTBranchAction::DecoratorEvaluate) != EBTBranchAction::None)
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("%s, Branch evaluation queued up"), *UBehaviorTreeTypes::DescribeNodeHelper(&RequestedBy));
		PendingBranchActionRequests.Emplace(&RequestedBy, EBTBranchAction::DecoratorEvaluate);
		return;
	}

	EvaluateBranch(RequestedBy);
}

void UBehaviorTreeComponent::EvaluateBranch(const UBTDecorator& RequestedBy)
{
	// search range depends on decorator's FlowAbortMode:
	//
	// - LowerPri: try entering branch = search only nodes under decorator
	//
	// - Self: leave execution = from node under decorator to end of tree
	//
	// - Both: check if active node is within inner child nodes and choose Self or LowerPri
	//

	EBTFlowAbortMode::Type AbortMode = RequestedBy.GetFlowAbortMode();
	if (AbortMode == EBTFlowAbortMode::None)
	{
		return;
	}

	const int32 InstanceIdx = FindInstanceContainingNode(RequestedBy.GetParentNode());
	if (InstanceIdx == INDEX_NONE)
	{
		return;
	}

#if ENABLE_VISUAL_LOG || DO_ENSURE
	const FBehaviorTreeInstance& ActiveInstance = InstanceStack.Last();
	if (ActiveInstance.ActiveNodeType == EBTActiveNode::ActiveTask)
	{
		EBTNodeRelativePriority RelativePriority = CalculateRelativePriority(&RequestedBy, ActiveInstance.ActiveNode);

		if (RelativePriority < EBTNodeRelativePriority::Same)
		{
			const FString ErrorMsg(FString::Printf(TEXT("%s: decorator %s requesting restart has lower priority than Current Task %s"),
				ANSI_TO_TCHAR(__FUNCTION__),
				*UBehaviorTreeTypes::DescribeNodeHelper(&RequestedBy),
				*UBehaviorTreeTypes::DescribeNodeHelper(ActiveInstance.ActiveNode)));

			UE_VLOG(GetOwner(), LogBehaviorTree, Error, TEXT("%s"), *ErrorMsg);
			ensureMsgf(false, TEXT("%s"), *ErrorMsg);
		}
	}
#endif // ENABLE_VISUAL_LOG || DO_ENSURE

	if (AbortMode == EBTFlowAbortMode::Both)
	{
		const bool bIsExecutingChildNodes = IsExecutingBranch(&RequestedBy, RequestedBy.GetChildIndex());
		AbortMode = bIsExecutingChildNodes ? EBTFlowAbortMode::Self : EBTFlowAbortMode::LowerPriority;
	}

	UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("%s, EvaluateBranch requested a new execution"), *UBehaviorTreeTypes::DescribeNodeHelper(&RequestedBy));

	EBTNodeResult::Type ContinueResult = (AbortMode == EBTFlowAbortMode::Self) ? EBTNodeResult::Failed : EBTNodeResult::Aborted;
	RequestExecution(RequestedBy.GetParentNode(), InstanceIdx, &RequestedBy, RequestedBy.GetChildIndex(), ContinueResult);
}


void UBehaviorTreeComponent::RequestBranchActivation(const UBTDecorator& RequestedBy, const bool bRequestEvenIfExecuting)
{
	if ((SuspendedBranchActions & EBTBranchAction::DecoratorActivate) != EBTBranchAction::None)
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("%s, Branch activation queued up"), *UBehaviorTreeTypes::DescribeNodeHelper(&RequestedBy));
		PendingBranchActionRequests.Emplace(&RequestedBy, bRequestEvenIfExecuting ? EBTBranchAction::DecoratorActivate_EvenIfExecuting : EBTBranchAction::DecoratorActivate_IfNotExecuting);
		return;
	}

	ActivateBranch(RequestedBy, bRequestEvenIfExecuting);
}


void UBehaviorTreeComponent::ActivateBranch(const UBTDecorator& RequestedBy, const bool bForceResquestEvenIfExecuting)
{
	const int32 InstanceIdx = FindInstanceContainingNode(&RequestedBy);
	if (InstanceIdx == INDEX_NONE)
	{
		return;
	}

	const bool bIsExecutingBranch = IsExecutingBranch(&RequestedBy, RequestedBy.GetChildIndex());
	const bool bAbortPending = IsAbortPending();

	checkf(InstanceIdx != FBTNodeIndex::InvalidIndex, TEXT("Index has used the InvalidIndex value!"));
	const bool bIsDeactivatingBranchRoot = ExecutionRequest.ContinueWithResult == EBTNodeResult::Failed && ExecutionRequest.SearchStart == FBTNodeIndex(IntCastChecked<uint16>(InstanceIdx), RequestedBy.GetExecutionIndex());

	const bool bLogRequestExecution = !bIsExecutingBranch || (bForceResquestEvenIfExecuting || bAbortPending || bIsDeactivatingBranchRoot);
	UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("%s, ActivateBranch(%s) executingBranch:%d abortPending:%d deactivatingBranchRoot:%d => %s"),
		*UBehaviorTreeTypes::DescribeNodeHelper(&RequestedBy),
		bForceResquestEvenIfExecuting ? TEXT("request even if executing") : TEXT("request if not executing"),
		bIsExecutingBranch ? 1 : 0,
		bAbortPending ? 1 : 0,
		bIsDeactivatingBranchRoot ? 1 : 0,
		bLogRequestExecution ? TEXT("request execution") : TEXT("skip"));

	if (!bIsExecutingBranch)
	{
		// This should endup into a branch activation
		EvaluateBranch(RequestedBy);
	}
	else if (bForceResquestEvenIfExecuting || bAbortPending || bIsDeactivatingBranchRoot)
	{
		// force result Aborted to restart from this branch
		RequestExecution(RequestedBy.GetParentNode(), InstanceIdx, &RequestedBy, RequestedBy.GetChildIndex(), EBTNodeResult::Aborted);
	}
}

EBTNodeRelativePriority UBehaviorTreeComponent::CalculateRelativePriority(const UBTNode* NodeA, const UBTNode* NodeB) const
{
	EBTNodeRelativePriority RelativePriority = EBTNodeRelativePriority::Same;

	if (NodeA != NodeB)
	{
		const int32 InstanceIndexA = FindInstanceContainingNode(NodeA);
		const int32 InstanceIndexB = FindInstanceContainingNode(NodeB);
		if (InstanceIndexA == InstanceIndexB)
		{
			RelativePriority = NodeA->GetExecutionIndex() < NodeB->GetExecutionIndex() ? EBTNodeRelativePriority::Higher : EBTNodeRelativePriority::Lower;
		}
		else
		{
			RelativePriority = (InstanceIndexA != INDEX_NONE && InstanceIndexB != INDEX_NONE) ? (InstanceIndexA < InstanceIndexB ? EBTNodeRelativePriority::Higher : EBTNodeRelativePriority::Lower)
				: (InstanceIndexA != INDEX_NONE ? EBTNodeRelativePriority::Higher : EBTNodeRelativePriority::Lower);
		}
	}

	return RelativePriority;
}

void UBehaviorTreeComponent::RequestBranchEvaluation(EBTNodeResult::Type ContinueWithResult)
{
	// task helpers can't continue with InProgress or Aborted result, it should be handled 
	// either by decorator helper or regular RequestExecution() (6 param version)
	if (ContinueWithResult == EBTNodeResult::Aborted || ContinueWithResult == EBTNodeResult::InProgress)
	{
		return;
	}

	if ((SuspendedBranchActions & EBTBranchAction::ActiveNodeEvaluate) != EBTBranchAction::None)
	{
		const UBTNode* ActiveNode = GetActiveNode();
		UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("Request evaluation queued up for node(%s) with result(%s)"), *UBehaviorTreeTypes::DescribeNodeHelper(ActiveNode), *UEnum::GetValueAsString(ContinueWithResult));
		PendingBranchActionRequests.Emplace(ActiveNode, ContinueWithResult, EBTBranchAction::ActiveNodeEvaluate);
		return;
	}

	EvaluateBranch(ContinueWithResult);
}

void UBehaviorTreeComponent::EvaluateBranch(EBTNodeResult::Type ContinueWithResult)
{
	if (InstanceStack.IsValidIndex(ActiveInstanceIdx))
	{
		const FBehaviorTreeInstance& ActiveInstance = InstanceStack[ActiveInstanceIdx];
		UBTCompositeNode* ExecuteParent = (ActiveInstance.ActiveNode == NULL) ? ActiveInstance.RootNode.Get() :
			(ActiveInstance.ActiveNodeType == EBTActiveNode::Composite) ? (UBTCompositeNode*)ActiveInstance.ActiveNode.Get() :
			ActiveInstance.ActiveNode->GetParentNode();

		RequestExecution(ExecuteParent, InstanceStack.Num() - 1,
			ActiveInstance.ActiveNode ? ActiveInstance.ActiveNode : ActiveInstance.RootNode, -1,
			ContinueWithResult, false);
	}
}

static void FindCommonParent(const TArray<FBehaviorTreeInstance>& Instances, const TArray<FBehaviorTreeInstanceId>& KnownInstances,
							 const UBTCompositeNode* InNodeA, uint16 InstanceIdxA,
							 const UBTCompositeNode* InNodeB, uint16 InstanceIdxB,
							 const UBTCompositeNode*& CommonParentNode, uint16& CommonInstanceIdx)
{
	// find two nodes in the same instance (choose lower index = closer to root)
	CommonInstanceIdx = (InstanceIdxA <= InstanceIdxB) ? InstanceIdxA : InstanceIdxB;

	const UBTCompositeNode* NodeA = (CommonInstanceIdx == InstanceIdxA) ? InNodeA : Instances[CommonInstanceIdx].ActiveNode->GetParentNode();
	const UBTCompositeNode* NodeB = (CommonInstanceIdx == InstanceIdxB) ? InNodeB : Instances[CommonInstanceIdx].ActiveNode->GetParentNode();

	// special case: node was taken from CommonInstanceIdx, but it had ActiveNode set to root (no parent)
	if (!NodeA && CommonInstanceIdx != InstanceIdxA)
	{
		NodeA = Instances[CommonInstanceIdx].RootNode;
	}
	if (!NodeB && CommonInstanceIdx != InstanceIdxB)
	{
		NodeB = Instances[CommonInstanceIdx].RootNode;
	}

	// if one of nodes is still empty, we have serious problem with execution flow - crash and log details
	if (!NodeA || !NodeB)
	{
		FString AssetAName = Instances.IsValidIndex(InstanceIdxA) && KnownInstances.IsValidIndex(Instances[InstanceIdxA].InstanceIdIndex) ? GetNameSafe(KnownInstances[Instances[InstanceIdxA].InstanceIdIndex].TreeAsset) : TEXT("unknown");
		FString AssetBName = Instances.IsValidIndex(InstanceIdxB) && KnownInstances.IsValidIndex(Instances[InstanceIdxB].InstanceIdIndex) ? GetNameSafe(KnownInstances[Instances[InstanceIdxB].InstanceIdIndex].TreeAsset) : TEXT("unknown");
		FString AssetCName = Instances.IsValidIndex(CommonInstanceIdx) && KnownInstances.IsValidIndex(Instances[CommonInstanceIdx].InstanceIdIndex) ? GetNameSafe(KnownInstances[Instances[CommonInstanceIdx].InstanceIdIndex].TreeAsset) : TEXT("unknown");

		UE_LOGF(LogBehaviorTree, Fatal, "Fatal error in FindCommonParent() call.\nInNodeA: %ls, InstanceIdxA: %d (%ls), NodeA: %ls\nInNodeB: %ls, InstanceIdxB: %d (%ls), NodeB: %ls\nCommonInstanceIdx: %d (%ls), ActiveNode: %ls%ls",
			*UBehaviorTreeTypes::DescribeNodeHelper(InNodeA), InstanceIdxA, *AssetAName, *UBehaviorTreeTypes::DescribeNodeHelper(NodeA),
			*UBehaviorTreeTypes::DescribeNodeHelper(InNodeB), InstanceIdxB, *AssetBName, *UBehaviorTreeTypes::DescribeNodeHelper(NodeB),
			CommonInstanceIdx, *AssetCName, *UBehaviorTreeTypes::DescribeNodeHelper(Instances[CommonInstanceIdx].ActiveNode),
			(Instances[CommonInstanceIdx].ActiveNode == Instances[CommonInstanceIdx].RootNode) ? TEXT(" (root)") : TEXT(""));

		return;
	}

	// find common parent of two nodes
	int32 NodeADepth = NodeA->GetTreeDepth();
	int32 NodeBDepth = NodeB->GetTreeDepth();

	while (NodeADepth > NodeBDepth)
	{
		NodeA = NodeA->GetParentNode();
		NodeADepth = NodeA->GetTreeDepth();
	}

	while (NodeBDepth > NodeADepth)
	{
		NodeB = NodeB->GetParentNode();
		NodeBDepth = NodeB->GetTreeDepth();
	}

	while (NodeA != NodeB)
	{
		NodeA = NodeA->GetParentNode();
		NodeB = NodeB->GetParentNode();
	}

	CommonParentNode = NodeA;
}

void UBehaviorTreeComponent::ScheduleExecutionUpdate()
{
	ScheduleNextTick(0.0f);

	ensureMsgf(!UE::BehaviorTree::bEnsureOnScheduleExecutionUpdateWithInvalidExecutionRequest || PendingExecution.IsSet() || ExecutionRequest.ExecuteNode, TEXT("Expecting either a pending execution or an execution node in the request."));
	bRequestedFlowUpdate = true;
}

void UBehaviorTreeComponent::RequestExecution(const UBTCompositeNode* RequestedOn, const int32 InstanceIdx, const UBTNode* RequestedBy,
											  const int32 RequestedByChildIndex, const EBTNodeResult::Type ContinueWithResult, const bool bStoreForDebugger)
{
	SCOPE_CYCLE_COUNTER(STAT_AI_BehaviorTree_SearchTime);
#if !UE_BUILD_SHIPPING // Disable in shipping builds
	// Code for timing BT Search
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(BehaviorTreeSearch);

	FScopedSwitchedCountedDurationTimer ScopedSwitchedCountedDurationTimer(FrameSearchTime, NumSearchTimeCalls, CVarBTRecordFrameSearchTimes.GetValueOnGameThread() != 0);
#endif

	UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("Execution request by %s (result: %s)"),
		*UBehaviorTreeTypes::DescribeNodeHelper(RequestedBy),
		*UBehaviorTreeTypes::DescribeNodeResult(ContinueWithResult));

	if (!bIsRunning || !InstanceStack.IsValidIndex(ActiveInstanceIdx) || (GetOwner() && GetOwner()->IsPendingKillPending()))
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("> skip: tree is not running"));
		return;
	}

	const bool bOutOfNodesPending = PendingExecution.IsSet() && PendingExecution.bOutOfNodes;
	if (bOutOfNodesPending)
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("> skip: tree ran out of nodes on previous restart and needs to process it first"));
		return;
	}

	checkf(InstanceIdx != FBTNodeIndex::InvalidIndex, TEXT("Index has used the InvalidIndex value!"));
	const uint16 InstanceIdxUint16 = IntCastChecked<uint16>(InstanceIdx);

	const bool bSwitchToHigherPriority = (ContinueWithResult == EBTNodeResult::Aborted);
	const bool bAlreadyHasRequest = (ExecutionRequest.ExecuteNode != NULL);
	const UBTNode* DebuggerNode = bStoreForDebugger ? RequestedBy : NULL;

	FBTNodeIndex ExecutionIdx;
	ExecutionIdx.InstanceIndex = InstanceIdxUint16;
	ExecutionIdx.ExecutionIndex = RequestedBy->GetExecutionIndex();
	uint16 LastExecutionIndex = MAX_uint16;

	// make sure that the request is not coming from a node that has pending branch actions since it won't be accessible anymore
	if (SuspendedBranchActions != EBTBranchAction::None)
	{
		if ((SuspendedBranchActions & ~(EBTBranchAction::Changing_Topology_Actions)) != EBTBranchAction::None)
		{
			UE_VLOG(GetOwner(), LogBehaviorTree, Warning, TEXT("Caller should be converted to new Evaluate/Activate/DeactivateBranch API instead of using this RequestExecution directly"));
		}

		for (const FBranchActionInfo& Info : PendingBranchActionRequests)
		{
			const UBTCompositeNode* BranchRoot = nullptr;
			switch (Info.Action)
			{
				case EBTBranchAction::DecoratorDeactivate:
					if (const UBTDecorator* Decorator = Cast<UBTDecorator>(Info.Node))
					{
						// This check was already in previous version, it will ensure and output a vlog later in the DeactivateBranch, no need to do anything now.
						if (Decorator->GetParentNode() && Decorator->GetParentNode()->Children.IsValidIndex(Decorator->GetChildIndex()))
						{
							BranchRoot = Decorator->GetParentNode()->Children[Decorator->GetChildIndex()].ChildComposite;
						}
					}
					break;
				case EBTBranchAction::UnregisterAuxNodes:
					if (const UBTCompositeNode* CompNode = Cast<UBTCompositeNode>(Info.Node))
					{
						BranchRoot = CompNode;
					}
					break;
			}

			if (BranchRoot)
			{
				const int32 BranchRootInstanceIdx = FindInstanceContainingNode(BranchRoot);
				if (BranchRootInstanceIdx != INDEX_NONE)
				{
					checkf(BranchRootInstanceIdx != FBTNodeIndex::InvalidIndex, TEXT("Index has used the InvalidIndex value!"));
					const uint16 BranchRootInstanceIdxUint16 = IntCastChecked<uint16>(BranchRootInstanceIdx);

					FBTNodeIndexRange Range( FBTNodeIndex(BranchRootInstanceIdxUint16, BranchRoot->GetExecutionIndex()), FBTNodeIndex(BranchRootInstanceIdxUint16, BranchRoot->GetLastExecutionIndex()));
					if (Range.Contains(ExecutionIdx))
					{
						UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("> skip: request by %s(%s) is in deactivated branch %s(%s) and was deactivated by %s"), *UBehaviorTreeTypes::DescribeNodeHelper(RequestedBy), *ExecutionIdx.Describe(), *UBehaviorTreeTypes::DescribeNodeHelper(BranchRoot), *Range.Describe(), *UBehaviorTreeTypes::DescribeNodeHelper(Info.Node));
						return;
					}
				}
			}
		}
	}
	else
	{
		checkf(PendingBranchActionRequests.Num() == 0, TEXT("All pending branches should have been flushed before requesting an execution"));
	}

	if (bSwitchToHigherPriority && RequestedByChildIndex >= 0)
	{
		ExecutionIdx.ExecutionIndex = RequestedOn->GetChildExecutionIndex(RequestedByChildIndex, EBTChildIndex::FirstNode);
		
		// first index outside allowed range		
		LastExecutionIndex = RequestedOn->GetChildExecutionIndex(RequestedByChildIndex + 1, EBTChildIndex::FirstNode);
	}

	const FBTNodeIndex SearchEnd(InstanceIdxUint16, LastExecutionIndex);

	// check if it's more important than currently requested
	if (bAlreadyHasRequest && ExecutionRequest.SearchStart.TakesPriorityOver(ExecutionIdx))
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("> skip: already has request with higher priority"));
		StoreDebuggerRestart(DebuggerNode, InstanceIdxUint16, true);

		// make sure to update end of search range
		if (bSwitchToHigherPriority)
		{
			if (ExecutionRequest.SearchEnd.IsSet() && ExecutionRequest.SearchEnd.TakesPriorityOver(SearchEnd))
			{
				UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("> expanding end of search range!"));
				ExecutionRequest.SearchEnd = SearchEnd;
			}
		}
		else
		{
			if (ExecutionRequest.SearchEnd.IsSet())
			{
				UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("> removing limit from end of search range!"));
				ExecutionRequest.SearchEnd = FBTNodeIndex();
			}
		}

		return;
	}

    // Not only checking against deactivated branch upon applying search data or while aborting task, 
    // but also while waiting after a latent task to abort
	if (SearchData.bFilterOutRequestFromDeactivatedBranch || bWaitingForLatentAborts)
	{
		// request on same node or with higher priority doesn't require additional checks
		if (SearchData.SearchRootNode != ExecutionIdx && SearchData.SearchRootNode.TakesPriorityOver(ExecutionIdx) && SearchData.DeactivatedBranchStart.IsSet())
		{
			ensureMsgf(SearchData.DeactivatedBranchStart.InstanceIndex == SearchData.DeactivatedBranchEnd.InstanceIndex, TEXT("Deactivated branch should always be in the same instance."));
			if (ExecutionIdx.InstanceIndex > SearchData.DeactivatedBranchStart.InstanceIndex)
			{
				UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("> skip: node index %s in a deactivated instance [%s..%s[ (applying search data for %s)"),
					*ExecutionIdx.Describe(), *SearchData.DeactivatedBranchStart.Describe(), *SearchData.DeactivatedBranchEnd.Describe(), *SearchData.SearchRootNode.Describe());
				StoreDebuggerRestart(DebuggerNode, InstanceIdxUint16, false);
				return;
			}
			else if (ExecutionIdx.InstanceIndex == SearchData.DeactivatedBranchStart.InstanceIndex && 
					ExecutionIdx.ExecutionIndex >= SearchData.DeactivatedBranchStart.ExecutionIndex &&
					ExecutionIdx.ExecutionIndex < SearchData.DeactivatedBranchEnd.ExecutionIndex)
			{
				UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("> skip: node index %s in a deactivated branch [%s..%s[ (applying search data for %s)"),
					*ExecutionIdx.Describe(), *SearchData.DeactivatedBranchStart.Describe(), *SearchData.DeactivatedBranchEnd.Describe(), *SearchData.SearchRootNode.Describe());
				StoreDebuggerRestart(DebuggerNode, InstanceIdxUint16, false);
				return;
			}
		}
	}

	// when it's aborting and moving to higher priority node:
	if (bSwitchToHigherPriority)
	{
		// check if decorators allow execution on requesting link
		// unless it's branch restart (abort result within current branch), when it can't be skipped because branch can be no longer valid
		const bool bShouldCheckDecorators = (RequestedByChildIndex >= 0) && !IsExecutingBranch(RequestedBy, RequestedByChildIndex);
		const bool bCanExecute = !bShouldCheckDecorators || RequestedOn->DoDecoratorsAllowExecution(*this, InstanceIdx, RequestedByChildIndex);
		if (!bCanExecute)
		{
			UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("> skip: decorators are not allowing execution"));
			StoreDebuggerRestart(DebuggerNode, InstanceIdxUint16, false);
			return;
		}

		// update common parent: requesting node with prev common/active node
		const UBTCompositeNode* CurrentNode = ExecutionRequest.ExecuteNode;
		uint16 CurrentInstanceIdx = ExecutionRequest.ExecuteInstanceIdx;
		if (ExecutionRequest.ExecuteNode == NULL)
		{
			FBehaviorTreeInstance& ActiveInstance = InstanceStack[ActiveInstanceIdx];
			CurrentNode = (ActiveInstance.ActiveNode == NULL) ? ActiveInstance.RootNode.Get() :
				(ActiveInstance.ActiveNodeType == EBTActiveNode::Composite) ? (UBTCompositeNode*)ActiveInstance.ActiveNode.Get() :
				ActiveInstance.ActiveNode->GetParentNode();

			CurrentInstanceIdx = ActiveInstanceIdx;
		}

		if (ExecutionRequest.ExecuteNode != RequestedOn)
		{
			const UBTCompositeNode* CommonParent = NULL;
			uint16 CommonInstanceIdx = MAX_uint16;

			FindCommonParent(InstanceStack, KnownInstances, RequestedOn, InstanceIdxUint16, CurrentNode, CurrentInstanceIdx, CommonParent, CommonInstanceIdx);

			// check decorators between common parent and restart parent
			int32 ItInstanceIdx = InstanceIdx;
			for (const UBTCompositeNode* It = RequestedOn; It && It != CommonParent;)
			{
				const UBTCompositeNode* ParentNode = It->GetParentNode();
				int32 ChildIdx = INDEX_NONE;

				if (ParentNode == nullptr)
				{
					// move up the tree stack
					if (ItInstanceIdx > 0)
					{
						ItInstanceIdx--;
						UBTNode* SubtreeTaskNode = InstanceStack[ItInstanceIdx].ActiveNode;
						ParentNode = SubtreeTaskNode->GetParentNode();
						ChildIdx = ParentNode->GetChildIndex(*SubtreeTaskNode);
					}
					else
					{
						// something went wrong...
						break;
					}
				}
				else
				{
					ChildIdx = ParentNode->GetChildIndex(*It);
				}

				const bool bCanExecuteTest = ParentNode->DoDecoratorsAllowExecution(*this, ItInstanceIdx, ChildIdx);
				if (!bCanExecuteTest)
				{
					UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("> skip: decorators are not allowing execution"));
					StoreDebuggerRestart(DebuggerNode, InstanceIdxUint16, false);
					return;
				}

				It = ParentNode;
			}

			ExecutionRequest.ExecuteNode = CommonParent;
			ExecutionRequest.ExecuteInstanceIdx = CommonInstanceIdx;
		}
	}
	else
	{
		// check if decorators allow execution on requesting link (only when restart comes from composite decorator)
		const bool bShouldCheckDecorators = RequestedOn->Children.IsValidIndex(RequestedByChildIndex) &&
			(RequestedOn->Children[RequestedByChildIndex].DecoratorOps.Num() > 0) &&
			RequestedBy->IsA(UBTDecorator::StaticClass());

		const bool bCanExecute = bShouldCheckDecorators && RequestedOn->DoDecoratorsAllowExecution(*this, InstanceIdx, RequestedByChildIndex);
		if (bCanExecute)
		{
			UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("> skip: decorators are still allowing execution"));
			StoreDebuggerRestart(DebuggerNode, InstanceIdxUint16, false);
			return;
		}

		ExecutionRequest.ExecuteNode = RequestedOn;
		ExecutionRequest.ExecuteInstanceIdx = InstanceIdxUint16;
	}

	// store it
	StoreDebuggerRestart(DebuggerNode, InstanceIdxUint16, true);

	// search end can be set only when switching to high priority
	// or previous request was limited and current limit is wider
	if ((!bAlreadyHasRequest && bSwitchToHigherPriority) ||
		(ExecutionRequest.SearchEnd.IsSet() && ExecutionRequest.SearchEnd.TakesPriorityOver(SearchEnd)))
	{
		UE_CVLOG(bAlreadyHasRequest, GetOwner(), LogBehaviorTree, Log, TEXT("%s"), (SearchEnd.ExecutionIndex < MAX_uint16) ? TEXT("> expanding end of search range!") : TEXT("> removing limit from end of search range!"));
		ExecutionRequest.SearchEnd = SearchEnd;
	}

	ExecutionRequest.SearchStart = ExecutionIdx;
	ExecutionRequest.ContinueWithResult = ContinueWithResult;
	ExecutionRequest.bTryNextChild = !bSwitchToHigherPriority;
	ExecutionRequest.bIsRestart = (RequestedBy != GetActiveNode());
	PendingExecution.Lock();
	
	// break out of current search if new request is more important than currently processed one
	// no point in starting new task just to abandon it in next tick
	if (SearchData.bSearchInProgress)
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("> aborting current task search!"));
		SearchData.bPostponeSearch = true;
	}

	// latent task abort:
	// - don't search, just accumulate requests and run them when abort is done
	// - rollback changes from search that caused abort to ensure proper state of tree
	const bool bIsActiveNodeAborting = InstanceStack.Num() && InstanceStack.Last().ActiveNodeType == EBTActiveNode::AbortingTask;
	const bool bInvalidateCurrentSearch = bWaitingForLatentAborts || bIsActiveNodeAborting;
	const bool bScheduleNewSearch = !bWaitingForLatentAborts;

	if (bInvalidateCurrentSearch)
	{
        // We are aborting the current search, but in the case we were searching to a next child, we cannot look for only higher priority as sub decorator might still fail
		// Previous search might have been a different range, so just open it up to cover all cases
		if (ExecutionRequest.SearchEnd.IsSet())
		{
			UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("> removing limit from end of search range because of request during task abortion!"));
			ExecutionRequest.SearchEnd = FBTNodeIndex();
		}
		RollbackSearchChanges();
	}
	
	if (bScheduleNewSearch)
	{
		ScheduleExecutionUpdate();
	}
}

void UBehaviorTreeComponent::ApplySearchUpdatesImpl(const TArray<FBehaviorTreeSearchUpdate>& UpdateList, int32 NewNodeExecutionIndex, bool bPostUpdate, bool bAllowTaskUpdates)
{
	for (int32 Index = 0; Index < UpdateList.Num(); Index++)
	{
		const FBehaviorTreeSearchUpdate& UpdateInfo = UpdateList[Index];
		// Check if we are in the right pass...
		if (UpdateInfo.bPostUpdate != bPostUpdate)
		{
			continue;
		}

		if (!InstanceStack.IsValidIndex(UpdateInfo.InstanceIndex))
		{
			continue;
		}

		// Push the stored instance index as the active one so calls from the following processing 
		// will get the right instance when calling GetActiveInstanceIdx();
		TGuardValue GuardActiveInstanceIndex(ActiveInstanceIdx, UpdateInfo.InstanceIndex);

		FBehaviorTreeInstance& UpdateInstance = InstanceStack[UpdateInfo.InstanceIndex];
		int32 ParallelTaskIdx = INDEX_NONE;
		bool bIsComponentActive = false;

		if (UpdateInfo.AuxNode)
		{
			bIsComponentActive = UpdateInstance.GetActiveAuxNodes().Contains(UpdateInfo.AuxNode);
		}
		else if (bAllowTaskUpdates && UpdateInfo.TaskNode)
		{
			ParallelTaskIdx = UpdateInstance.GetParallelTasks().IndexOfByKey(UpdateInfo.TaskNode);
			bIsComponentActive = (ParallelTaskIdx != INDEX_NONE && UpdateInstance.GetParallelTasks()[ParallelTaskIdx].Status == EBTTaskStatus::Active);
		}
		else
		{
			continue;
		}

		const UBTNode* UpdateNode = UpdateInfo.AuxNode ? (const UBTNode*)UpdateInfo.AuxNode : (const UBTNode*)UpdateInfo.TaskNode;
		checkSlow(UpdateNode);

		if ((UpdateInfo.Mode == EBTNodeUpdateMode::Remove && !bIsComponentActive) ||
			(UpdateInfo.Mode == EBTNodeUpdateMode::Add && (bIsComponentActive || UpdateNode->GetExecutionIndex() > NewNodeExecutionIndex)))
		{
			UpdateInfo.bApplySkipped = true;
			UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("Skipping: %s for %s: %s (already in the right state)"),
				*UBehaviorTreeTypes::DescribeNodeUpdateMode(UpdateInfo.Mode),
				UpdateInfo.AuxNode ? TEXT("auxiliary node") : TEXT("parallel's main task"),
				*UBehaviorTreeTypes::DescribeNodeHelper(UpdateNode));

			continue;
		}

		UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("Applying: %s for %s: %s"),
			*UBehaviorTreeTypes::DescribeNodeUpdateMode(UpdateInfo.Mode),
			UpdateInfo.AuxNode ? TEXT("auxiliary node") : TEXT("parallel's main task"),
			*UBehaviorTreeTypes::DescribeNodeHelper(UpdateNode));

		if (UpdateInfo.AuxNode)
		{
			// special case: service node at root of top most subtree - don't remove/re-add them when tree is in looping mode
			// don't bother with decorators parent == root means that they are on child branches
			if (bLoopExecution && UpdateInfo.AuxNode->GetMyNode() == InstanceStack[0].RootNode &&
				UpdateInfo.AuxNode->IsA(UBTService::StaticClass()))
			{
				if (UpdateInfo.Mode == EBTNodeUpdateMode::Remove ||
					InstanceStack[0].GetActiveAuxNodes().Contains(UpdateInfo.AuxNode))
				{
					UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("> skip [looped execution]"));
					continue;
				}
			}

			uint8* NodeMemory = (uint8*)UpdateNode->GetNodeMemory<uint8>(UpdateInstance);
			if (UpdateInfo.Mode == EBTNodeUpdateMode::Remove)
			{
				UpdateInstance.RemoveFromActiveAuxNodes(*this, UpdateInfo.AuxNode);
				UpdateInfo.AuxNode->WrappedOnCeaseRelevant(*this, NodeMemory);
			}
			else
			{
				UpdateInstance.AddToActiveAuxNodes(*this, UpdateInfo.AuxNode);
				UpdateInfo.AuxNode->WrappedOnBecomeRelevant(*this, NodeMemory);
			}
		}
		else if (UpdateInfo.TaskNode)
		{
			if (UpdateInfo.Mode == EBTNodeUpdateMode::Remove)
			{
				// remove all message observers from node to abort to avoid calling OnTaskFinished from AbortTask
				UnregisterMessageObserversFrom(UpdateInfo.TaskNode);

				uint8* NodeMemory = (uint8*)UpdateNode->GetNodeMemory<uint8>(UpdateInstance);
				EBTNodeResult::Type NodeResult = UpdateInfo.TaskNode->WrappedAbortTask(*this, NodeMemory);

				UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("Parallel task aborted: %s (%s)"),
					*UBehaviorTreeTypes::DescribeNodeHelper(UpdateInfo.TaskNode),
					(NodeResult == EBTNodeResult::InProgress) ? TEXT("in progress") : TEXT("instant"));

				// check if task node is still valid, could've received LatentAbortFinished during AbortTask call
				const bool bStillValid = InstanceStack.IsValidIndex(UpdateInfo.InstanceIndex) &&
					InstanceStack[UpdateInfo.InstanceIndex].GetParallelTasks().IsValidIndex(ParallelTaskIdx) &&
					InstanceStack[UpdateInfo.InstanceIndex].GetParallelTasks()[ParallelTaskIdx] == UpdateInfo.TaskNode;

				if (bStillValid)
				{
					// mark as pending abort
					if (NodeResult == EBTNodeResult::InProgress)
					{
						UpdateInstance.MarkParallelTaskAsAbortingAt(ParallelTaskIdx);
						bWaitingForLatentAborts = true;
					}

					OnTaskFinished(UpdateInfo.TaskNode, NodeResult);
				}
			}
			else
			{
				UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("Parallel task: %s added to active list"),
					*UBehaviorTreeTypes::DescribeNodeHelper(UpdateInfo.TaskNode));

				UpdateInstance.AddToParallelTasks(FBehaviorTreeParallelTask(UpdateInfo.TaskNode, EBTTaskStatus::Active));
			}
		}
	}
}

void UBehaviorTreeComponent::ApplySearchUpdates(const TArray<FBehaviorTreeSearchUpdate>& UpdateList, int32 NewNodeExecutionIndex, bool bPostUpdate)
{
	ApplySearchUpdatesImpl(UpdateList, NewNodeExecutionIndex, bPostUpdate, true);
}

void UBehaviorTreeComponent::ApplyAllSearchUpdates(const TArray<FBehaviorTreeSearchUpdate>& UpdateList, int32 NewNodeExecutionIndex, bool bDoPostUpdate, bool bAllowTaskUpdates)
{
	const int32 NumIterations = bDoPostUpdate ? 2 : 1;
	constexpr int32 PostUpdateIter = 1;

	for (int32 UpdateIter = 0; UpdateIter < NumIterations; ++UpdateIter)
	{
		const bool bPostUpdate = UpdateIter == PostUpdateIter;

		ApplySearchUpdatesImpl(UpdateList, NewNodeExecutionIndex, bPostUpdate, bAllowTaskUpdates);
	}

#if ENABLE_VISUAL_LOG
	auto LogAuxNodes = [this]()
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, VeryVerbose, TEXT("Logging All Aux Nodes"));
		for (int32 InstanceIndex = 0; InstanceIndex < InstanceStack.Num(); InstanceIndex++)
		{
			FBehaviorTreeInstance& InstanceInfo = InstanceStack[InstanceIndex];
			InstanceInfo.ExecuteOnEachAuxNode([this](const UBTAuxiliaryNode& AuxNode)
				{
					UE_VLOG(GetOwner(), LogBehaviorTree, VeryVerbose, TEXT("%s"), *UBehaviorTreeTypes::DescribeNodeHelper(&AuxNode));
				});
		}
	};

	UE_IFVLOG(LogAuxNodes());
#endif // ENABLE_VISUAL_LOG
}

void UBehaviorTreeComponent::TickNewlyAddedAuxNodesHelper()
{
	for (const FBehaviorTreeSearchUpdate& UpdateInfo : SearchData.PendingUpdates)
	{
		if (!UpdateInfo.bApplySkipped && UpdateInfo.Mode == EBTNodeUpdateMode::Add && UpdateInfo.AuxNode && InstanceStack.IsValidIndex(UpdateInfo.InstanceIndex))
		{
			FBehaviorTreeInstance& InstanceInfo = InstanceStack[UpdateInfo.InstanceIndex];
			uint8* NodeMemory = UpdateInfo.AuxNode->GetNodeMemory<uint8>(InstanceInfo);

			// We do not care about the next needed DeltaTime, it will be recalculated in the tick later.
			float NextNeededDeltaTime = 0.0f;
			UpdateInfo.AuxNode->WrappedTickNode(*this, NodeMemory, CurrentFrameDeltaTime, NextNeededDeltaTime);
		}
	}
}

void UBehaviorTreeComponent::ApplySearchData(UBTNode* NewActiveNode)
{
	// search is finalized, can't rollback anymore at this point
	SearchData.RollbackInstanceIdx = INDEX_NONE;
	SearchData.RollbackDeactivatedBranchStart = FBTNodeIndex();
	SearchData.RollbackDeactivatedBranchEnd = FBTNodeIndex();

	// send all deactivation notifies for bookkeeping
	for (int32 Idx = 0; Idx < SearchData.PendingNotifies.Num(); Idx++)
	{
		const FBehaviorTreeSearchUpdateNotify& NotifyInfo = SearchData.PendingNotifies[Idx];
		if (InstanceStack.IsValidIndex(NotifyInfo.InstanceIndex))
		{
			InstanceStack[NotifyInfo.InstanceIndex].DeactivationNotify.ExecuteIfBound(*this, NotifyInfo.NodeResult);
		}	
	}

	// apply changes to aux nodes and parallel tasks
	const int32 NewNodeExecutionIndex = NewActiveNode ? NewActiveNode->GetExecutionIndex() : 0;

	SearchData.bFilterOutRequestFromDeactivatedBranch = true;

	ApplyAllSearchUpdates(SearchData.PendingUpdates, NewNodeExecutionIndex);
	
	SearchData.bFilterOutRequestFromDeactivatedBranch = false;

	// tick newly added aux nodes to compensate for tick-search order changes
	TickNewlyAddedAuxNodesHelper();

	// clear update list
	// nothing should be added during application or tick - all changes are supposed to go to ExecutionRequest accumulator first
	SearchData.PendingUpdates.Reset();
	SearchData.PendingNotifies.Reset();
	SearchData.DeactivatedBranchStart = FBTNodeIndex();
	SearchData.DeactivatedBranchEnd = FBTNodeIndex();
}

void UBehaviorTreeComponent::ApplyDiscardedSearch()
{
	if (UE::BehaviorTree::bApplyAuxNodesFromFailedSearches)
	{
		// Apply aux nodes from last search with the currently active node as the 'NewNode'
		int32 NewNodeExecutionIdx = 0;

		if (InstanceStack.IsValidIndex(ActiveInstanceIdx))
		{
			FBehaviorTreeInstance& BTInstance = InstanceStack[ActiveInstanceIdx];

			NewNodeExecutionIdx = BTInstance.ActiveNode ? BTInstance.ActiveNode->GetExecutionIndex() : 0;
		}

		ApplyAllSearchUpdates(/*UpdateList*/ SearchData.PendingUpdates, /*NewNodeExecutionIndex*/ NewNodeExecutionIdx, /*bDoPostUpdate*/ true, /*bAllowTaskUpdates*/ false);

		// tick newly added aux nodes to compensate for tick-search order changes
		TickNewlyAddedAuxNodesHelper();
	}

	SearchData.PendingUpdates.Reset();

	// don't send deactivation notifies
	SearchData.PendingNotifies.Reset();
}

void UBehaviorTreeComponent::TickComponent(float DeltaTime, const ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	ensureMsgf(bIsRunning || NextTickDeltaTime != UE::BehaviorTree::DisableTick, TEXT("Tree should never be ticked if it is no longer running"));

	// Tick can be optimized by the tick function to not be called every frame so we need
	// to set the current frame delta time based on that information for other tick scenarios (e.g. manual ticking in unit tests)
	const UWorld* World = GetWorld();
	const bool bUseWorldDTForCurrentFrame = (ThisTickFunction != nullptr && World != nullptr);
	
	CurrentFrameDeltaTime = bUseWorldDTForCurrentFrame ? World->GetDeltaSeconds() : DeltaTime;

	// Warn if BT asked to be ticked the next frame and did not.
	if (bTickedOnce && NextTickDeltaTime == 0.0f && bUseWorldDTForCurrentFrame)
	{
		const double CurrentGameTime = World->GetTimeSeconds();
		if (CurrentGameTime - LastRequestedDeltaTimeGameTime - CurrentFrameDeltaTime > KINDA_SMALL_NUMBER)
		{
			UE_VLOG(GetOwner(), LogBehaviorTree, Error, TEXT("BT(%llu) expected to be tick next frame, current deltatime(%f) and calculated deltatime(%f)."), GFrameCounter, CurrentFrameDeltaTime, CurrentGameTime - LastRequestedDeltaTimeGameTime);
		}
	}

	// Check if we really have reached the asked DeltaTime, 
	// If not then accumulate it and reschedule
	NextTickDeltaTime -= DeltaTime;
	if (NextTickDeltaTime > 0.0f)
	{
		// The TickManager is using global time to calculate delta since last ticked time. When the value is big, we can get into float precision errors compare to our calculation.
		if (ThisTickFunction != nullptr && NextTickDeltaTime > KINDA_SMALL_NUMBER)
		{
			UE_VLOG(GetOwner(), LogBehaviorTree, Error, TEXT("BT(%i) did not need to be tick, ask deltatime of %fs got %fs with a diff of %fs."), GFrameCounter, NextTickDeltaTime + AccumulatedTickDeltaTime + DeltaTime, DeltaTime + AccumulatedTickDeltaTime, NextTickDeltaTime);
		}
		AccumulatedTickDeltaTime += DeltaTime;
		ScheduleNextTick(NextTickDeltaTime);
		return;
	}

	AccumulatedTickDeltaTime += DeltaTime;
	ON_SCOPE_EXIT
	{
		AccumulatedTickDeltaTime = 0.0f;
	};

	DeltaTime = AccumulatedTickDeltaTime;

	const bool bWasTickedOnce = bTickedOnce;
	bTickedOnce = true;

	bool bDoneSomething = MessagesToProcess.Num() > 0;
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	SCOPE_CYCLE_COUNTER(STAT_AI_Overall);
	SCOPE_CYCLE_COUNTER(STAT_AI_BehaviorTree_Tick);
#if CSV_PROFILER_STATS
	// Configurable CSV_SCOPED_TIMING_STAT_EXCLUSIVE(BehaviorTreeTick);
	FScopedCsvStatExclusive _ScopedCsvStatExclusive_BehaviorTreeTick(CSVTickStatName);
#endif

	check(IsValidChecked(this));
	float NextNeededDeltaTime = UE::BehaviorTree::DisableTick;

	checkf(PendingBranchActionRequests.Num() == 0, TEXT("Pending branches action requests should always be flushed immediately with the new system"))

	// tick active auxiliary nodes (in execution order, before task)
	// do it before processing execution request to give BP driven logic chance to accumulate execution requests
	// newly added aux nodes are ticked as part of SearchData application
	{
		FBTSuspendBranchActionsScoped ScopedSuspend(*this, EBTBranchAction::Changing_Topology_Actions);
		for (int32 InstanceIndex = 0; InstanceIndex < InstanceStack.Num(); InstanceIndex++)
		{
			FBehaviorTreeInstance& InstanceInfo = InstanceStack[InstanceIndex];
			InstanceInfo.ExecuteOnEachAuxNode([&InstanceInfo, this, &bDoneSomething, DeltaTime, &NextNeededDeltaTime](const UBTAuxiliaryNode& AuxNode)
				{
					uint8* NodeMemory = AuxNode.GetNodeMemory<uint8>(InstanceInfo);
					SCOPE_CYCLE_UOBJECT(AuxNode, &AuxNode);
					bDoneSomething |= AuxNode.WrappedTickNode(*this, NodeMemory, DeltaTime, NextNeededDeltaTime);
				});
		}
	}

	// make sure that we continue execution after all pending latent aborts finished
	const bool bJustFinishedLatentAborts = TrackPendingLatentAborts();
	if (bJustFinishedLatentAborts)
	{
		if (bRequestedStop)
		{
			StopTree(EBTStopMode::Safe);
		}
		else
		{
			// force new search if there were any execution requests while waiting for aborting task
			if (ExecutionRequest.ExecuteNode)
			{
				UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("> found valid ExecutionRequest, locking PendingExecution data to force new search!"));
				PendingExecution.Lock();

				if (ExecutionRequest.SearchEnd.IsSet())
				{
					UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("> removing limit from end of search range! [abort done]"));
					ExecutionRequest.SearchEnd = FBTNodeIndex();
				}
			}

			ScheduleExecutionUpdate();
		}
	}

	bool bActiveAuxiliaryNodeDTDirty = false;
	if (bRequestedFlowUpdate)
	{
		ProcessExecutionRequest();
		bDoneSomething = true;

        // Since hierarchy might changed in the ProcessExecutionRequest, we need to go through all the active auxiliary nodes again to fetch new next DeltaTime
		bActiveAuxiliaryNodeDTDirty = true;
		NextNeededDeltaTime = UE::BehaviorTree::DisableTick;
	}

	if (InstanceStack.Num() > 0 && bIsRunning && !bIsPaused)
	{
		FBTSuspendBranchActionsScoped ScopedSuspend(*this, EBTBranchAction::Changing_Topology_Actions);

		// tick active parallel tasks (in execution order, before task)
		for (int32 InstanceIndex = 0; InstanceIndex < InstanceStack.Num(); InstanceIndex++)
		{
			FBehaviorTreeInstance& InstanceInfo = InstanceStack[InstanceIndex];
			InstanceInfo.ExecuteOnEachParallelTask([&InstanceInfo, &bDoneSomething, this, DeltaTime, &NextNeededDeltaTime](const FBehaviorTreeParallelTask& ParallelTaskInfo, const int32 Index)
				{
					const UBTTaskNode* ParallelTask = ParallelTaskInfo.TaskNode;
					SCOPE_CYCLE_UOBJECT(ParallelTask, ParallelTask);
					uint8* NodeMemory = ParallelTask->GetNodeMemory<uint8>(InstanceInfo);
					bDoneSomething |= ParallelTask->WrappedTickTask(*this, NodeMemory, DeltaTime, NextNeededDeltaTime);
				});
		}

		// tick active task
		if (InstanceStack.IsValidIndex(ActiveInstanceIdx))
		{
			FBehaviorTreeInstance& ActiveInstance = InstanceStack[ActiveInstanceIdx];
			if (ActiveInstance.ActiveNodeType == EBTActiveNode::ActiveTask ||
				ActiveInstance.ActiveNodeType == EBTActiveNode::AbortingTask)
			{
				UBTTaskNode* ActiveTask = (UBTTaskNode*)ActiveInstance.ActiveNode;
				uint8* NodeMemory = ActiveTask->GetNodeMemory<uint8>(ActiveInstance);
				SCOPE_CYCLE_UOBJECT(ActiveTask, ActiveTask);
				bDoneSomething |= ActiveTask->WrappedTickTask(*this, NodeMemory, DeltaTime, NextNeededDeltaTime);
			}
		}

		// tick aborting task from abandoned subtree
		if (InstanceStack.IsValidIndex(ActiveInstanceIdx + 1))
		{
			FBehaviorTreeInstance& LastInstance = InstanceStack.Last();
			if (LastInstance.ActiveNodeType == EBTActiveNode::AbortingTask)
			{
				UBTTaskNode* ActiveTask = (UBTTaskNode*)LastInstance.ActiveNode;
				uint8* NodeMemory = ActiveTask->GetNodeMemory<uint8>(LastInstance);
				SCOPE_CYCLE_UOBJECT(ActiveTask, ActiveTask);
				bDoneSomething |= ActiveTask->WrappedTickTask(*this, NodeMemory, DeltaTime, NextNeededDeltaTime);
			}
		}
	}

	// Go through all active auxiliary nodes to calculate the next NeededDeltaTime if needed
	if (bActiveAuxiliaryNodeDTDirty)
	{
		for (int32 InstanceIndex = 0; InstanceIndex < InstanceStack.Num() && NextNeededDeltaTime > 0.0f; InstanceIndex++)
		{
			FBehaviorTreeInstance& InstanceInfo = InstanceStack[InstanceIndex];
			for (const UBTAuxiliaryNode* AuxNode : InstanceInfo.GetActiveAuxNodes())
			{
				uint8* NodeMemory = AuxNode->GetNodeMemory<uint8>(InstanceInfo);
				const float NextNodeNeededDeltaTime = AuxNode->GetNextNeededDeltaTime(*this, NodeMemory);
				if (NextNeededDeltaTime > NextNodeNeededDeltaTime)
				{
					NextNeededDeltaTime = NextNodeNeededDeltaTime;
				}
			}
		}
	}

	if (bWasTickedOnce && !bDoneSomething)
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Error, TEXT("BT(%i) planned to do something but actually did not."), GFrameCounter);
	}
	ScheduleNextTick(NextNeededDeltaTime);

#if DO_ENSURE
	// Adding code to track an problem earlier that is happening by RequestExecution from a decorator that has lower priority.
	// The idea here is to try to rule out that the tick leaves the behavior tree is a bad state with lower priority decorators(AuxNodes).
	static bool bWarnOnce = false;
	if (!bWarnOnce)
	{
		for (int32 InstanceIndex = 0; InstanceIndex < InstanceStack.Num(); InstanceIndex++)
		{
			const FBehaviorTreeInstance& InstanceInfo = InstanceStack[InstanceIndex];
			if (!InstanceInfo.ActiveNode)
			{
				break;
			}

			const uint16 ActiveExecutionIdx = InstanceInfo.ActiveNode->GetExecutionIndex();
			for (const UBTAuxiliaryNode* ActiveAuxNode : InstanceInfo.GetActiveAuxNodes())
			{
				if (ActiveAuxNode->GetExecutionIndex() >= ActiveExecutionIdx)
				{
					FString ErrorMsg(FString::Printf(TEXT("%s: leaving the tick of behavior tree with a lower priority active node %s, Current Tasks : "),
						ANSI_TO_TCHAR(__FUNCTION__),
						*UBehaviorTreeTypes::DescribeNodeHelper(ActiveAuxNode)));

					for (int32 ParentInstanceIndex = 0; ParentInstanceIndex <= InstanceIndex; ++ParentInstanceIndex)
					{
						ErrorMsg += *UBehaviorTreeTypes::DescribeNodeHelper(InstanceStack[ParentInstanceIndex].ActiveNode);
						ErrorMsg += TEXT("\\");
					}

					UE_VLOG(GetOwner(), LogBehaviorTree, Error, TEXT("%s"), *ErrorMsg);
					ensureMsgf(false, TEXT("%s"), *ErrorMsg);
					bWarnOnce = true;
					break;
				}
			}
		}
	}
#endif // DO_ENSURE
}

void UBehaviorTreeComponent::ScheduleNextTick(const float NextNeededDeltaTime)
{
	NextTickDeltaTime = NextNeededDeltaTime;
	if (bRequestedFlowUpdate)
	{
		NextTickDeltaTime = 0.0f;
	}

	UE_VLOG(GetOwner(), LogBehaviorTree, VeryVerbose, TEXT("BT(%i) schedule next tick %f, asked %f."), GFrameCounter, NextTickDeltaTime, NextNeededDeltaTime);
	if (NextTickDeltaTime == UE::BehaviorTree::DisableTick)
	{
		if (IsComponentTickEnabled())
		{
			SetComponentTickEnabled(false);
		}
	}
	else
	{
		if (!IsComponentTickEnabled())
		{
			SetComponentTickEnabled(true);
		}
		// We need to force a small dt to tell the TickTaskManager we might not want to be tick every frame.
		const float FORCE_TICK_INTERVAL_DT = KINDA_SMALL_NUMBER;
		SetComponentTickIntervalAndCooldown(!bTickedOnce && NextTickDeltaTime < FORCE_TICK_INTERVAL_DT ? FORCE_TICK_INTERVAL_DT : NextTickDeltaTime);
	}
	UWorld* MyWorld = GetWorld();
	LastRequestedDeltaTimeGameTime = MyWorld ? MyWorld->GetTimeSeconds() : 0.;
}

void UBehaviorTreeComponent::ProcessExecutionRequest()
{
	bRequestedFlowUpdate = false;
	if (!IsRegistered() || !InstanceStack.IsValidIndex(ActiveInstanceIdx))
	{
		// it shouldn't be called, component is no longer valid
		return;
	}

	if (bIsPaused)
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("Ignoring ProcessExecutionRequest call due to BTComponent still being paused"));
		return;
	}

	if (bWaitingForLatentAborts)
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("Ignoring ProcessExecutionRequest call, aborting task must finish first"));
		return;
	}

	UE_VLOG(GetOwner(), LogBehaviorTree, VeryVerbose, TEXT("%hs Active Node %s"), __FUNCTION__, *UBehaviorTreeTypes::DescribeNodeHelper(InstanceStack[ActiveInstanceIdx].ActiveNode));

	if (PendingExecution.IsSet())
	{
		ProcessPendingExecution();
		return;
	}

	if (ExecutionRequest.ExecuteNode == nullptr)
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Display, TEXT("Ignoring ProcessExecutionRequest call, node to be executed is not set."));
		return;
	}

	bool bIsSearchValid = true;
	SearchData.RollbackInstanceIdx = ActiveInstanceIdx;
	SearchData.RollbackDeactivatedBranchStart = SearchData.DeactivatedBranchStart;
	SearchData.RollbackDeactivatedBranchEnd = SearchData.DeactivatedBranchEnd;

	// Setting search root node here it can be refer to when calling 'DeactivateUpTo'
	SearchData.SearchRootNode = FBTNodeIndex(ExecutionRequest.ExecuteInstanceIdx, ExecutionRequest.ExecuteNode->GetExecutionIndex());

	EBTNodeResult::Type NodeResult = ExecutionRequest.ContinueWithResult;
	UBTTaskNode* NextTask = NULL;

	{
		SCOPE_CYCLE_COUNTER(STAT_AI_BehaviorTree_SearchTime);

#if !UE_BUILD_SHIPPING
		// Code for timing BT Search
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(BehaviorTreeSearch);

		FScopedSwitchedCountedDurationTimer ScopedSwitchedCountedDurationTimer(FrameSearchTime, NumSearchTimeCalls, CVarBTRecordFrameSearchTimes.GetValueOnGameThread() != 0);
#endif

		// copy current memory in case we need to rollback search
		CopyInstanceMemoryToPersistent();

		// deactivate up to ExecuteNode
		if (InstanceStack[ActiveInstanceIdx].ActiveNode != ExecutionRequest.ExecuteNode)
		{
			int32 LastDeactivatedChildIndex = INDEX_NONE;
			const bool bDeactivated = DeactivateUpTo(ExecutionRequest.ExecuteNode, ExecutionRequest.ExecuteInstanceIdx, NodeResult, LastDeactivatedChildIndex);
			if (!bDeactivated)
			{
				// error occurred and tree will restart, all pending deactivation notifies will be lost
				// this should never happen

				BT_SEARCHLOG(SearchData, Error, TEXT("Unable to deactivate up to %s. Active node is %s. All pending updates will be lost!"), 
					*UBehaviorTreeTypes::DescribeNodeHelper(ExecutionRequest.ExecuteNode), 
					*UBehaviorTreeTypes::DescribeNodeHelper(InstanceStack[ActiveInstanceIdx].ActiveNode));
				SearchData.PendingUpdates.Reset();

				return;
			}
			else if (LastDeactivatedChildIndex != INDEX_NONE)
			{
				// Calculating/expanding the deactivated branch for filtering execution request while applying changes.
				FBTNodeIndex NewDeactivatedBranchStart(ExecutionRequest.ExecuteInstanceIdx, ExecutionRequest.ExecuteNode->GetChildExecutionIndex(LastDeactivatedChildIndex, EBTChildIndex::FirstNode));
				FBTNodeIndex NewDeactivatedBranchEnd(ExecutionRequest.ExecuteInstanceIdx, ExecutionRequest.ExecuteNode->GetChildExecutionIndex(LastDeactivatedChildIndex + 1, EBTChildIndex::FirstNode));

				ensureMsgf(!SearchData.DeactivatedBranchStart.IsSet(), TEXT("There should not have more than one deactivated branch. (Previous start:%s, New start:%s"), *SearchData.DeactivatedBranchStart.Describe(), *NewDeactivatedBranchStart.Describe());
				SearchData.DeactivatedBranchStart = NewDeactivatedBranchStart;
				ensureMsgf(!SearchData.DeactivatedBranchEnd.IsSet(), TEXT("There should not have more than one deactivated branch. (Previous end:%s, New end:%s"), *SearchData.DeactivatedBranchEnd.Describe(), *NewDeactivatedBranchEnd.Describe());
				SearchData.DeactivatedBranchEnd = NewDeactivatedBranchEnd;
			}
		}

		FBehaviorTreeInstance& ActiveInstance = InstanceStack[ActiveInstanceIdx];
		const UBTCompositeNode* TestNode = ExecutionRequest.ExecuteNode;
		SearchData.AssignSearchId();
		SearchData.bPostponeSearch = false;
		SearchData.bSearchInProgress = true;

		// activate root node if needed (can't be handled by parent composite...)
		if (ActiveInstance.ActiveNode == NULL)
		{
			ActiveInstance.ActiveNode = InstanceStack[ActiveInstanceIdx].RootNode;
			ActiveInstance.RootNode->OnNodeActivation(SearchData);
			BT_SEARCHLOG(SearchData, Verbose, TEXT("Activated root node: %s"), *UBehaviorTreeTypes::DescribeNodeHelper(ActiveInstance.RootNode));
		}

		// additional operations for restarting:
		if (!ExecutionRequest.bTryNextChild)
		{
			// mark all decorators less important than current search start node for removal
			const FBTNodeIndex DeactivateIdx(ExecutionRequest.SearchStart.InstanceIndex, ExecutionRequest.SearchStart.ExecutionIndex - 1);
			UnregisterAuxNodesUpTo(ExecutionRequest.SearchStart.ExecutionIndex ? DeactivateIdx : ExecutionRequest.SearchStart);

			// reactivate top search node, so it could use search range correctly
			BT_SEARCHLOG(SearchData, Verbose, TEXT("Reactivate node: %s [restart]"), *UBehaviorTreeTypes::DescribeNodeHelper(TestNode));
			ExecutionRequest.ExecuteNode->OnNodeRestart(SearchData);

			SearchData.SearchStart = ExecutionRequest.SearchStart;
			SearchData.SearchEnd = ExecutionRequest.SearchEnd;

			BT_SEARCHLOG(SearchData, Verbose, TEXT("Clamping search range: %s .. %s"),
				*SearchData.SearchStart.Describe(), *SearchData.SearchEnd.Describe());
		}
		else
		{
			// mark all decorators less important than current search start node for removal
			// (keep aux nodes for requesting node since it is higher priority)
			if (ExecutionRequest.ContinueWithResult == EBTNodeResult::Failed)
			{
				BT_SEARCHLOG(SearchData, Verbose, TEXT("Unregistering aux nodes up to %s"), *ExecutionRequest.SearchStart.Describe());
				UnregisterAuxNodesUpTo(ExecutionRequest.SearchStart);
			}

			// make sure it's reset before starting new search
			SearchData.SearchStart = FBTNodeIndex();
			SearchData.SearchEnd = FBTNodeIndex();
		}

		// store blackboard values from search start (can be changed by aux node removal/adding)
#if USE_BEHAVIORTREE_DEBUGGER
		StoreDebuggerBlackboard(SearchStartBlackboard);
#endif

		// start looking for next task
		while (TestNode && NextTask == NULL)
		{
			BT_SEARCHLOG(SearchData, Verbose, TEXT("Testing node: %s"), *UBehaviorTreeTypes::DescribeNodeHelper(TestNode));
			const int32 ChildBranchIdx = TestNode->FindChildToExecute(SearchData, NodeResult);
			const UBTNode* StoreNode = TestNode;

			if (SearchData.bPostponeSearch)
			{
				// break out of current search loop
				TestNode = NULL;
				bIsSearchValid = false;
			}
			else if (ChildBranchIdx == BTSpecialChild::ReturnToParent)
			{
				const UBTNode* ChildNode = TestNode;
				const UBTCompositeNode* PrevTestNode = TestNode;
				TestNode = TestNode->GetParentNode();

				// does it want to move up the tree?
				if (TestNode == NULL)
				{
					// special case for leaving instance: deactivate root manually
					PrevTestNode->OnNodeDeactivation(SearchData, NodeResult);

					// don't remove top instance from stack, so it could be looped
					if (ActiveInstanceIdx > 0)
					{
						StoreDebuggerSearchStep(InstanceStack[ActiveInstanceIdx].ActiveNode, ActiveInstanceIdx, NodeResult);
						StoreDebuggerRemovedInstance(ActiveInstanceIdx);
						InstanceStack[ActiveInstanceIdx].DeactivateNodes(SearchData, ActiveInstanceIdx);

						// store notify for later use if search is not reverted
						SearchData.PendingNotifies.Add(FBehaviorTreeSearchUpdateNotify(ActiveInstanceIdx, NodeResult));

						// and leave subtree
						ActiveInstanceIdx--;

						StoreDebuggerSearchStep(InstanceStack[ActiveInstanceIdx].ActiveNode, ActiveInstanceIdx, NodeResult);
						ChildNode = InstanceStack[ActiveInstanceIdx].ActiveNode;
						TestNode = ChildNode->GetParentNode();
					}
				}

				if (TestNode)
				{
					const bool bRequestedFromValidInstance = ActiveInstanceIdx <= ExecutionRequest.ExecuteInstanceIdx;
					TestNode->OnChildDeactivation(SearchData, *ChildNode, NodeResult, bRequestedFromValidInstance);
				}
			}
			else if (TestNode->Children.IsValidIndex(ChildBranchIdx))
			{
				// was new task found?
				NextTask = TestNode->Children[ChildBranchIdx].ChildTask;

				// or it wants to move down the tree?
				TestNode = TestNode->Children[ChildBranchIdx].ChildComposite;
			}

			// store after node deactivation had chance to modify result
			StoreDebuggerSearchStep(StoreNode, ActiveInstanceIdx, NodeResult);
		}

		// is search within requested bounds?
		if (NextTask)
		{
			const FBTNodeIndex NextTaskIdx(ActiveInstanceIdx, NextTask->GetExecutionIndex());
			bIsSearchValid = NextTaskIdx.TakesPriorityOver(ExecutionRequest.SearchEnd);
			
			// is new task is valid, but wants to ignore rerunning itself
			// check it's the same as active node (or any of active parallel tasks)
			if (bIsSearchValid && NextTask->ShouldIgnoreRestartSelf())
			{
				const bool bIsTaskRunning = InstanceStack[ActiveInstanceIdx].HasActiveNode(NextTaskIdx.ExecutionIndex);
				if (bIsTaskRunning)
				{
					BT_SEARCHLOG(SearchData, Verbose, TEXT("Task doesn't allow restart and it's already running! Discarding search."));
					bIsSearchValid = false;
				}
			}
		}

		// valid search - if search requires aborting current task and that abort happens to be latent
		// try to keep current (before search) state of tree until everything is ready for next execution
		// - observer changes will be applied just before starting new task (ProcessPendingExecution)
		// - memory needs to be updated as well, but this requires keeping another copy
		//   it's easier to just discard everything on first execution request and start new search when abort finishes

		if (!bIsSearchValid || SearchData.bPostponeSearch)
		{
			RollbackSearchChanges();

			UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("Search %s, reverted all changes."), !bIsSearchValid ? TEXT("is not valid") : TEXT("will be retried"));
		}

		SearchData.bSearchInProgress = false;
		// finish timer scope
	}

	if (!SearchData.bPostponeSearch)
	{
		// clear request accumulator
		ExecutionRequest = FBTNodeExecutionInfo();

		// unlock execution data, can get locked again if AbortCurrentTask starts any new requests
		PendingExecution.Unlock();

		if (bIsSearchValid)
		{
			// abort task if needed
			if (InstanceStack.Last().ActiveNodeType == EBTActiveNode::ActiveTask)
			{
				// prevent new execution requests for nodes inside the deactivated branch 
				// that may result from the aborted task.
				SearchData.bFilterOutRequestFromDeactivatedBranch = true;

				AbortCurrentTask();

				SearchData.bFilterOutRequestFromDeactivatedBranch = false;
			}

			// set next task to execute only when not lock for execution as everything has been cancelled/rollback
			if (!PendingExecution.IsLocked())
			{
				PendingExecution.NextTask = NextTask;
				PendingExecution.bOutOfNodes = (NextTask == NULL);
			}
		}

		ProcessPendingExecution();
	}
	else
	{
		// more important execution request was found
		// stop everything and search again in next tick
		ScheduleExecutionUpdate();
	}
}

void UBehaviorTreeComponent::ProcessPendingExecution()
{ 
	UE_VLOG(GetOwner(), LogBehaviorTree, VeryVerbose, TEXT("%hs"), __FUNCTION__);

	// can't continue if current task is still aborting
	if (bWaitingForLatentAborts || !PendingExecution.IsSet())
	{
		return;
	}

	FBTPendingExecutionInfo SavedInfo = PendingExecution;
	PendingExecution = FBTPendingExecutionInfo();

	// collect all aux nodes that have lower priority than new task
	// occurs when normal execution is forced to revisit lower priority nodes (e.g. loop decorator)
	const FBTNodeIndex NextTaskIdx = SavedInfo.NextTask ? FBTNodeIndex(ActiveInstanceIdx, SavedInfo.NextTask->GetExecutionIndex()) : FBTNodeIndex(0, 0);
	UnregisterAuxNodesUpTo(NextTaskIdx);

	// Suspending any all branch actions as it is impossible for decorators to have the right answer if they are in an executing branch or not.
	SuspendBranchActions(EBTBranchAction::All);

	// change aux nodes
	ApplySearchData(SavedInfo.NextTask);

	// make sure that we don't have any additional instances on stack
	if (InstanceStack.Num() > (ActiveInstanceIdx + 1))
	{
		for (int32 InstanceIndex = ActiveInstanceIdx + 1; InstanceIndex < InstanceStack.Num(); InstanceIndex++)
		{
			InstanceStack[InstanceIndex].Cleanup(*this, EBTMemoryClear::StoreSubtree);
		}

		InstanceStack.SetNum(ActiveInstanceIdx + 1);
	}

	// execute next task / notify out of nodes
	// validate active instance as well, execution can be delayed AND can have AbortCurrentTask call before using instance index
	if (SavedInfo.NextTask && InstanceStack.IsValidIndex(ActiveInstanceIdx))
	{
		// ResumeBranchActions() is done inside ExecuteTask after the active task is set but before we execute the task.
		ExecuteTask(SavedInfo.NextTask);
	}
	else
	{
		ResumeBranchActions();
		OnTreeFinished();
	}
}

void UBehaviorTreeComponent::RollbackSearchChanges()
{
	if (SearchData.RollbackInstanceIdx >= 0)
	{
		checkf(SearchData.RollbackInstanceIdx != FBTNodeIndex::InvalidIndex, TEXT("Index has used the InvalidIndex value!"));
		ActiveInstanceIdx = IntCastChecked<uint16>(SearchData.RollbackInstanceIdx);
		SearchData.DeactivatedBranchStart = SearchData.RollbackDeactivatedBranchStart;
		SearchData.DeactivatedBranchEnd = SearchData.RollbackDeactivatedBranchEnd;

		SearchData.RollbackInstanceIdx = INDEX_NONE;
		SearchData.RollbackDeactivatedBranchStart = FBTNodeIndex();
		SearchData.RollbackDeactivatedBranchEnd = FBTNodeIndex();

		if (SearchData.bPreserveActiveNodeMemoryOnRollback)
		{
			for (int32 Idx = 0; Idx < InstanceStack.Num(); Idx++)
			{
				FBehaviorTreeInstance& InstanceData = InstanceStack[Idx];
				FBehaviorTreeInstanceId& InstanceInfo = KnownInstances[InstanceData.InstanceIdIndex];

				const uint16 NodeMemorySize = InstanceData.ActiveNode ? InstanceData.ActiveNode->GetInstanceMemorySize() : 0;
				if (NodeMemorySize)
				{
					// copy over stored data in persistent, rollback is one time action and it won't be needed anymore
					const uint8* NodeMemory = InstanceData.ActiveNode->GetNodeMemory<uint8>(InstanceData);
					uint8* DestMemory = InstanceInfo.InstanceMemory.GetData() + InstanceData.ActiveNode->GetMemoryOffset();

					FMemory::Memcpy(DestMemory, NodeMemory, NodeMemorySize);
				}

				InstanceData.SetInstanceMemory(InstanceInfo.InstanceMemory);
			}
		}
		else
		{
			CopyInstanceMemoryFromPersistent();
		}

		// apply new observer changes
		ApplyDiscardedSearch();
	}
}

bool UBehaviorTreeComponent::DeactivateUpTo(const UBTCompositeNode* Node, uint16 NodeInstanceIdx, EBTNodeResult::Type& NodeResult, int32& OutLastDeactivatedChildIndex)
{
	const UBTNode* DeactivatedChild = InstanceStack[ActiveInstanceIdx].ActiveNode;
	bool bDeactivateRoot = true;

	if (DeactivatedChild == NULL && ActiveInstanceIdx > NodeInstanceIdx)
	{
		// use tree's root node if instance didn't activated itself yet
		DeactivatedChild = InstanceStack[ActiveInstanceIdx].RootNode;
		bDeactivateRoot = false;
	}

	while (DeactivatedChild)
	{
		const UBTCompositeNode* NotifyParent = DeactivatedChild->GetParentNode();
		if (NotifyParent)
		{
			OutLastDeactivatedChildIndex = NotifyParent->GetChildIndex(SearchData, *DeactivatedChild);
			const bool bRequestedFromValidInstance = ActiveInstanceIdx <= NodeInstanceIdx;
			NotifyParent->OnChildDeactivation(SearchData, OutLastDeactivatedChildIndex, NodeResult, bRequestedFromValidInstance);

			BT_SEARCHLOG(SearchData, Verbose, TEXT("Deactivate node: %s"), *UBehaviorTreeTypes::DescribeNodeHelper(DeactivatedChild));
			StoreDebuggerSearchStep(DeactivatedChild, ActiveInstanceIdx, NodeResult);
			DeactivatedChild = NotifyParent;
		}
		else
		{
			// special case for leaving instance: deactivate root manually
			if (bDeactivateRoot)
			{
				InstanceStack[ActiveInstanceIdx].RootNode->OnNodeDeactivation(SearchData, NodeResult);
			}

			BT_SEARCHLOG(SearchData, Verbose, TEXT("%s node: %s [leave subtree]"),
				bDeactivateRoot ? TEXT("Deactivate") : TEXT("Skip over"),
				*UBehaviorTreeTypes::DescribeNodeHelper(InstanceStack[ActiveInstanceIdx].RootNode));

			// clear flag, it's valid only for newest instance
			bDeactivateRoot = true;

			// shouldn't happen, but it's better to have built in failsafe just in case
			if (ActiveInstanceIdx == 0)
			{
				BT_SEARCHLOG(SearchData, Error, TEXT("Execution path does NOT contain common parent node, restarting tree! AI:%s"),
					*GetNameSafe(SearchData.OwnerComp.GetOwner()));

				RestartTree();
				return false;
			}

			// store notify for later use if search is not reverted
			SearchData.PendingNotifies.Add(FBehaviorTreeSearchUpdateNotify(ActiveInstanceIdx, NodeResult));

			ActiveInstanceIdx--;
			DeactivatedChild = InstanceStack[ActiveInstanceIdx].ActiveNode;
		}

		if (DeactivatedChild == Node)
		{
			break;
		}
	}

	return true;
}

void UBehaviorTreeComponent::UnregisterAuxNodesUpTo(const FBTNodeIndex& Index)
{
	for (int32 InstanceIndex = 0; InstanceIndex < InstanceStack.Num(); InstanceIndex++)
	{
		FBehaviorTreeInstance& InstanceInfo = InstanceStack[InstanceIndex];
		for (const UBTAuxiliaryNode* AuxNode : InstanceInfo.GetActiveAuxNodes())
		{
			// Could safely use static_cast() here but its a bit safer using IntCastChecked() for future code changes.
			const uint16 InstanceIndexUint16 = IntCastChecked<uint16>(InstanceIndex);
			FBTNodeIndex AuxIdx(InstanceIndexUint16, AuxNode->GetExecutionIndex());
			if (Index.TakesPriorityOver(AuxIdx))
			{
				SearchData.AddUniqueUpdate(FBehaviorTreeSearchUpdate(AuxNode, InstanceIndexUint16, EBTNodeUpdateMode::Remove));
			}
		}
	}
}

void UBehaviorTreeComponent::UnregisterAuxNodesInRange(const FBTNodeIndex& FromIndex, const FBTNodeIndex& ToIndex)
{
	for (int32 InstanceIndex = 0; InstanceIndex < InstanceStack.Num(); InstanceIndex++)
	{
		FBehaviorTreeInstance& InstanceInfo = InstanceStack[InstanceIndex];
		for (const UBTAuxiliaryNode* AuxNode : InstanceInfo.GetActiveAuxNodes())
		{
			// Could safely use static_cast() here but its a bit safer using IntCastChecked() for future code changes.
			const uint16 InstanceIndexUint16 = IntCastChecked<uint16>(InstanceIndex);
			FBTNodeIndex AuxIdx(InstanceIndexUint16, AuxNode->GetExecutionIndex());
			if (FromIndex.TakesPriorityOver(AuxIdx) && AuxIdx.TakesPriorityOver(ToIndex))
			{
				SearchData.AddUniqueUpdate(FBehaviorTreeSearchUpdate(AuxNode, InstanceIndexUint16, EBTNodeUpdateMode::Remove));
			}
		}
	}
}

void UBehaviorTreeComponent::UnregisterAuxNodesInBranch(const UBTCompositeNode* Node, bool bApplyImmediately)
{
	const int32 InstanceIdx = FindInstanceContainingNode(Node);
	if (InstanceIdx != INDEX_NONE)
	{
		check(Node);

		checkf(InstanceIdx != FBTNodeIndex::InvalidIndex, TEXT("Index has used the InvalidIndex value!"));
		const uint16 InstanceIdxUint16 = IntCastChecked<uint16>(InstanceIdx);

		TArray<FBehaviorTreeSearchUpdate> UpdateListCopy;
		if (bApplyImmediately)
		{
			UpdateListCopy = SearchData.PendingUpdates;
			SearchData.PendingUpdates.Reset();
		}

		const FBTNodeIndex FromIndex(InstanceIdxUint16, Node->GetExecutionIndex());
		const FBTNodeIndex ToIndex(InstanceIdxUint16, Node->GetLastExecutionIndex());
		UnregisterAuxNodesInRange(FromIndex, ToIndex);

		if (bApplyImmediately)
		{
			FBTSuspendBranchActionsScoped ScopedSuspend(*this, EBTBranchAction::All);
			ApplyAllSearchUpdates(/*UpdateList*/ SearchData.PendingUpdates, /*NewNodeExecutionIndex*/ 0, /*bDoPostUpdate*/ false);
			SearchData.PendingUpdates = UpdateListCopy;
		}
	}
}

void UBehaviorTreeComponent::ExecuteTask(UBTTaskNode* TaskNode)
{
	SCOPE_CYCLE_COUNTER(STAT_AI_BehaviorTree_ExecutionTime);

	// We expect that there should be valid instances on the stack
	if (!ensure(InstanceStack.IsValidIndex(ActiveInstanceIdx)))
	{
		return;
	}

	FBehaviorTreeInstance& ActiveInstance = InstanceStack[ActiveInstanceIdx];

	// task service activation is not part of search update (although deactivation is, through DeactivateUpTo), start them before execution
	for (UBTService* ServiceNode : TaskNode->Services)
	{
		uint8* NodeMemory = (uint8*)ServiceNode->GetNodeMemory<uint8>(ActiveInstance);

		ActiveInstance.AddToActiveAuxNodes(*this, ServiceNode);

		UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("Activating task service: %s"), *UBehaviorTreeTypes::DescribeNodeHelper(ServiceNode));
		ServiceNode->WrappedOnBecomeRelevant(*this, NodeMemory);
	}

	// Services were already ticked for this frame, need to tick the new ones. 
	for (UBTService* ServiceNode : TaskNode->Services)
	{
		uint8* NodeMemory = (uint8*)ServiceNode->GetNodeMemory<uint8>(ActiveInstance);

		// We do not care about the next needed DeltaTime, it will be recalculated in the tick later.
		float NextNeededDeltaTime = 0.0f;
		ServiceNode->WrappedTickNode(*this, NodeMemory, CurrentFrameDeltaTime, NextNeededDeltaTime);
	}

	ActiveInstance.ActiveNode = TaskNode;
	ActiveInstance.ActiveNodeType = EBTActiveNode::ActiveTask;

	// make a snapshot for debugger
	StoreDebuggerExecutionStep(EBTExecutionSnap::Regular);

	UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("Execute task: %s"), *UBehaviorTreeTypes::DescribeNodeHelper(TaskNode));

	// store instance before execution, it could result in pushing a subtree
	uint16 InstanceIdx = ActiveInstanceIdx;

	EBTNodeResult::Type TaskResult;
	{
		SCOPE_CYCLE_UOBJECT(TaskNode, TaskNode);
		uint8* NodeMemory = (uint8*)(TaskNode->GetNodeMemory<uint8>(ActiveInstance));
		TaskResult = TaskNode->WrappedExecuteTask(*this, NodeMemory);
	}

	// pass task finished if wasn't already notified (FinishLatentTask)
	const UBTNode* ActiveNodeAfterExecution = GetActiveNode();
	if (ActiveNodeAfterExecution == TaskNode)
	{
		// update task's runtime values after it had a chance to initialize memory
		UpdateDebuggerAfterExecution(TaskNode, InstanceIdx);

		OnTaskFinished(TaskNode, TaskResult);
	}

	// It is now ok to resume any branch actions as the new active instance is set.
	// Before that, decorators evaluating the IsExecutingBranch would be wrong.
	ResumeBranchActions();
}

void UBehaviorTreeComponent::AbortCurrentTask()
{
	const int32 CurrentInstanceIdx = InstanceStack.Num() - 1;
	FBehaviorTreeInstance& CurrentInstance = InstanceStack[CurrentInstanceIdx];
	CurrentInstance.ActiveNodeType = EBTActiveNode::AbortingTask;

	UBTTaskNode* CurrentTask = (UBTTaskNode*)CurrentInstance.ActiveNode;

	// remove all observers before requesting abort
	UnregisterMessageObserversFrom(CurrentTask);

	// protect memory of this task from rollbacks
	// at this point, invalid search rollback already happened
	// only reason to do the rollback is restoring tree state during abort for accumulated requests
	// but this task needs to remain unchanged: it's still aborting and internal memory can be modified on AbortTask call
	SearchData.bPreserveActiveNodeMemoryOnRollback = true;

	UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("Abort task: %s"), *UBehaviorTreeTypes::DescribeNodeHelper(CurrentTask));

	// abort task using current state of tree
	uint8* NodeMemory = (uint8*)(CurrentTask->GetNodeMemory<uint8>(CurrentInstance));
	EBTNodeResult::Type TaskResult = CurrentTask->WrappedAbortTask(*this, NodeMemory);

	// pass task finished if wasn't already notified (FinishLatentAbort)
	if (CurrentInstance.ActiveNodeType == EBTActiveNode::AbortingTask &&
		CurrentInstanceIdx == (InstanceStack.Num() - 1))
	{
		OnTaskFinished(CurrentTask, TaskResult);
	}
}

void UBehaviorTreeComponent::RegisterMessageObserver(const UBTTaskNode* TaskNode, FName MessageType)
{
	if (TaskNode)
	{
		FBTNodeIndex NodeIdx;
		NodeIdx.ExecutionIndex = TaskNode->GetExecutionIndex();

		const int32 InstanceIndex = InstanceStack.Num() - 1;
		NodeIdx.InstanceIndex = IntCastChecked<uint16>(InstanceIndex);

		TaskMessageObservers.Add(NodeIdx,
			FAIMessageObserver::Create(this, MessageType, FOnAIMessage::CreateUObject(const_cast<UBTTaskNode*>(TaskNode), &UBTTaskNode::ReceivedMessage))
			);

		UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("Message[%s] observer added for %s"),
			*MessageType.ToString(), *UBehaviorTreeTypes::DescribeNodeHelper(TaskNode));
	}
}

void UBehaviorTreeComponent::RegisterMessageObserver(const UBTTaskNode* TaskNode, FName MessageType, FAIRequestID RequestID)
{
	if (TaskNode)
	{
		FBTNodeIndex NodeIdx;
		NodeIdx.ExecutionIndex = TaskNode->GetExecutionIndex();

		const int32 InstanceIndex = InstanceStack.Num() - 1;
		NodeIdx.InstanceIndex = IntCastChecked<uint16>(InstanceIndex);

		TaskMessageObservers.Add(NodeIdx,
			FAIMessageObserver::Create(this, MessageType, RequestID, FOnAIMessage::CreateUObject(const_cast<UBTTaskNode*>(TaskNode), &UBTTaskNode::ReceivedMessage))
			);

		UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("Message[%s:%d] observer added for %s"),
			*MessageType.ToString(), RequestID.GetID(), *UBehaviorTreeTypes::DescribeNodeHelper(TaskNode));
	}
}

void UBehaviorTreeComponent::UnregisterMessageObserversFrom(const FBTNodeIndex& TaskIdx)
{
	const int32 NumRemoved = TaskMessageObservers.Remove(TaskIdx);
	if (NumRemoved)
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("Message observers removed for task[%d:%d] (num:%d)"),
			TaskIdx.InstanceIndex, TaskIdx.ExecutionIndex, NumRemoved);
	}
}

void UBehaviorTreeComponent::UnregisterMessageObserversFrom(const UBTTaskNode* TaskNode)
{
	if (TaskNode && InstanceStack.Num())
	{
		const FBehaviorTreeInstance& ActiveInstance = InstanceStack.Last();

		FBTNodeIndex NodeIdx;
		NodeIdx.ExecutionIndex = TaskNode->GetExecutionIndex();

		const int32 InstanceContainingNodeIdx = FindInstanceContainingNode(TaskNode);
		
		// InstanceContainingNodeIdx could be INDEX_NONE which means we can't use IntCastChecked() here
		checkf(InstanceContainingNodeIdx != FBTNodeIndex::InvalidIndex, TEXT("Index has used the InvalidIndex value!"));
		checkf(InstanceContainingNodeIdx <= MAX_uint16, TEXT("Narrowing conversion causing loss of data"));

		NodeIdx.InstanceIndex = static_cast<uint16>(InstanceContainingNodeIdx);
		
		UnregisterMessageObserversFrom(NodeIdx);
	}
}

void UBehaviorTreeComponent::RegisterParallelTask(const UBTTaskNode* TaskNode)
{
	if (InstanceStack.IsValidIndex(ActiveInstanceIdx))
	{
		FBehaviorTreeInstance& InstanceInfo = InstanceStack[ActiveInstanceIdx];
		InstanceInfo.AddToParallelTasks(FBehaviorTreeParallelTask(TaskNode, EBTTaskStatus::Active));

		UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("Parallel task: %s added to active list"),
			*UBehaviorTreeTypes::DescribeNodeHelper(TaskNode));

		if (InstanceInfo.ActiveNode == TaskNode)
		{
			// switch to inactive state, so it could start background tree
			InstanceInfo.ActiveNodeType = EBTActiveNode::InactiveTask;
		}
	}
}

void UBehaviorTreeComponent::UnregisterParallelTask(const UBTTaskNode* TaskNode, uint16 InstanceIdx)
{
	bool bShouldUpdate = false;
	if (InstanceStack.IsValidIndex(InstanceIdx))
	{
		FBehaviorTreeInstance& InstanceInfo = InstanceStack[InstanceIdx];
		for (int32 TaskIndex = InstanceInfo.GetParallelTasks().Num() - 1; TaskIndex >= 0; TaskIndex--)
		{
			if (InstanceInfo.GetParallelTasks()[TaskIndex].TaskNode == TaskNode)
			{
				UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("Parallel task: %s removed from active list"),
					*UBehaviorTreeTypes::DescribeNodeHelper(TaskNode));

				InstanceInfo.RemoveParallelTaskAt(TaskIndex);
				bShouldUpdate = true;
				break;
			}
		}
	}
}

bool UBehaviorTreeComponent::TrackPendingLatentAborts()
{
	// nothing to track if we are not currently waiting for latent aborts
	if (!bWaitingForLatentAborts)
	{
		return false;
	}

	// update our internal flag	
	bWaitingForLatentAborts = HasActiveLatentAborts();

	// return true if we are no longer waiting (at this point we know that we were previously waiting on latent abortes)
	return !bWaitingForLatentAborts;
}

void UBehaviorTreeComponent::TrackNewLatentAborts()
{
	// already waiting for latent aborts, no need to look for new ones 
	if (bWaitingForLatentAborts)
	{
		return;
	}

	bWaitingForLatentAborts = HasActiveLatentAborts();
}

bool UBehaviorTreeComponent::HasActiveLatentAborts() const
{
	bool bHasActiveLatentAborts = InstanceStack.Num() ? (InstanceStack.Last().ActiveNodeType == EBTActiveNode::AbortingTask) : false;

	for (int32 InstanceIndex = 0; InstanceIndex < InstanceStack.Num() && !bHasActiveLatentAborts; InstanceIndex++)
	{
		const FBehaviorTreeInstance& InstanceInfo = InstanceStack[InstanceIndex];
		for (const FBehaviorTreeParallelTask& ParallelInfo : InstanceInfo.GetParallelTasks())
		{
			if (ParallelInfo.Status == EBTTaskStatus::Aborting)
			{
				bHasActiveLatentAborts = true;
				break;
			}
		}
	}

	return bHasActiveLatentAborts;
}

bool UBehaviorTreeComponent::PushInstance(UBehaviorTree& TreeAsset)
{
	// check if blackboard class match
	if (TreeAsset.BlackboardAsset && BlackboardComp && !BlackboardComp->IsCompatibleWith(TreeAsset.BlackboardAsset))
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Warning, TEXT("Failed to execute tree %s: blackboard %s is not compatibile with current: %s!"),
			*TreeAsset.GetName(), *GetNameSafe(TreeAsset.BlackboardAsset), *GetNameSafe(BlackboardComp->GetBlackboardAsset()));

		return false;
	}

	UBehaviorTreeManager* BTManager = UBehaviorTreeManager::GetCurrent(GetWorld());
	if (BTManager == NULL)
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Warning, TEXT("Failed to execute tree %s: behavior tree manager not found!"), *TreeAsset.GetName());
		return false;
	}

	// check if parent node allows it
	const UBTNode* ActiveNode = GetActiveNode();
	const UBTCompositeNode* ActiveParent = ActiveNode ? ActiveNode->GetParentNode() : NULL;
	if (ActiveParent)
	{
		uint8* ParentMemory = GetNodeMemory((UBTNode*)ActiveParent, InstanceStack.Num() - 1);
		int32 ChildIdx = ActiveNode ? ActiveParent->GetChildIndex(*ActiveNode) : INDEX_NONE;

		const bool bIsAllowed = ActiveParent->CanPushSubtree(*this, ParentMemory, ChildIdx);
		if (!bIsAllowed)
		{
			UE_VLOG(GetOwner(), LogBehaviorTree, Warning, TEXT("Failed to execute tree %s: parent of active node does not allow it! (%s)"),
				*TreeAsset.GetName(), *UBehaviorTreeTypes::DescribeNodeHelper(ActiveParent));
			return false;
		}
	}

	UBTCompositeNode* RootNode = NULL;
	uint16 InstanceMemorySize = 0;

	const bool bLoaded = BTManager->LoadTree(TreeAsset, RootNode, InstanceMemorySize);
	if (bLoaded)
	{
		FBehaviorTreeInstance& NewInstance = InstanceStack.AddDefaulted_GetRef();
		NewInstance.InstanceIdIndex = UpdateInstanceId(&TreeAsset, ActiveNode, InstanceStack.Num() - 1);
		NewInstance.RootNode = RootNode;
		NewInstance.ActiveNode = NULL;
		NewInstance.ActiveNodeType = EBTActiveNode::Composite;

		// initialize memory and node instances
		FBehaviorTreeInstanceId& InstanceInfo = KnownInstances[NewInstance.InstanceIdIndex];
		int32 NodeInstanceIndex = InstanceInfo.FirstNodeInstance;
		const bool bFirstTime = (InstanceInfo.InstanceMemory.Num() != InstanceMemorySize);
		if (bFirstTime)
		{
			InstanceInfo.InstanceMemory.AddZeroed(InstanceMemorySize);
			InstanceInfo.RootNode = RootNode;
		}

		NewInstance.SetInstanceMemory(InstanceInfo.InstanceMemory);
		NewInstance.Initialize(*this, *RootNode, NodeInstanceIndex, bFirstTime ? EBTMemoryInit::Initialize : EBTMemoryInit::RestoreSubtree);


		const int32 InstanceIndex = InstanceStack.Num() - 1;
		ActiveInstanceIdx = IntCastChecked<uint16>(InstanceIndex);

		// start root level services now (they won't be removed on looping tree anyway)
		for (int32 ServiceIndex = 0; ServiceIndex < RootNode->Services.Num(); ServiceIndex++)
		{
			UBTService* ServiceNode = RootNode->Services[ServiceIndex];
			uint8* NodeMemory = (uint8*)ServiceNode->GetNodeMemory<uint8>(InstanceStack[ActiveInstanceIdx]);

			// send initial on search start events in case someone is using them for init logic
			ServiceNode->NotifyParentActivation(SearchData);

			InstanceStack[ActiveInstanceIdx].AddToActiveAuxNodes(*this,ServiceNode);
			ServiceNode->WrappedOnBecomeRelevant(*this, NodeMemory);
		}

		FBehaviorTreeDelegates::OnTreeStarted.Broadcast(*this, TreeAsset);

		if ((SuspendedBranchActions & EBTBranchAction::SubTreeEvaluate) != EBTBranchAction::None)
		{
			UE_VLOG(GetOwner(), LogBehaviorTree, Verbose, TEXT("Evaluate sub tree(%s) root(%s) queued up"), *TreeAsset.GetName(), *UBehaviorTreeTypes::DescribeNodeHelper(RootNode));
			PendingBranchActionRequests.Emplace(RootNode, EBTBranchAction::SubTreeEvaluate);
			return true;
		}

		// start new task
		RequestExecution(RootNode, ActiveInstanceIdx, RootNode, 0, EBTNodeResult::InProgress);
		return true;
	}

	return false;
}

uint8 UBehaviorTreeComponent::UpdateInstanceId(UBehaviorTree* TreeAsset, const UBTNode* OriginNode, int32 OriginInstanceIdx)
{
	FBehaviorTreeInstanceId InstanceId;
	InstanceId.TreeAsset = TreeAsset;

	// build path from origin node
	{
		const uint16 ExecutionIndex = OriginNode ? OriginNode->GetExecutionIndex() : MAX_uint16;
		InstanceId.Path.Add(ExecutionIndex);
	}

	for (int32 InstanceIndex = OriginInstanceIdx - 1; InstanceIndex >= 0; InstanceIndex--)
	{
		const uint16 ExecutionIndex = InstanceStack[InstanceIndex].ActiveNode ? InstanceStack[InstanceIndex].ActiveNode->GetExecutionIndex() : MAX_uint16;
		InstanceId.Path.Add(ExecutionIndex);
	}

	// try to find matching existing Id
	for (int32 InstanceIndex = 0; InstanceIndex < KnownInstances.Num(); InstanceIndex++)
	{
		if (KnownInstances[InstanceIndex] == InstanceId)
		{
			return IntCastChecked<uint8>(InstanceIndex);
		}
	}

	// add new one
	InstanceId.FirstNodeInstance = NodeInstances.Num();

	const int32 NewIndex = KnownInstances.Add(InstanceId);
	return IntCastChecked<uint8>(NewIndex);
}

int32 UBehaviorTreeComponent::FindInstanceContainingNode(const UBTNode* Node) const
{
	int32 InstanceIdx = INDEX_NONE;

	const UBTNode* TemplateNode = FindTemplateNode(Node);
	if (TemplateNode && InstanceStack.Num())
	{
		if (InstanceStack[ActiveInstanceIdx].ActiveNode != TemplateNode)
		{
			const UBTNode* RootNode = TemplateNode;
			while (RootNode->GetParentNode())
			{
				RootNode = RootNode->GetParentNode();
			}

			for (int32 InstanceIndex = 0; InstanceIndex < InstanceStack.Num(); InstanceIndex++)
			{
				if (InstanceStack[InstanceIndex].RootNode == RootNode)
				{
					InstanceIdx = InstanceIndex;
					break;
				}
			}
		}
		else
		{
			InstanceIdx = ActiveInstanceIdx;
		}
	}

	return InstanceIdx;
}

UBTNode* UBehaviorTreeComponent::FindTemplateNode(const UBTNode* Node) const
{
	if (Node == NULL || !Node->IsInstanced() || Node->GetParentNode() == NULL)
	{
		return (UBTNode*)Node;
	}

	UBTCompositeNode* ParentNode = Node->GetParentNode();
	for (int32 ChildIndex = 0; ChildIndex < ParentNode->Children.Num(); ChildIndex++)
	{
		FBTCompositeChild& ChildInfo = ParentNode->Children[ChildIndex];

		if (ChildInfo.ChildTask)
		{
			if (ChildInfo.ChildTask->GetExecutionIndex() == Node->GetExecutionIndex())
			{
				return ChildInfo.ChildTask;
			}

			for (int32 ServiceIndex = 0; ServiceIndex < ChildInfo.ChildTask->Services.Num(); ServiceIndex++)
			{
				if (ChildInfo.ChildTask->Services[ServiceIndex]->GetExecutionIndex() == Node->GetExecutionIndex())
				{
					return ChildInfo.ChildTask->Services[ServiceIndex];
				}
			}
		}

		for (int32 DecoratorIndex = 0; DecoratorIndex < ChildInfo.Decorators.Num(); DecoratorIndex++)
		{
			if (ChildInfo.Decorators[DecoratorIndex]->GetExecutionIndex() == Node->GetExecutionIndex())
			{
				return ChildInfo.Decorators[DecoratorIndex];
			}
		}
	}

	for (int32 ServiceIndex = 0; ServiceIndex < ParentNode->Services.Num(); ServiceIndex++)
	{
		if (ParentNode->Services[ServiceIndex]->GetExecutionIndex() == Node->GetExecutionIndex())
		{
			return ParentNode->Services[ServiceIndex];
		}
	}

	return NULL;
}

uint8* UBehaviorTreeComponent::GetNodeMemory(const UBTNode* Node, int32 InstanceIdx) const
{
	uint8* NodeMemory = InstanceStack.IsValidIndex(InstanceIdx) ? (uint8*)Node->GetNodeMemory<uint8>(InstanceStack[InstanceIdx]) : nullptr;
	return NodeMemory ? NodeMemory : GetNodeMemoryAux(Node, InstanceIdx);
}

uint8* UBehaviorTreeComponent::GetNodeMemoryAux(const UBTNode* Node, int32 InstanceIdx) const
{
	return nullptr;
}

void UBehaviorTreeComponent::RemoveAllInstances()
{
	ensureMsgf(SuspendedBranchActions == EBTBranchAction::None, TEXT("Cannot remove all instances if some branch actions are suspended!"));

	if (InstanceStack.Num())
	{
		StopTree(EBTStopMode::Forced);
	}

	if (!ensureMsgf(InstanceStack.Num() == 0, TEXT("Queued stop could not cleanup the instance stack, CurrentRoot(%s), AssetToStart(%s)"), *GetNameSafe(GetRootTree()), *GetNameSafe(TreeStartInfo.Asset)))
	{
		InstanceStack.Reset();
	}

	FBehaviorTreeInstance DummyInstance;
	for (int32 Idx = 0; Idx < KnownInstances.Num(); Idx++)
	{
		const FBehaviorTreeInstanceId& Info = KnownInstances[Idx];
		if (Info.InstanceMemory.Num())
		{
			// instance memory will be removed on Cleanup in EBTMemoryClear::Destroy mode
			// prevent from calling it multiple times - StopTree does it for current InstanceStack
			DummyInstance.SetInstanceMemory(Info.InstanceMemory);
			DummyInstance.InstanceIdIndex = IntCastChecked<uint8>(Idx);
			DummyInstance.RootNode = Info.RootNode;

			DummyInstance.Cleanup(*this, EBTMemoryClear::Destroy);
		}
	}

	KnownInstances.Reset();
	NodeInstances.Reset();
}

void UBehaviorTreeComponent::CopyInstanceMemoryToPersistent()
{
	for (int32 InstanceIndex = 0; InstanceIndex < InstanceStack.Num(); InstanceIndex++)
	{
		const FBehaviorTreeInstance& InstanceData = InstanceStack[InstanceIndex];
		FBehaviorTreeInstanceId& InstanceInfo = KnownInstances[InstanceData.InstanceIdIndex];

		InstanceInfo.InstanceMemory = InstanceData.GetInstanceMemory();
	}
}

void UBehaviorTreeComponent::CopyInstanceMemoryFromPersistent()
{
	for (int32 InstanceIndex = 0; InstanceIndex < InstanceStack.Num(); InstanceIndex++)
	{
		FBehaviorTreeInstance& InstanceData = InstanceStack[InstanceIndex];
		const FBehaviorTreeInstanceId& InstanceInfo = KnownInstances[InstanceData.InstanceIdIndex];

		InstanceData.SetInstanceMemory(InstanceInfo.InstanceMemory);
	}
}

FString UBehaviorTreeComponent::GetDebugInfoString() const 
{ 
	FString DebugInfo;
	for (int32 InstanceIndex = 0; InstanceIndex < InstanceStack.Num(); InstanceIndex++)
	{
		const FBehaviorTreeInstance& InstanceData = InstanceStack[InstanceIndex];
		const FBehaviorTreeInstanceId& InstanceInfo = KnownInstances[InstanceData.InstanceIdIndex];
		DebugInfo += FString::Printf(TEXT("Behavior tree: %s\n"), *GetNameSafe(InstanceInfo.TreeAsset));

		UBTNode* Node = InstanceData.ActiveNode;
		FString NodeTrace;

		while (Node)
		{
			uint8* NodeMemory = (uint8*)(Node->GetNodeMemory<uint8>(InstanceData));
			NodeTrace = FString::Printf(TEXT("  %s\n"), *Node->GetRuntimeDescription(*this, NodeMemory, EBTDescriptionVerbosity::Basic)) + NodeTrace;
			Node = Node->GetParentNode();
		}

		DebugInfo += NodeTrace;
	}

	return DebugInfo;
}

FString UBehaviorTreeComponent::DescribeActiveTasks() const
{
	FString ActiveTask("None");
	if (InstanceStack.Num())
	{
		const FBehaviorTreeInstance& LastInstance = InstanceStack.Last();
		if (LastInstance.ActiveNodeType == EBTActiveNode::ActiveTask)
		{
			ActiveTask = UBehaviorTreeTypes::DescribeNodeHelper(LastInstance.ActiveNode);
		}
	}

	FString ParallelTasks;
	for (int32 InstanceIndex = 0; InstanceIndex < InstanceStack.Num(); InstanceIndex++)
	{
		const FBehaviorTreeInstance& InstanceInfo = InstanceStack[InstanceIndex];
		for (const FBehaviorTreeParallelTask& ParallelInfo : InstanceInfo.GetParallelTasks())
		{
			if (ParallelInfo.Status == EBTTaskStatus::Active)
			{
				ParallelTasks += UBehaviorTreeTypes::DescribeNodeHelper(ParallelInfo.TaskNode);
				ParallelTasks += TEXT(", ");
			}
		}
	}

	if (ParallelTasks.Len() > 0)
	{
		ActiveTask += TEXT(" (");
		ActiveTask += ParallelTasks.LeftChop(2);
		ActiveTask += TEXT(')');
	}

	return ActiveTask;
}

FString UBehaviorTreeComponent::DescribeActiveTrees() const
{
	FString Assets;
	for (int32 InstanceIndex = 0; InstanceIndex < InstanceStack.Num(); InstanceIndex++)
	{
		const FBehaviorTreeInstanceId& InstanceInfo = KnownInstances[InstanceStack[InstanceIndex].InstanceIdIndex];
		Assets += InstanceInfo.TreeAsset->GetName();
		Assets += TEXT(", ");
	}

	return Assets.Len() ? Assets.LeftChop(2) : TEXT("None");
}

double UBehaviorTreeComponent::GetTagCooldownEndTime(FGameplayTag CooldownTag) const
{
	const double CooldownEndTime = CooldownTagsMap.FindRef(CooldownTag);
	return CooldownEndTime;
}

void UBehaviorTreeComponent::AddCooldownTagDuration(FGameplayTag CooldownTag, float CooldownDuration, bool bAddToExistingDuration)
{
	if (CooldownTag.IsValid())
	{
		double* CurrentEndTime = CooldownTagsMap.Find(CooldownTag);

		// If we are supposed to add to an existing duration, do that, otherwise we set a new value.
		if (bAddToExistingDuration && (CurrentEndTime != nullptr))
		{
			*CurrentEndTime += CooldownDuration;
		}
		else
		{
			CooldownTagsMap.Add(CooldownTag, (GetWorld()->GetTimeSeconds() + CooldownDuration));
		}
	}
}

void UBehaviorTreeComponent::SetDynamicSubtree(FGameplayTag InjectTag, UBehaviorTree* BehaviorAsset)
{
	SetDynamicSubtree(InjectTag, BehaviorAsset, nullptr);
}

void UBehaviorTreeComponent::SetDynamicSubtree(FGameplayTag InjectTag, UBehaviorTree* BehaviorAsset, UBTCompositeNode* OptionalStartingNode)
{
	bool bInjected = false;
	auto ReplaceInjectedBehaviorInTask = [InjectTag, BehaviorAsset, this, &bInjected](UBTTaskNode& TaskNode, const FBehaviorTreeInstance& InstanceInfo, int32 InstanceIndex)
		{
			UBTTask_RunBehaviorDynamic* SubtreeTask = Cast<UBTTask_RunBehaviorDynamic>(&TaskNode);
			if (SubtreeTask && SubtreeTask->HasMatchingTag(InjectTag))
			{
				const uint8* NodeMemory = SubtreeTask->GetNodeMemory<uint8>(InstanceInfo);
				if (UBTTask_RunBehaviorDynamic* InstancedNode = Cast<UBTTask_RunBehaviorDynamic>(SubtreeTask->GetNodeInstance(*this, (uint8*)NodeMemory)))
				{
					const bool bAssetChanged = InstancedNode->SetBehaviorAsset(BehaviorAsset);
					if (bAssetChanged)
					{
						UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("Replaced subtree in %s with %s (tag: %s)"),
							*UBehaviorTreeTypes::DescribeNodeHelper(SubtreeTask), *GetNameSafe(BehaviorAsset), *InjectTag.ToString());
						bInjected = true;
						if (InstanceInfo.ActiveNodeType == EBTActiveNode::ActiveTask && SubtreeTask == InstanceInfo.ActiveNode)
						{
							UBTCompositeNode* RestartNode = SubtreeTask->GetParentNode();
							int32 RestartChildIdx = RestartNode->GetChildIndex(*SubtreeTask);
							RequestExecution(RestartNode, InstanceIndex, SubtreeTask, RestartChildIdx, EBTNodeResult::Aborted);
						}
					}
				}
			}
		};

	if (OptionalStartingNode)
	{
		ForEachChildTask(*OptionalStartingNode, FindInstanceContainingNode(OptionalStartingNode), ReplaceInjectedBehaviorInTask);
	}
	else
	{
		ForEachChildTask(ReplaceInjectedBehaviorInTask);
	}

	if (!bInjected)
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Log, TEXT("Failed to inject subtree %s at tag %s"), *GetNameSafe(BehaviorAsset), *InjectTag.ToString());
	}
}

void UBehaviorTreeComponent::ForEachChildTask(TFunctionRef<void(class UBTTaskNode&, const FBehaviorTreeInstance&, int32 InstanceIndex)> Functor)
{
	for (int32 InstanceIndex = 0; InstanceIndex < InstanceStack.Num(); InstanceIndex++)
	{
		const FBehaviorTreeInstance& InstanceInfo = InstanceStack[InstanceIndex];
		ForEachChildTask(*InstanceInfo.RootNode, InstanceIndex, Functor);
	}
}

void UBehaviorTreeComponent::ForEachChildTask(UBTCompositeNode& StartNode, int32 InstanceIndex, TFunctionRef<void(UBTTaskNode&, const FBehaviorTreeInstance&, int32 InstanceIndex)> Functor)
{
	if (!InstanceStack.IsValidIndex(InstanceIndex))
	{
		return;
	}

	for (int32 Idx = 0; Idx < StartNode.Children.Num(); Idx++)
	{
		const FBTCompositeChild& ChildInfo = StartNode.Children[Idx];
		if (ChildInfo.ChildComposite)
		{
			ForEachChildTask(*ChildInfo.ChildComposite, InstanceIndex, Functor);
		}
		else
		{
			if (ChildInfo.ChildTask)
			{
				Functor(*ChildInfo.ChildTask, InstanceStack[InstanceIndex], InstanceIndex);
			}
		}
	}
}

#if ENABLE_VISUAL_LOG
void UBehaviorTreeComponent::DescribeSelfToVisLog(FVisualLogEntry* Snapshot) const
{
	if (!IsValidChecked(this))
	{
		return;
	}
	
	Super::DescribeSelfToVisLog(Snapshot);

	for (int32 InstanceIndex = 0; InstanceIndex < InstanceStack.Num(); InstanceIndex++)
	{
		const FBehaviorTreeInstance& InstanceInfo = InstanceStack[InstanceIndex];
		const FBehaviorTreeInstanceId& InstanceId = KnownInstances[InstanceInfo.InstanceIdIndex];

		FVisualLogStatusCategory StatusCategory;
		StatusCategory.Category = FString::Printf(TEXT("BehaviorTree %d (asset: %s)"), InstanceIndex, *GetNameSafe(InstanceId.TreeAsset));

		if (InstanceInfo.GetActiveAuxNodes().Num() > 0)
		{
			FString ObserversDesc;
			for (const UBTAuxiliaryNode* AuxNode : InstanceInfo.GetActiveAuxNodes())
			{
				ObserversDesc += FString::Printf(TEXT("%d. %s: %s\n"), AuxNode->GetExecutionIndex(), *AuxNode->GetNodeName(), *AuxNode->GetStaticDescription());
			}
			StatusCategory.Add(TEXT("Observers"), ObserversDesc);
		}

		TArray<FString> Descriptions;
		UBTNode* Node = InstanceInfo.ActiveNode;
		while (Node)
		{
			uint8* NodeMemory = (uint8*)(Node->GetNodeMemory<uint8>(InstanceInfo));
			Descriptions.Add(Node->GetRuntimeDescription(*this, NodeMemory, EBTDescriptionVerbosity::Detailed));
		
			Node = Node->GetParentNode();
		}

		for (int32 DescriptionIndex = Descriptions.Num() - 1; DescriptionIndex >= 0; DescriptionIndex--)
		{
			int32 SplitIdx = INDEX_NONE;
			if (Descriptions[DescriptionIndex].FindChar(TEXT(','), SplitIdx))
			{
				const FString KeyDesc = Descriptions[DescriptionIndex].Left(SplitIdx);
				const FString ValueDesc = Descriptions[DescriptionIndex].Mid(SplitIdx + 1).TrimStart();

				StatusCategory.Add(KeyDesc, ValueDesc);
			}
			else
			{
				StatusCategory.Add(Descriptions[DescriptionIndex], TEXT(""));
			}
		}

		if (StatusCategory.Data.Num() == 0)
		{
			StatusCategory.Add(TEXT("root"), TEXT("not initialized"));
		}

		Snapshot->Status.Add(StatusCategory);
	}

	if (CooldownTagsMap.Num() > 0)
	{
		FVisualLogStatusCategory StatusCategory;
		StatusCategory.Category = TEXT("Cooldown Tags");

		for (const auto& CooldownTagPair : CooldownTagsMap)
		{
			const FString TimeStr = FString::Printf(TEXT("%.2fs"), CooldownTagPair.Value);
			StatusCategory.Add(CooldownTagPair.Key.ToString(), TimeStr);
		}

		Snapshot->Status.Add(StatusCategory);
	}
}

#endif // ENABLE_VISUAL_LOG

void UBehaviorTreeComponent::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UBehaviorTreeComponent* This = CastChecked<UBehaviorTreeComponent>(InThis);
	for (FBehaviorTreeInstance& ActiveInstance : This->InstanceStack)
	{
		Collector.AddReferencedObject(ActiveInstance.RootNode);
		Collector.AddReferencedObject(ActiveInstance.ActiveNode);
		Collector.AddReferencedObjects(ActiveInstance.ActiveAuxNodes);
		for (FBehaviorTreeParallelTask& ParallelTask : ActiveInstance.ParallelTasks)
		{
			Collector.AddReferencedObject(ParallelTask.TaskNode);
		}
	}

	for (FBehaviorTreeSearchUpdate& PendingUpdate : This->SearchData.PendingUpdates)
	{
		Collector.AddReferencedObject(PendingUpdate.TaskNode);
		Collector.AddReferencedObject(PendingUpdate.AuxNode);
	}

	Collector.AddReferencedObject(This->ExecutionRequest.ExecuteNode);
	Collector.AddReferencedObject(This->PendingExecution.NextTask);

	for (FBranchActionInfo& PendingBranchActionRequest : This->PendingBranchActionRequests)
	{
		Collector.AddReferencedObject(PendingBranchActionRequest.Node);
	}


	Super::AddReferencedObjects(InThis, Collector);
}

void UBehaviorTreeComponent::StoreDebuggerExecutionStep(EBTExecutionSnap::Type SnapType)
{
#if USE_BEHAVIORTREE_DEBUGGER
	if (!IsDebuggerActive())
	{
		return;
	}

	FBehaviorTreeExecutionStep CurrentStep;
	CurrentStep.bIsExecutionPaused = bIsPaused;
	CurrentStep.ExecutionStepId = DebuggerSteps.Num() ? DebuggerSteps.Last().ExecutionStepId + 1 : 0;
	CurrentStep.TimeStamp = GetWorld()->GetTimeSeconds();
	CurrentStep.BlackboardValues = SearchStartBlackboard;

	for (int32 InstanceIndex = 0; InstanceIndex < InstanceStack.Num(); InstanceIndex++)
	{
		const FBehaviorTreeInstance& ActiveInstance = InstanceStack[InstanceIndex];
		
		FBehaviorTreeDebuggerInstance StoreInfo;
		StoreDebuggerInstance(StoreInfo, IntCastChecked<uint16>(InstanceIndex), SnapType);
		CurrentStep.InstanceStack.Add(StoreInfo);
	}

	for (int32 InstanceIndex = RemovedInstances.Num() - 1; InstanceIndex >= 0; InstanceIndex--)
	{
		CurrentStep.InstanceStack.Add(RemovedInstances[InstanceIndex]);
	}

	CurrentSearchFlow.Reset();
	CurrentRestarts.Reset();
	RemovedInstances.Reset();

	UBehaviorTreeManager* ManagerCDO = (UBehaviorTreeManager*)UBehaviorTreeManager::StaticClass()->GetDefaultObject();
	while (DebuggerSteps.Num() >= ManagerCDO->MaxDebuggerSteps)
	{
		DebuggerSteps.RemoveAt(0, /*Count=*/1, EAllowShrinking::No);
	}
	DebuggerSteps.Add(CurrentStep);
#endif
}

void UBehaviorTreeComponent::StoreDebuggerInstance(FBehaviorTreeDebuggerInstance& InstanceInfo, uint16 InstanceIdx, EBTExecutionSnap::Type SnapType) const
{
#if USE_BEHAVIORTREE_DEBUGGER
	if (!InstanceStack.IsValidIndex(InstanceIdx))
	{
		return;
	}

	const FBehaviorTreeInstance& ActiveInstance = InstanceStack[InstanceIdx];
	const FBehaviorTreeInstanceId& ActiveInstanceInfo = KnownInstances[ActiveInstance.InstanceIdIndex];
	InstanceInfo.TreeAsset = ActiveInstanceInfo.TreeAsset;
	InstanceInfo.RootNode = ActiveInstance.RootNode;

	if (SnapType == EBTExecutionSnap::Regular)
	{
		// traverse execution path
		UBTNode* StoreNode = ActiveInstance.ActiveNode ? ActiveInstance.ActiveNode : ActiveInstance.RootNode;
		while (StoreNode)
		{
			InstanceInfo.ActivePath.Add(StoreNode->GetExecutionIndex());
			StoreNode = StoreNode->GetParentNode();
		}

		// add aux nodes
		for (const UBTAuxiliaryNode* AuxNode : ActiveInstance.GetActiveAuxNodes())
		{
			InstanceInfo.AdditionalActiveNodes.Add(AuxNode->GetExecutionIndex());
		}

		// add active parallels
		for (const FBehaviorTreeParallelTask& TaskInfo : ActiveInstance.GetParallelTasks())
		{
			InstanceInfo.AdditionalActiveNodes.Add(TaskInfo.TaskNode->GetExecutionIndex());
		}

		// runtime values
		StoreDebuggerRuntimeValues(InstanceInfo.RuntimeDesc, ActiveInstance.RootNode, InstanceIdx);
	}

	// handle restart triggers
	if (CurrentRestarts.IsValidIndex(InstanceIdx))
	{
		InstanceInfo.PathFromPrevious = CurrentRestarts[InstanceIdx];
	}

	// store search flow, but remove nodes on execution path
	if (CurrentSearchFlow.IsValidIndex(InstanceIdx))
	{
		for (int32 FlowIndex = 0; FlowIndex < CurrentSearchFlow[InstanceIdx].Num(); FlowIndex++)
		{
			if (!InstanceInfo.ActivePath.Contains(CurrentSearchFlow[InstanceIdx][FlowIndex].ExecutionIndex))
			{
				InstanceInfo.PathFromPrevious.Add(CurrentSearchFlow[InstanceIdx][FlowIndex]);
			}
		}
	}
#endif
}

void UBehaviorTreeComponent::StoreDebuggerRemovedInstance(uint16 InstanceIdx) const
{
#if USE_BEHAVIORTREE_DEBUGGER
	if (!IsDebuggerActive())
	{
		return;
	}

	FBehaviorTreeDebuggerInstance StoreInfo;
	StoreDebuggerInstance(StoreInfo, InstanceIdx, EBTExecutionSnap::OutOfNodes);

	RemovedInstances.Add(StoreInfo);
#endif
}

void UBehaviorTreeComponent::StoreDebuggerSearchStep(const UBTNode* Node, uint16 InstanceIdx, EBTNodeResult::Type NodeResult) const
{
#if USE_BEHAVIORTREE_DEBUGGER
	if (!IsDebuggerActive())
	{
		return;
	}

	if (Node && NodeResult != EBTNodeResult::InProgress && NodeResult != EBTNodeResult::Aborted)
	{
		FBehaviorTreeDebuggerInstance::FNodeFlowData FlowInfo;
		FlowInfo.ExecutionIndex = Node->GetExecutionIndex();
		FlowInfo.bPassed = (NodeResult == EBTNodeResult::Succeeded);
		
		if (CurrentSearchFlow.Num() < (InstanceIdx + 1))
		{
			CurrentSearchFlow.SetNum(InstanceIdx + 1);
		}

		if (CurrentSearchFlow[InstanceIdx].Num() == 0 || CurrentSearchFlow[InstanceIdx].Last().ExecutionIndex != FlowInfo.ExecutionIndex)
		{
			CurrentSearchFlow[InstanceIdx].Add(FlowInfo);
		}
	}
#endif
}

void UBehaviorTreeComponent::StoreDebuggerSearchStep(const UBTNode* Node, uint16 InstanceIdx, bool bPassed) const
{
#if USE_BEHAVIORTREE_DEBUGGER
	if (!IsDebuggerActive())
	{
		return;
	}

	if (Node && !bPassed)
	{
		FBehaviorTreeDebuggerInstance::FNodeFlowData FlowInfo;
		FlowInfo.ExecutionIndex = Node->GetExecutionIndex();
		FlowInfo.bPassed = bPassed;

		if (CurrentSearchFlow.Num() < (InstanceIdx + 1))
		{
			CurrentSearchFlow.SetNum(InstanceIdx + 1);
		}

		CurrentSearchFlow[InstanceIdx].Add(FlowInfo);
	}
#endif
}

void UBehaviorTreeComponent::StoreDebuggerRestart(const UBTNode* Node, uint16 InstanceIdx, bool bAllowed)
{
#if USE_BEHAVIORTREE_DEBUGGER
	if (!IsDebuggerActive())
	{
		return;
	}

	if (Node)
	{
		FBehaviorTreeDebuggerInstance::FNodeFlowData FlowInfo;
		FlowInfo.ExecutionIndex = Node->GetExecutionIndex();
		FlowInfo.bTrigger = bAllowed;
		FlowInfo.bDiscardedTrigger = !bAllowed;

		if (CurrentRestarts.Num() < (InstanceIdx + 1))
		{
			CurrentRestarts.SetNum(InstanceIdx + 1);
		}

		CurrentRestarts[InstanceIdx].Add(FlowInfo);
	}
#endif
}

void UBehaviorTreeComponent::StoreDebuggerRuntimeValues(TArray<FString>& RuntimeDescriptions, UBTNode* RootNode, uint16 InstanceIdx) const
{
#if USE_BEHAVIORTREE_DEBUGGER
	if (!InstanceStack.IsValidIndex(InstanceIdx))
	{
		return;
	}

	const FBehaviorTreeInstance& InstanceInfo = InstanceStack[InstanceIdx];

	TArray<FString> RuntimeValues;
	for (UBTNode* Node = RootNode; Node; Node = Node->GetNextNode())
	{
		uint8* NodeMemory = (uint8*)Node->GetNodeMemory<uint8>(InstanceInfo);

		RuntimeValues.Reset();

		constexpr EBTDescriptionVerbosity::Type DescriptionVerbosity = EBTDescriptionVerbosity::Basic;
		if (Node->HasInstance())
		{
			if (UBTNode* NodeInstance = Node->GetNodeInstance(*this, NodeMemory))
			{
				NodeInstance->DescribeRuntimeValues(*this, NodeMemory, DescriptionVerbosity, RuntimeValues);
			}
		}
		else
		{
			Node->DescribeRuntimeValues(*this, NodeMemory, DescriptionVerbosity, RuntimeValues);
		}

		FString ComposedDesc;
		for (int32 ValueIndex = 0; ValueIndex < RuntimeValues.Num(); ValueIndex++)
		{
			if (ComposedDesc.Len())
			{
				ComposedDesc.AppendChar(TEXT('\n'));
			}

			ComposedDesc += RuntimeValues[ValueIndex];
		}

		RuntimeDescriptions.SetNum(Node->GetExecutionIndex() + 1);
		RuntimeDescriptions[Node->GetExecutionIndex()] = ComposedDesc;
	}
#endif
}

void UBehaviorTreeComponent::UpdateDebuggerAfterExecution(const UBTTaskNode* TaskNode, uint16 InstanceIdx) const
{
#if USE_BEHAVIORTREE_DEBUGGER
	if (!IsDebuggerActive() || !InstanceStack.IsValidIndex(InstanceIdx))
	{
		return;
	}

	FBehaviorTreeExecutionStep& CurrentStep = DebuggerSteps.Last();

	// store runtime values
	TArray<FString> RuntimeValues;
	const FBehaviorTreeInstance& InstanceToUpdate = InstanceStack[InstanceIdx];
	uint8* NodeMemory = (uint8*)TaskNode->GetNodeMemory<uint8>(InstanceToUpdate);
	TaskNode->DescribeRuntimeValues(*this, NodeMemory, EBTDescriptionVerbosity::Basic, RuntimeValues);

	FString ComposedDesc;
	for (int32 ValueIndex = 0; ValueIndex < RuntimeValues.Num(); ValueIndex++)
	{
		if (ComposedDesc.Len())
		{
			ComposedDesc.AppendChar(TEXT('\n'));
		}

		ComposedDesc += RuntimeValues[ValueIndex];
	}

	// accessing RuntimeDesc should never be out of bounds (active task MUST be part of active instance)
	const uint16& ExecutionIndex = TaskNode->GetExecutionIndex();
	if (CurrentStep.InstanceStack[InstanceIdx].RuntimeDesc.IsValidIndex(ExecutionIndex))
	{
		CurrentStep.InstanceStack[InstanceIdx].RuntimeDesc[ExecutionIndex] = ComposedDesc;
	}
	else
	{
		UE_VLOG(GetOwner(), LogBehaviorTree, Error, TEXT("Incomplete debugger data! No runtime description for executed task, instance %d has only %d entries!"),
			InstanceIdx, CurrentStep.InstanceStack[InstanceIdx].RuntimeDesc.Num());
	}
#endif
}

void UBehaviorTreeComponent::StoreDebuggerBlackboard(TMap<FName, FString>& BlackboardValueDesc) const
{
#if USE_BEHAVIORTREE_DEBUGGER
	if (!IsDebuggerActive())
	{
		return;
	}

	if (BlackboardComp && BlackboardComp->HasValidAsset())
	{
		const int32 NumKeys = BlackboardComp->GetNumKeys();
		BlackboardValueDesc.Empty(NumKeys);

		for (int32 KeyIndex = 0; KeyIndex < NumKeys; KeyIndex++)
		{
			const FBlackboard::FKey Key = FBlackboard::FKey(KeyIndex);

			FString Value = BlackboardComp->DescribeKeyValue(Key, EBlackboardDescription::OnlyValue);
			if (Value.Len() == 0)
			{
				Value = TEXT("n/a");
			}

			BlackboardValueDesc.Add(BlackboardComp->GetKeyName(Key), Value);
		}
	}
#endif
}

// Code for timing BT Search for FramePro
#if !UE_BUILD_SHIPPING
void UBehaviorTreeComponent::EndFrame()
{
	if (CVarBTRecordFrameSearchTimes.GetValueOnGameThread() != 0)
	{
		const double FrameSearchTimeMilliSecsDouble = FrameSearchTime * 1000.;
		const double AvFrameSearchTimeMilliSecsDouble = (NumSearchTimeCalls > 0) ? FrameSearchTimeMilliSecsDouble / static_cast<double>(NumSearchTimeCalls) : 0.;
		const float FrameSearchTimeMilliSecsFloat = static_cast<float>(FrameSearchTimeMilliSecsDouble);
		const float NumSearchTimeCallsFloat = static_cast<float>(NumSearchTimeCalls);
		const float AvFrameSearchTimeMilliSecsFloat = static_cast<float>(AvFrameSearchTimeMilliSecsDouble);

		FPlatformMisc::CustomNamedStat("BehaviorTreeSearchTimeFrameMs", FrameSearchTimeMilliSecsFloat, "BehaviorTree", "MilliSecs");
		FPlatformMisc::CustomNamedStat("BehaviorTreeSearchCallsFrame", NumSearchTimeCallsFloat, "BehaviorTree", "Count");
		FPlatformMisc::CustomNamedStat("BehaviorTreeSearchTimeFrameAvMs", AvFrameSearchTimeMilliSecsFloat, "BehaviorTree", "MilliSecs");

		FrameSearchTime = 0.;
		NumSearchTimeCalls = 0;
	}
}
#endif

bool UBehaviorTreeComponent::IsDebuggerActive()
{
#if USE_BEHAVIORTREE_DEBUGGER
	if (ActiveDebuggerCounter <= 0)
	{
		static bool bAlwaysGatherData = false;
		static uint64 PrevFrameCounter = 0;

		if (GFrameCounter != PrevFrameCounter)
		{
			GConfig->GetBool(TEXT("/Script/UnrealEd.EditorPerProjectUserSettings"), TEXT("bAlwaysGatherBehaviorTreeDebuggerData"), bAlwaysGatherData, GEditorPerProjectIni);
			PrevFrameCounter = GFrameCounter;
		}

		return bAlwaysGatherData;
	}

	return true;
#else
	return false;
#endif
}
