// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTreeDelegates.h"

FBehaviorTreeDelegates::FOnTreeStarted FBehaviorTreeDelegates::OnTreeStarted;
FBehaviorTreeDelegates::FOnTreeDebugTool FBehaviorTreeDelegates::OnDebugSelected;
FBehaviorTreeDelegates::FOnTreeDebugTool FBehaviorTreeDelegates::OnDebugLocked;
