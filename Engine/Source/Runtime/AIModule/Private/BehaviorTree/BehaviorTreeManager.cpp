// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/BehaviorTreeManager.h"
#include "UObject/UObjectHash.h"
#include "UObject/Class.h"
#include "UObject/UObjectIterator.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "AISystem.h"
#include "BehaviorTreeNodeInitializationData.h"
#include "BehaviorTree/BehaviorTreeTypes.h"
#include "BehaviorTree/BTNode.h"
#include "BehaviorTree/BTTaskNode.h"
#include "BehaviorTree/BTDecorator.h"
#include "BehaviorTree/BTService.h"
#include "BehaviorTree/BTCompositeNode.h"
#include "BehaviorTree/BehaviorTree.h"
#include "BehaviorTree/Tasks/BTTask_RunBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BehaviorTreeManager)

#if WITH_EDITOR
#include "Kismet2/KismetEditorUtilities.h"
#endif // WITH_EDITOR

DEFINE_STAT(STAT_AI_BehaviorTree_Tick);
DEFINE_STAT(STAT_AI_BehaviorTree_LoadTime);
DEFINE_STAT(STAT_AI_BehaviorTree_SearchTime);
DEFINE_STAT(STAT_AI_BehaviorTree_ExecutionTime);
DEFINE_STAT(STAT_AI_BehaviorTree_AuxUpdateTime);
DEFINE_STAT(STAT_AI_BehaviorTree_Cleanup);
DEFINE_STAT(STAT_AI_BehaviorTree_StopTree);
DEFINE_STAT(STAT_AI_BehaviorTree_NumTemplates);
DEFINE_STAT(STAT_AI_BehaviorTree_NumInstances);
DEFINE_STAT(STAT_AI_BehaviorTree_InstanceMemory);

UBehaviorTreeManager::UBehaviorTreeManager(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	MaxDebuggerSteps = 100;
}

void UBehaviorTreeManager::FinishDestroy()
{
	SET_DWORD_STAT(STAT_AI_BehaviorTree_NumTemplates, 0);

	for (int32 Idx = 0; Idx < ActiveComponents.Num(); Idx++)
	{
		if (ActiveComponents[Idx] && !ActiveComponents[Idx]->HasAnyFlags(RF_BeginDestroyed))
		{
			ActiveComponents[Idx]->Cleanup();
		}
	}

	ActiveComponents.Reset();
	Super::FinishDestroy();
}

uint16 UBehaviorTreeManager::GetAlignedDataSize(uint16 Size)
{
	// round to 4 bytes
	return ((Size + 3) & ~3);
}

static void MergeDecoratorOpsHelper(TArray<FBTDecoratorLogic>& LinkOps, const TArray<FBTDecoratorLogic>& InjectedOps, const int32 NumOriginalDecorators, const int32 NumInjectedDecorators)
{
	if (LinkOps.Num() == 0 && InjectedOps.Num() == 0)
	{
		return;
	}

	const int32 NumOriginalOps = LinkOps.Num();
	if (NumOriginalDecorators > 0)
	{
		// and operator for two groups of composites: original and new one
		FBTDecoratorLogic MainAndOp(EBTDecoratorLogic::And, LinkOps.Num() ? 2 : IntCastChecked<uint16>(NumOriginalDecorators + 1));
		LinkOps.Insert(MainAndOp, 0);

		if (NumOriginalOps == 0)
		{
			// add Test operations, original link didn't have composite operators
			for (int32 Idx = 0; Idx < NumOriginalDecorators; Idx++)
			{
				FBTDecoratorLogic TestOp(EBTDecoratorLogic::Test, IntCastChecked<uint16>(Idx));
				LinkOps.Add(TestOp);
			}
		}
	}

	// add injected operators
	if (InjectedOps.Num() == 0)
	{
		FBTDecoratorLogic InjectedAndOp(EBTDecoratorLogic::And, IntCastChecked<uint16>(NumInjectedDecorators));
		LinkOps.Add(InjectedAndOp);

		for (int32 Idx = 0; Idx < NumInjectedDecorators; Idx++)
		{
			FBTDecoratorLogic TestOp(EBTDecoratorLogic::Test, IntCastChecked<uint16>(NumOriginalDecorators + Idx));
			LinkOps.Add(TestOp);
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < InjectedOps.Num(); Idx++)
		{
			FBTDecoratorLogic InjectedOpCopy = InjectedOps[Idx];
			if (InjectedOpCopy.Operation == EBTDecoratorLogic::Test)
			{
				InjectedOpCopy.Number = IntCastChecked<uint16>((int32)InjectedOpCopy.Number + NumOriginalDecorators);
			}

			LinkOps.Add(InjectedOpCopy);
		}
	}
}

static void InitializeNodeHelper(UBTCompositeNode* ParentNode, UBTNode* NodeOb,
	uint8 TreeDepth, uint16& ExecutionIndex, TArray<FBehaviorTreeNodeInitializationData>& InitList,
	UBehaviorTree& TreeAsset, UObject* NodeOuter)
{
	InitList.Add(FBehaviorTreeNodeInitializationData(NodeOb, ParentNode, ExecutionIndex, TreeDepth, NodeOb->GetInstanceMemorySize(), NodeOb->GetSpecialMemorySize()));
	{
		FScopedBTLoggingContext LogContext(NodeOb);
		NodeOb->InitializeFromAsset(TreeAsset);
	}
	ExecutionIndex++;

	UBTCompositeNode* CompositeOb = Cast<UBTCompositeNode>(NodeOb);
	if (CompositeOb)
	{
		for (int32 ServiceIndex = 0; ServiceIndex < CompositeOb->Services.Num(); ServiceIndex++)
		{
			if (CompositeOb->Services[ServiceIndex] == NULL)
			{
				UE_LOGF(LogBehaviorTree, Warning, "%ls has missing service node! (parent: %ls)",
					*TreeAsset.GetName(), *UBehaviorTreeTypes::DescribeNodeHelper(CompositeOb));

				CompositeOb->Services.RemoveAt(ServiceIndex, EAllowShrinking::No);
				ServiceIndex--;
				continue;
			}

			UBTService* Service = Cast<UBTService>(StaticDuplicateObject(CompositeOb->Services[ServiceIndex], NodeOuter));;
			CompositeOb->Services[ServiceIndex] = Service;

			InitList.Add(FBehaviorTreeNodeInitializationData(Service, CompositeOb, ExecutionIndex, TreeDepth,
				Service->GetInstanceMemorySize(), Service->GetSpecialMemorySize()));

			FScopedBTLoggingContext LogContext(Service);
			Service->InitializeFromAsset(TreeAsset);
			// don't initialize parent link for services on composite node
			ExecutionIndex++;
		}

		for (int32 ChildIndex = 0; ChildIndex < CompositeOb->Children.Num(); ChildIndex++)
		{
			FBTCompositeChild& ChildInfo = CompositeOb->Children[ChildIndex];
			for (int32 DecoratorIndex = 0; DecoratorIndex < ChildInfo.Decorators.Num(); DecoratorIndex++)
			{
				if (ChildInfo.Decorators[DecoratorIndex] == NULL)
				{
					UE_LOGF(LogBehaviorTree, Warning, "%ls has missing decorator node! (parent: %ls, branch: %d)",
						*TreeAsset.GetName(), *UBehaviorTreeTypes::DescribeNodeHelper(CompositeOb), ChildIndex);

					ChildInfo.Decorators.RemoveAt(DecoratorIndex, EAllowShrinking::No);
					DecoratorIndex--;
					continue;
				}

				UBTDecorator* Decorator = Cast<UBTDecorator>(StaticDuplicateObject(ChildInfo.Decorators[DecoratorIndex], NodeOuter));
				ChildInfo.Decorators[DecoratorIndex] = Decorator;

				InitList.Add(FBehaviorTreeNodeInitializationData(Decorator, CompositeOb, ExecutionIndex, TreeDepth,
					Decorator->GetInstanceMemorySize(), Decorator->GetSpecialMemorySize()));

				FScopedBTLoggingContext LogContext(Decorator);
				Decorator->InitializeFromAsset(TreeAsset);
				Decorator->InitializeParentLink(IntCastChecked<uint8>(ChildIndex));
				ExecutionIndex++;
			}

			// inject root level decorators from subtree task
			UBTTask_RunBehavior* SubtreeTask = Cast<UBTTask_RunBehavior>(ChildInfo.ChildTask);
			if (SubtreeTask && SubtreeTask->GetSubtreeAsset() && SubtreeTask->GetSubtreeAsset()->RootDecorators.Num())
			{
				UBehaviorTree& SubtreeAsset = *SubtreeTask->GetSubtreeAsset();
				const int32 NumOrgDecorators = ChildInfo.Decorators.Num();

				for (int32 DecoratorIndex = 0; DecoratorIndex < SubtreeAsset.RootDecorators.Num(); DecoratorIndex++)
				{
					if (SubtreeAsset.RootDecorators[DecoratorIndex] == NULL)
					{
						UE_LOGF(LogBehaviorTree, Warning, "%ls can't inject decorator from %ls! (parent: %ls, branch: %d)",
							*TreeAsset.GetName(), *SubtreeAsset.GetName(),
							*UBehaviorTreeTypes::DescribeNodeHelper(CompositeOb), ChildIndex);
						continue;
					}

					UBTDecorator* Decorator = Cast<UBTDecorator>(StaticDuplicateObject(SubtreeAsset.RootDecorators[DecoratorIndex], NodeOuter));
					ChildInfo.Decorators.Add(Decorator);

					InitList.Add(FBehaviorTreeNodeInitializationData(Decorator, CompositeOb, ExecutionIndex, TreeDepth,
						Decorator->GetInstanceMemorySize(), Decorator->GetSpecialMemorySize()));

					// initialize with parent tree
					Decorator->MarkInjectedNode();
					FScopedBTLoggingContext LogContext(Decorator);
					Decorator->InitializeFromAsset(TreeAsset);
					Decorator->InitializeParentLink(IntCastChecked<uint8>(ChildIndex));
					ExecutionIndex++;
				}

				const int32 NumInjectedDecorators = ChildInfo.Decorators.Num() - NumOrgDecorators;
				MergeDecoratorOpsHelper(ChildInfo.DecoratorOps, SubtreeAsset.RootDecoratorOps, NumOrgDecorators, NumInjectedDecorators);
			}

			UBTNode* ChildNode = NULL;
			
			if (ChildInfo.ChildComposite)
			{
				ChildInfo.ChildComposite = Cast<UBTCompositeNode>(StaticDuplicateObject(ChildInfo.ChildComposite, NodeOuter));
				ChildNode = ChildInfo.ChildComposite;
			}
			else if (ChildInfo.ChildTask)
			{
				ChildInfo.ChildTask = Cast<UBTTaskNode>(StaticDuplicateObject(ChildInfo.ChildTask, NodeOuter));
				ChildNode = ChildInfo.ChildTask;

				for (int32 ServiceIndex = 0; ServiceIndex < ChildInfo.ChildTask->Services.Num(); ServiceIndex++)
				{
					if (ChildInfo.ChildTask->Services[ServiceIndex] == NULL)
					{
						UE_LOGF(LogBehaviorTree, Warning, "%ls has missing service node! (parent: %ls)",
							*TreeAsset.GetName(), *UBehaviorTreeTypes::DescribeNodeHelper(ChildInfo.ChildTask));

						ChildInfo.ChildTask->Services.RemoveAt(ServiceIndex, EAllowShrinking::No);
						ServiceIndex--;
						continue;
					}

					UBTService* Service = Cast<UBTService>(StaticDuplicateObject(ChildInfo.ChildTask->Services[ServiceIndex], NodeOuter));
					ChildInfo.ChildTask->Services[ServiceIndex] = Service;

					InitList.Add(FBehaviorTreeNodeInitializationData(Service, CompositeOb, ExecutionIndex, TreeDepth,
						Service->GetInstanceMemorySize(), Service->GetSpecialMemorySize()));

					FScopedBTLoggingContext LogContext(Service);
					Service->InitializeFromAsset(TreeAsset);
					Service->InitializeParentLink(IntCastChecked<uint8>(ChildIndex));
					ExecutionIndex++;
				}
			}

			if (ChildNode)
			{
				InitializeNodeHelper(CompositeOb, ChildNode, TreeDepth + 1, ExecutionIndex, InitList, TreeAsset, NodeOuter);
			}
		}

		CompositeOb->InitializeComposite(ExecutionIndex - 1);
	}
}

bool UBehaviorTreeManager::LoadTree(UBehaviorTree& Asset, UBTCompositeNode*& Root, uint16& InstanceMemorySize)
{
	SCOPE_CYCLE_COUNTER(STAT_AI_BehaviorTree_LoadTime);

	for (int32 TemplateIndex = 0; TemplateIndex < LoadedTemplates.Num(); TemplateIndex++)
	{
		FBehaviorTreeTemplateInfo& TemplateInfo = LoadedTemplates[TemplateIndex];
		if (TemplateInfo.Asset == &Asset)
		{
			Root = TemplateInfo.Template;
			InstanceMemorySize = TemplateInfo.InstanceMemorySize;
			return true;
		}
	}

	if (Asset.RootNode)
	{
		FBehaviorTreeTemplateInfo TemplateInfo;
		TemplateInfo.Asset = &Asset;
		TemplateInfo.Template = Cast<UBTCompositeNode>(StaticDuplicateObject(Asset.RootNode, this));

		TArray<FBehaviorTreeNodeInitializationData> InitList;
		uint16 ExecutionIndex = 0;
		InitializeNodeHelper(NULL, TemplateInfo.Template, 0, ExecutionIndex, InitList, Asset, this);

#if USE_BEHAVIORTREE_DEBUGGER
		// fill in information about next nodes in execution index, before sorting memory offsets
		for (int32 Index = 0; Index < InitList.Num() - 1; Index++)
		{
			InitList[Index].Node->InitializeExecutionOrder(InitList[Index + 1].Node);
		}
#endif

		// sort nodes by memory size, so they can be packed better
		// it still won't protect against structures, that are internally misaligned (-> uint8, uint32)
		// but since all Engine level nodes are good... 
		InitList.Sort(FBehaviorTreeNodeInitializationData::FMemorySort());
		uint16 MemoryOffset = 0;
		for (int32 Index = 0; Index < InitList.Num(); Index++)
		{
			InitList[Index].Node->InitializeNode(InitList[Index].ParentNode, InitList[Index].ExecutionIndex, InitList[Index].SpecialDataSize + MemoryOffset, InitList[Index].TreeDepth);
			MemoryOffset += InitList[Index].DataSize;
		}
		
		TemplateInfo.InstanceMemorySize = MemoryOffset;

		INC_DWORD_STAT(STAT_AI_BehaviorTree_NumTemplates);
		LoadedTemplates.Add(TemplateInfo);
		Root = TemplateInfo.Template;
		InstanceMemorySize = TemplateInfo.InstanceMemorySize;
		return true;
	}

	return false;
}

void UBehaviorTreeManager::InitializeMemoryHelper(const TArray<UBTDecorator*>& Nodes, TArray<uint16>& MemoryOffsets, int32& MemorySize, bool bForceInstancing)
{
	TArray<FBehaviorTreeNodeInitializationData> InitList;
	for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); NodeIndex++)
	{
		UBTNode* Node = Nodes[NodeIndex];

		const bool bUsesInstancing = Node->HasInstance();
		if (bForceInstancing)
		{
			Node->ForceInstancing(true);
		}

		InitList.Add(FBehaviorTreeNodeInitializationData(Nodes[NodeIndex], NULL, 0, 0, Nodes[NodeIndex]->GetInstanceMemorySize(), Nodes[NodeIndex]->GetSpecialMemorySize()));

		if (bForceInstancing && !bUsesInstancing)
		{
			Node->ForceInstancing(false);
		}
	}

	InitList.Sort(FBehaviorTreeNodeInitializationData::FMemorySort());

	uint16 MemoryOffset = 0;
	MemoryOffsets.AddZeroed(Nodes.Num());

	for (int32 Index = 0; Index < InitList.Num(); Index++)
	{
		MemoryOffsets[Index] = InitList[Index].SpecialDataSize + MemoryOffset;
		MemoryOffset += InitList[Index].DataSize;
	}

	MemorySize = MemoryOffset;
}

//----------------------------------------------------------------------//
// stats dumping
//----------------------------------------------------------------------//
struct FNodeClassCounter
{
	TMap<UClass*, uint32> NodeClassUsage;

	FNodeClassCounter()
	{}

	void Declare(UClass* NodeClass)
	{
		NodeClassUsage.FindOrAdd(NodeClass);
	}

	void CountNode(UBTNode* Node)
	{
		uint32& Count = NodeClassUsage.FindOrAdd(Node->GetClass());
		++Count;
	}

	void Append(const FNodeClassCounter& Other)
	{
		for (auto Iterator : Other.NodeClassUsage)
		{
			uint32& Count = NodeClassUsage.FindOrAdd(Iterator.Key);
			Count += Iterator.Value;
		}
	}

	void Print(const TCHAR* Separator=TEXT(" "))
	{
		for (auto Iterator : NodeClassUsage)
		{
			UE_LOGF(LogBehaviorTree, Display, "%ls%ls%ls(%ls)%ls%d"
				, Separator
				, *Iterator.Key->GetName()
				, Separator
				, Iterator.Key->HasAnyClassFlags(CLASS_CompiledFromBlueprint) ? TEXT("BP") : TEXT("C++")
				, Separator
				, Iterator.Value);
		}
	}
};

void StatNodeUsage(UBTNode* Node, FNodeClassCounter& NodeCounter)
{
	NodeCounter.CountNode(Node);

	UBTCompositeNode* CompositeOb = Cast<UBTCompositeNode>(Node);
	if (CompositeOb)
	{
		for (int32 ServiceIndex = 0; ServiceIndex < CompositeOb->Services.Num(); ServiceIndex++)
		{
			if (CompositeOb->Services[ServiceIndex])
			{
				NodeCounter.CountNode(CompositeOb->Services[ServiceIndex]);
			}
		}

		for (int32 ChildIndex = 0; ChildIndex < CompositeOb->Children.Num(); ChildIndex++)
		{
			FBTCompositeChild& ChildInfo = CompositeOb->Children[ChildIndex];
			for (int32 DecoratorIndex = 0; DecoratorIndex < ChildInfo.Decorators.Num(); DecoratorIndex++)
			{
				if (ChildInfo.Decorators[DecoratorIndex])
				{
					NodeCounter.CountNode(ChildInfo.Decorators[DecoratorIndex]);
				}
			}

			UBTNode* ChildNode = NULL;

			if (ChildInfo.ChildComposite)
			{
				StatNodeUsage(ChildInfo.ChildComposite, NodeCounter);
			}
			else if (ChildInfo.ChildTask)
			{
				for (int32 ServiceIndex = 0; ServiceIndex < ChildInfo.ChildTask->Services.Num(); ServiceIndex++)
				{
					if (ChildInfo.ChildTask->Services[ServiceIndex])
					{
						NodeCounter.CountNode(ChildInfo.ChildTask->Services[ServiceIndex]);
					}
				}

				NodeCounter.CountNode(ChildInfo.ChildTask);
			}
		}
	}
}

void UBehaviorTreeManager::DumpUsageStats() const
{
	FNodeClassCounter AllNodesCounter;
	
	for (TObjectIterator<UClass> It; It; ++It)
	{
		if (It->IsChildOf(UBTNode::StaticClass()) && It->HasAnyClassFlags(CLASS_Abstract) == false
#if WITH_EDITOR
			&& !(FKismetEditorUtilities::IsClassABlueprintSkeleton(*It)
				|| It->HasAnyClassFlags(CLASS_NewerVersionExists))
#endif
			)

		{
			AllNodesCounter.Declare(*It);
		}
	}

	UE_LOGF(LogBehaviorTree, Display, "----------------------UBehaviorTreeManager::DumpUsageStats----------------------\nBehavior Trees:");

	// get all BTNode classes
	
	for (TObjectIterator<UBehaviorTree> It; It; ++It)
	{
		FNodeClassCounter TreeNodeCounter;
		UE_LOGF(LogBehaviorTree, Display, "--- %ls ---", *(It->GetName()));
		StatNodeUsage(It->RootNode, TreeNodeCounter);
		TreeNodeCounter.Print();
		AllNodesCounter.Append(TreeNodeCounter);
	}
	
	UE_LOGF(LogBehaviorTree, Display, "--- Total Nodes usage:");
	AllNodesCounter.Print(TEXT(","));
}

void UBehaviorTreeManager::AddActiveComponent(UBehaviorTreeComponent& Component)
{
	ActiveComponents.AddUnique(&Component);
}

void UBehaviorTreeManager::RemoveActiveComponent(UBehaviorTreeComponent& Component)
{
	ActiveComponents.Remove(&Component);
}

UBehaviorTreeManager* UBehaviorTreeManager::GetCurrent(UWorld* World)
{
	UAISystem* AISys = UAISystem::GetCurrentSafe(World);
	return AISys ? AISys->GetBehaviorTreeManager() : nullptr;
}

UBehaviorTreeManager* UBehaviorTreeManager::GetCurrent(UObject* WorldContextObject)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::ReturnNull);
	UAISystem* AISys = UAISystem::GetCurrentSafe(World);
	return AISys ? AISys->GetBehaviorTreeManager() : nullptr;
}

