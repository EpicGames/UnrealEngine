// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/BehaviorTreeTypes.h"
#include "BehaviorTree/BehaviorTree.h"
#include "GameFramework/Actor.h"
#include "BehaviorTree/BTDecorator.h"
#include "BehaviorTree/BTService.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Enum.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_NativeEnum.h"
#include "BehaviorTree/BlackboardData.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Object.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Vector.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Rotator.h"
#include "VisualLogger/VisualLogger.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Bool.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Class.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Float.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Int.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Name.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_String.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Struct.h"
#include "BehaviorTree/BTTaskNode.h"
#include "BehaviorTree/BTCompositeNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BehaviorTreeTypes)

#if WITH_EDITORONLY_DATA
FText UE::BehaviorTree::Messages::BlackboardNotSet(NSLOCTEXT("BehaviorTree", "BlackboardNotSet", "Blackboard not set"));
#endif

//----------------------------------------------------------------------//
// FBehaviorTreeInstance
//----------------------------------------------------------------------//
void FBehaviorTreeInstance::Initialize(UBehaviorTreeComponent& OwnerComp, UBTCompositeNode& Node, int32& InstancedIndex, EBTMemoryInit::Type InitType)
{
	for (int32 ServiceIndex = 0; ServiceIndex < Node.Services.Num(); ServiceIndex++)
	{
		Node.Services[ServiceIndex]->InitializeInSubtree(OwnerComp, Node.Services[ServiceIndex]->GetNodeMemory<uint8>(*this), InstancedIndex, InitType);
	}

	uint8* NodeMemory = Node.GetNodeMemory<uint8>(*this);
	Node.InitializeInSubtree(OwnerComp, NodeMemory, InstancedIndex, InitType);

	UBTCompositeNode* InstancedComposite = Cast<UBTCompositeNode>(Node.GetNodeInstance(OwnerComp, NodeMemory));
	if (InstancedComposite)
	{
		InstancedComposite->InitializeComposite(Node.GetLastExecutionIndex());
	}

	for (int32 ChildIndex = 0; ChildIndex < Node.Children.Num(); ChildIndex++)
	{
		FBTCompositeChild& ChildInfo = Node.Children[ChildIndex];

		for (int32 DecoratorIndex = 0; DecoratorIndex < ChildInfo.Decorators.Num(); DecoratorIndex++)
		{
			UBTDecorator* DecoratorOb = ChildInfo.Decorators[DecoratorIndex];
			uint8* DecoratorMemory = DecoratorOb->GetNodeMemory<uint8>(*this);
			DecoratorOb->InitializeInSubtree(OwnerComp, DecoratorMemory, InstancedIndex, InitType);

			UBTDecorator* InstancedDecoratorOb = Cast<UBTDecorator>(DecoratorOb->GetNodeInstance(OwnerComp, DecoratorMemory));
			if (InstancedDecoratorOb)
			{
				InstancedDecoratorOb->InitializeParentLink(DecoratorOb->GetChildIndex());
			}
		}

		if (ChildInfo.ChildComposite)
		{
			Initialize(OwnerComp, *(ChildInfo.ChildComposite), InstancedIndex, InitType);
		}
		else if (ChildInfo.ChildTask)
		{
			for (int32 ServiceIndex = 0; ServiceIndex < ChildInfo.ChildTask->Services.Num(); ServiceIndex++)
			{
				UBTService* ServiceOb = ChildInfo.ChildTask->Services[ServiceIndex];
				uint8* ServiceMemory = ServiceOb->GetNodeMemory<uint8>(*this);
				ServiceOb->InitializeInSubtree(OwnerComp, ServiceMemory, InstancedIndex, InitType);

				UBTService* InstancedServiceOb = Cast<UBTService>(ServiceOb->GetNodeInstance(OwnerComp, ServiceMemory));
				if (InstancedServiceOb)
				{
					InstancedServiceOb->InitializeParentLink(ServiceOb->GetChildIndex());
				}
			}

			ChildInfo.ChildTask->InitializeInSubtree(OwnerComp, ChildInfo.ChildTask->GetNodeMemory<uint8>(*this), InstancedIndex, InitType);
		}
	}
}

void FBehaviorTreeInstance::Cleanup(UBehaviorTreeComponent& OwnerComp, EBTMemoryClear::Type CleanupType)
{
	if (!ensureMsgf(OwnerComp.KnownInstances.IsValidIndex(InstanceIdIndex), TEXT("Expected InstanceIdIndex to be in known instances (Root:%s, Num:%i, Index:%i)"), *GetNameSafe(RootNode->GetTreeAsset()), OwnerComp.KnownInstances.Num(), InstanceIdIndex))
	{
		return;
	}

	FBehaviorTreeInstanceId& Info = OwnerComp.KnownInstances[InstanceIdIndex];
	if (Info.FirstNodeInstance >= 0)
	{
		const int32 MaxAllowedIdx = OwnerComp.NodeInstances.Num();
		const int32 LastNodeIdx = OwnerComp.KnownInstances.IsValidIndex(InstanceIdIndex + 1) ?
			FMath::Min(OwnerComp.KnownInstances[InstanceIdIndex + 1].FirstNodeInstance, MaxAllowedIdx) :
			MaxAllowedIdx;

		for (int32 Idx = Info.FirstNodeInstance; Idx < LastNodeIdx; Idx++)
		{
			OwnerComp.NodeInstances[Idx]->OnInstanceDestroyed(OwnerComp);
		}
	}

	CleanupNodes(OwnerComp, *RootNode, CleanupType);

	// remove memory when instance is destroyed - it will need full initialize anyway
	if (CleanupType == EBTMemoryClear::Destroy)
	{
		Info.InstanceMemory.Empty();
	}
	else
	{
		Info.InstanceMemory = InstanceMemory;
	}
}

void FBehaviorTreeInstance::CleanupNodes(UBehaviorTreeComponent& OwnerComp, UBTCompositeNode& Node, EBTMemoryClear::Type CleanupType)
{
	for (int32 ServiceIndex = 0; ServiceIndex < Node.Services.Num(); ServiceIndex++)
	{
		Node.Services[ServiceIndex]->CleanupInSubtree(OwnerComp, Node.Services[ServiceIndex]->GetNodeMemory<uint8>(*this), CleanupType);
	}

	Node.CleanupInSubtree(OwnerComp, Node.GetNodeMemory<uint8>(*this), CleanupType);

	for (int32 ChildIndex = 0; ChildIndex < Node.Children.Num(); ChildIndex++)
	{
		FBTCompositeChild& ChildInfo = Node.Children[ChildIndex];

		for (int32 DecoratorIndex = 0; DecoratorIndex < ChildInfo.Decorators.Num(); DecoratorIndex++)
		{
			ChildInfo.Decorators[DecoratorIndex]->CleanupInSubtree(OwnerComp, ChildInfo.Decorators[DecoratorIndex]->GetNodeMemory<uint8>(*this), CleanupType);
		}

		if (ChildInfo.ChildComposite)
		{
			CleanupNodes(OwnerComp, *(ChildInfo.ChildComposite), CleanupType);
		}
		else if (ChildInfo.ChildTask)
		{
			for (int32 ServiceIndex = 0; ServiceIndex < ChildInfo.ChildTask->Services.Num(); ServiceIndex++)
			{
				ChildInfo.ChildTask->Services[ServiceIndex]->CleanupInSubtree(OwnerComp, ChildInfo.ChildTask->Services[ServiceIndex]->GetNodeMemory<uint8>(*this), CleanupType);
			}

			ChildInfo.ChildTask->CleanupInSubtree(OwnerComp, ChildInfo.ChildTask->GetNodeMemory<uint8>(*this), CleanupType);
		}
	}
}

#if STATS

void FBehaviorTreeInstance::IncMemoryStats() const
{
	INC_MEMORY_STAT_BY(STAT_AI_BehaviorTree_InstanceMemory, GetAllocatedSize());
}

void FBehaviorTreeInstance::DecMemoryStats() const
{
	DEC_MEMORY_STAT_BY(STAT_AI_BehaviorTree_InstanceMemory, GetAllocatedSize());
}

uint32 FBehaviorTreeInstance::GetAllocatedSize() const
{
	return sizeof(*this) + ActiveAuxNodes.GetAllocatedSize() + ParallelTasks.GetAllocatedSize() + InstanceMemory.GetAllocatedSize();
}

#define MEM_STAT_UPDATE_WRAPPER(cmd) \
	DecMemoryStats();\
	cmd; \
	IncMemoryStats();

#else

#define MEM_STAT_UPDATE_WRAPPER(cmd) cmd;

#endif // STATS

FBehaviorTreeInstance::FBehaviorTreeInstance()
{
	IncMemoryStats(); 
	INC_DWORD_STAT(STAT_AI_BehaviorTree_NumInstances);
}

FBehaviorTreeInstance::FBehaviorTreeInstance(const FBehaviorTreeInstance& Other)
	: RootNode(Other.RootNode)
	, ActiveNode(Other.ActiveNode)
	, InstanceIdIndex(Other.InstanceIdIndex)
	, ActiveNodeType(Other.ActiveNodeType)
	, DeactivationNotify(Other.DeactivationNotify)
{
	ActiveAuxNodes = Other.ActiveAuxNodes;
	ParallelTasks = Other.ParallelTasks;
	InstanceMemory = Other.InstanceMemory;

#if DO_ENSURE
	bIteratingNodes = Other.bIteratingNodes;
	ParallelTaskIndex = Other.ParallelTaskIndex;
#endif // DO_ENSURE

	IncMemoryStats();
	INC_DWORD_STAT(STAT_AI_BehaviorTree_NumInstances);
}

FBehaviorTreeInstance::FBehaviorTreeInstance(int32 MemorySize)
{
	InstanceMemory.AddZeroed(MemorySize);
	IncMemoryStats();
	INC_DWORD_STAT(STAT_AI_BehaviorTree_NumInstances);
}

FBehaviorTreeInstance::FBehaviorTreeInstance(FBehaviorTreeInstance&& Other)
	: RootNode(Other.RootNode)
	, ActiveNode(Other.ActiveNode)
	, InstanceIdIndex(Other.InstanceIdIndex)
	, ActiveNodeType(Other.ActiveNodeType)
	, DeactivationNotify(Other.DeactivationNotify)
{
	ActiveAuxNodes = MoveTemp(Other.ActiveAuxNodes);
	ParallelTasks = MoveTemp(Other.ParallelTasks);
	InstanceMemory = MoveTemp(Other.InstanceMemory);

#if DO_ENSURE
	bIteratingNodes = Other.bIteratingNodes;
	ParallelTaskIndex = Other.ParallelTaskIndex;
#endif // DO_ENSURE

	IncMemoryStats();
	INC_DWORD_STAT(STAT_AI_BehaviorTree_NumInstances);
}

FBehaviorTreeInstance& FBehaviorTreeInstance::operator=(FBehaviorTreeInstance&& Other)
{
	RootNode = Other.RootNode;
	ActiveNode = Other.ActiveNode;
	InstanceIdIndex = Other.InstanceIdIndex;
	ActiveNodeType = Other.ActiveNodeType;
	DeactivationNotify = Other.DeactivationNotify;

	ActiveAuxNodes = MoveTemp(Other.ActiveAuxNodes);
	ParallelTasks = MoveTemp(Other.ParallelTasks);
	InstanceMemory = MoveTemp(Other.InstanceMemory);

#if DO_ENSURE
	bIteratingNodes = Other.bIteratingNodes;
	ParallelTaskIndex = Other.ParallelTaskIndex;
#endif // DO_ENSURE

	IncMemoryStats();
	INC_DWORD_STAT(STAT_AI_BehaviorTree_NumInstances);

	return *this;
}

FBehaviorTreeInstance::~FBehaviorTreeInstance()
{
	DecMemoryStats();
	DEC_DWORD_STAT(STAT_AI_BehaviorTree_NumInstances);
}

void FBehaviorTreeInstance::AddToActiveAuxNodes(UBTAuxiliaryNode* AuxNode)
{
	AddToActiveAuxNodesImpl(AuxNode);
}

void FBehaviorTreeInstance::AddToActiveAuxNodesImpl(UBTAuxiliaryNode* AuxNode)
{
#if DO_ENSURE
	ensureAlwaysMsgf(bIteratingNodes == false, TEXT("Adding aux node while iterating through them is not allowed."));
#endif // DO_ENSURE

	MEM_STAT_UPDATE_WRAPPER(ActiveAuxNodes.Add(AuxNode));
}

void FBehaviorTreeInstance::AddToActiveAuxNodes(UBehaviorTreeComponent& OwnerComp, UBTAuxiliaryNode* AuxNode)
{
	UE_VLOG(OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT("%hs %s")
		, __FUNCTION__
		, *UBehaviorTreeTypes::DescribeNodeHelper(AuxNode));

	AddToActiveAuxNodesImpl(AuxNode);
}

void FBehaviorTreeInstance::RemoveFromActiveAuxNodes(UBTAuxiliaryNode* AuxNode)
{
	RemoveFromActiveAuxNodesImpl(AuxNode);
}

void FBehaviorTreeInstance::RemoveFromActiveAuxNodesImpl(UBTAuxiliaryNode* AuxNode)
{
#if DO_ENSURE
	ensureAlwaysMsgf(bIteratingNodes == false, TEXT("Removing aux node while iterating through them is not allowed."));
#endif // DO_ENSURE

	MEM_STAT_UPDATE_WRAPPER(ActiveAuxNodes.RemoveSingleSwap(AuxNode));
}

void FBehaviorTreeInstance::RemoveFromActiveAuxNodes(UBehaviorTreeComponent& OwnerComp, UBTAuxiliaryNode* AuxNode)
{
	UE_VLOG(OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT("%hs %s")
		, __FUNCTION__
		, *UBehaviorTreeTypes::DescribeNodeHelper(AuxNode));

	RemoveFromActiveAuxNodesImpl(AuxNode);
}

void FBehaviorTreeInstance::ResetActiveAuxNodes()
{
#if DO_ENSURE
	ensureAlwaysMsgf(bIteratingNodes == false, TEXT("Resetting aux node list while iterating through them is not allowed."));
#endif // DO_ENSURE
	MEM_STAT_UPDATE_WRAPPER(ActiveAuxNodes.Reset());
}

void FBehaviorTreeInstance::AddToParallelTasks(FBehaviorTreeParallelTask&& ParallelTask)
{
#if DO_ENSURE
	ensureMsgf(ParallelTaskIndex == INDEX_NONE, TEXT("Adding to the the list of parallel tasks from ExecuteOnEachParallelTask is not allowed."));
#endif // DO_ENSURE
	MEM_STAT_UPDATE_WRAPPER(ParallelTasks.Add(ParallelTask));
}

void FBehaviorTreeInstance::RemoveParallelTaskAt(int32 TaskIndex)
{
	check(ParallelTasks.IsValidIndex(TaskIndex));
#if DO_ENSURE
	ensureMsgf(ParallelTaskIndex == INDEX_NONE || ParallelTaskIndex == TaskIndex, 
		TEXT("Removing from the list of parallel tasks from ExecuteOnEachParallelTask is only supported for the current task. Otherwise the iteration is broken."));
#endif // DO_ENSURE

	MEM_STAT_UPDATE_WRAPPER(ParallelTasks.RemoveAt(TaskIndex, /*Count=*/1, EAllowShrinking::No));
}

void FBehaviorTreeInstance::MarkParallelTaskAsAbortingAt(int32 TaskIndex)
{
	check(ParallelTasks.IsValidIndex(TaskIndex));
	ParallelTasks[TaskIndex].Status = EBTTaskStatus::Aborting;
}

void FBehaviorTreeInstance::SetInstanceMemory(const TArray<uint8>& Memory)
{
	MEM_STAT_UPDATE_WRAPPER(InstanceMemory = Memory);
}

#undef MEM_STAT_UPDATE_WRAPPER

void FBehaviorTreeInstance::ExecuteOnEachAuxNode(TFunctionRef<void(const UBTAuxiliaryNode&)> ExecFunc)
{
#if DO_ENSURE
	TGuardValue<bool> IteratingGuard(bIteratingNodes, true);
#endif // DO_ENSURE

	for (const UBTAuxiliaryNode* AuxNode : ActiveAuxNodes)
	{
		check(AuxNode != NULL);
		ExecFunc(*AuxNode);
	}
}

void FBehaviorTreeInstance::ExecuteOnEachParallelTask(TFunctionRef<void(const FBehaviorTreeParallelTask&, const int32)> ExecFunc)
{
	// calling ExecFunc might unregister parallel task, modifying array we're iterating on - iterator needs to be moved one step back in that case
	for (int32 Index = 0; Index < ParallelTasks.Num(); ++Index)
	{
		const FBehaviorTreeParallelTask& ParallelTaskInfo = ParallelTasks[Index];
		const UBTTaskNode* CachedParallelTask = ParallelTaskInfo.TaskNode;
		const int32 CachedNumTasks = ParallelTasks.Num();

#if DO_ENSURE
		ensureAlways(ParallelTaskIndex == INDEX_NONE);
		TGuardValue<int32> IndexGuard(ParallelTaskIndex, Index);
#endif // DO_ENSURE

		ExecFunc(ParallelTaskInfo, Index);

		const bool bIsStillValid = ParallelTasks.IsValidIndex(Index) && (ParallelTaskInfo.TaskNode == CachedParallelTask);
		if (!bIsStillValid)
		{
			// move iterator back if current task was unregistered
			Index--;
		}
	}
}

bool FBehaviorTreeInstance::HasActiveNode(uint16 TestExecutionIndex) const
{
	if (ActiveNode && ActiveNode->GetExecutionIndex() == TestExecutionIndex)
	{
		return (ActiveNodeType == EBTActiveNode::ActiveTask);
	}

	for (int32 Idx = 0; Idx < ParallelTasks.Num(); Idx++)
	{
		const FBehaviorTreeParallelTask& ParallelTask = ParallelTasks[Idx];
		if (ParallelTask.TaskNode && ParallelTask.TaskNode->GetExecutionIndex() == TestExecutionIndex)
		{
			return (ParallelTask.Status == EBTTaskStatus::Active);
		}
	}

	for (int32 Idx = 0; Idx < ActiveAuxNodes.Num(); Idx++)
	{
		if (ActiveAuxNodes[Idx] && ActiveAuxNodes[Idx]->GetExecutionIndex() == TestExecutionIndex)
		{
			return true;
		}
	}

	return false;
}

void FBehaviorTreeInstance::DeactivateNodes(FBehaviorTreeSearchData& SearchData, uint16 InstanceIndex)
{
	for (int32 Idx = SearchData.PendingUpdates.Num() - 1; Idx >= 0; Idx--)
	{
		FBehaviorTreeSearchUpdate& UpdateInfo = SearchData.PendingUpdates[Idx];
		if (UpdateInfo.InstanceIndex == InstanceIndex && UpdateInfo.Mode == EBTNodeUpdateMode::Add)
		{
			UE_VLOG(SearchData.OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT("Search node update[%s]: %s"),
				*UBehaviorTreeTypes::DescribeNodeUpdateMode(EBTNodeUpdateMode::Remove),
				*UBehaviorTreeTypes::DescribeNodeHelper(UpdateInfo.AuxNode ? (UBTNode*)UpdateInfo.AuxNode : (UBTNode*)UpdateInfo.TaskNode));

			SearchData.PendingUpdates.RemoveAt(Idx, EAllowShrinking::No);
		}
	}

	for (int32 Idx = 0; Idx < ParallelTasks.Num(); Idx++)
	{
		const FBehaviorTreeParallelTask& ParallelTask = ParallelTasks[Idx];
		if (ParallelTask.TaskNode && ParallelTask.Status == EBTTaskStatus::Active)
		{
			SearchData.AddUniqueUpdate(FBehaviorTreeSearchUpdate(ParallelTask.TaskNode, InstanceIndex, EBTNodeUpdateMode::Remove));
		}
	}

	for (int32 Idx = 0; Idx < ActiveAuxNodes.Num(); Idx++)
	{
		if (ActiveAuxNodes[Idx])
		{
			SearchData.AddUniqueUpdate(FBehaviorTreeSearchUpdate(ActiveAuxNodes[Idx], InstanceIndex, EBTNodeUpdateMode::Remove));
		}
	}
}


//----------------------------------------------------------------------//
// FBTNodeIndex
//----------------------------------------------------------------------//

bool FBTNodeIndex::TakesPriorityOver(const FBTNodeIndex& Other) const
{
	// instance closer to root is more important
	if (InstanceIndex != Other.InstanceIndex)
	{
		return InstanceIndex < Other.InstanceIndex;
	}

	// higher priority is more important
	return ExecutionIndex < Other.ExecutionIndex;
}

//----------------------------------------------------------------------//
// FBehaviorTreeSearchData
//----------------------------------------------------------------------//

int32 FBehaviorTreeSearchData::NextSearchId = 1;

void FBehaviorTreeSearchData::AddUniqueUpdate(const FBehaviorTreeSearchUpdate& UpdateInfo)
{
	UE_VLOG(OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT("Search node update[%s]: %s"),
		*UBehaviorTreeTypes::DescribeNodeUpdateMode(UpdateInfo.Mode),
		*UBehaviorTreeTypes::DescribeNodeHelper(UpdateInfo.AuxNode ? (UBTNode*)UpdateInfo.AuxNode : (UBTNode*)UpdateInfo.TaskNode));

	bool bSkipAdding = false;
	for (int32 UpdateIndex = 0; UpdateIndex < PendingUpdates.Num(); UpdateIndex++)
	{
		const FBehaviorTreeSearchUpdate& Info = PendingUpdates[UpdateIndex];
		if (Info.AuxNode == UpdateInfo.AuxNode && Info.TaskNode == UpdateInfo.TaskNode)
		{
			// duplicate, skip
			if (Info.Mode == UpdateInfo.Mode)
			{
				UE_VLOG(OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT(">> skipped: duplicated operation"));
				bSkipAdding = true;
				break;
			}

			// don't add pairs add-remove
			bSkipAdding = (Info.Mode == EBTNodeUpdateMode::Remove) || (UpdateInfo.Mode == EBTNodeUpdateMode::Remove);
			UE_CVLOG(bSkipAdding, OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT(">> skipped: paired add/remove"));

			PendingUpdates.RemoveAt(UpdateIndex, EAllowShrinking::No);
		}
	}
	
	// don't add Remove updates for inactive aux nodes, as they will block valid Add update coming later from the same search
	// check only aux nodes, it happens due to UBTCompositeNode::NotifyDecoratorsOnActivation
	if (!bSkipAdding && UpdateInfo.Mode == EBTNodeUpdateMode::Remove && UpdateInfo.AuxNode)
	{
		const bool bIsActive = OwnerComp.IsAuxNodeActive(UpdateInfo.AuxNode, UpdateInfo.InstanceIndex);
		bSkipAdding = !bIsActive;
		UE_CVLOG(bSkipAdding, OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT(">> skipped: did not push a remove to PendingUpdates due to inactive aux node"));
	}

	if (!bSkipAdding)
	{
		const int32 Idx = PendingUpdates.Add(UpdateInfo);
		PendingUpdates[Idx].bPostUpdate = (UpdateInfo.Mode == EBTNodeUpdateMode::Add) && (Cast<UBTService>(UpdateInfo.AuxNode) != NULL);
	}
}

void FBehaviorTreeSearchData::AssignSearchId()
{
	SearchId = NextSearchId;
	NextSearchId++;
}

void FBehaviorTreeSearchData::Reset()
{
	PendingUpdates.Reset();
	PendingNotifies.Reset();
	SearchRootNode = FBTNodeIndex();
	SearchStart = FBTNodeIndex();
	SearchEnd = FBTNodeIndex();
	RollbackInstanceIdx = INDEX_NONE;
	DeactivatedBranchStart = FBTNodeIndex();
	DeactivatedBranchEnd = FBTNodeIndex();
	bFilterOutRequestFromDeactivatedBranch = false;
	bSearchInProgress = false;
	bPostponeSearch = false;
	bPreserveActiveNodeMemoryOnRollback = false;
}

//----------------------------------------------------------------------//
// FBlackboardKeySelector
//----------------------------------------------------------------------//
void FBlackboardKeySelector::ResolveSelectedKey(const UBlackboardData& BlackboardAsset)
{
	if (SelectedKeyName.IsNone() == false || !bNoneIsAllowedValue)
	{
		if (SelectedKeyName.IsNone() && !bNoneIsAllowedValue)
		{
			InitSelection(BlackboardAsset);
		}

		SelectedKeyID = BlackboardAsset.GetKeyID(SelectedKeyName);
		SelectedKeyType = BlackboardAsset.GetKeyType(FBlackboard::FKey(IntCastChecked<uint16>(SelectedKeyID)));
		UE_CLOGF(IsSet() == false, LogBehaviorTree, Warning
			, "%ls> Failed to find key \'%ls\' in BB asset %ls. BB Key Selector will be set to \'Invalid\'"
			, *UBehaviorTreeTypes::GetBTLoggingContext()
			, *SelectedKeyName.ToString()
			, *BlackboardAsset.GetFullName()
		);
	}
}

void FBlackboardKeySelector::InitSelection(const UBlackboardData& BlackboardAsset)
{
	for (const UBlackboardData* It = &BlackboardAsset; It; It = It->Parent)
	{
		for (int32 KeyIndex = 0; KeyIndex < It->Keys.Num(); KeyIndex++)
		{
			const FBlackboardEntry& EntryInfo = It->Keys[KeyIndex];
			if (EntryInfo.KeyType)
			{
				bool bFilterPassed = true;
				if (AllowedTypes.Num())
				{
					bFilterPassed = false;
					for (int32 FilterIndex = 0; FilterIndex < AllowedTypes.Num(); FilterIndex++)
					{
						if (EntryInfo.KeyType->IsAllowedByFilter(AllowedTypes[FilterIndex]))
						{
							bFilterPassed = true;
							break;
						}
					}
				}

				if (bFilterPassed)
				{
					SelectedKeyName = EntryInfo.EntryName;
					break;
				}
			}
		}
	}
}

namespace UE::BlackboardKeySelector
{
	template <typename T>
	TNotNull<T*> CreateFilter(const FName PropertyName) requires TIsDerivedFrom<T, UBlackboardKeyType>::IsDerived
	{
		return static_cast<T*>(NewObject<T>(
			GetTransientPackage()
			, MakeUniqueObjectName(GetTransientPackage(), T::StaticClass(), *FString::Printf(TEXT("%s_%s"), *PropertyName.ToString(), *T::StaticClass()->GetName()))
			, RF_Public));
	}
} // UE::BlackboardKeySelector

void FBlackboardKeySelector::AddObjectFilter(UObject*, const FName PropertyName, TSubclassOf<UObject> AllowedClass)
{
	UBlackboardKeyType_Object* FilterOb = UE::BlackboardKeySelector::CreateFilter<UBlackboardKeyType_Object>(PropertyName);
	FilterOb->BaseClass = AllowedClass;
	AllowedTypes.Add(FilterOb);
}

void FBlackboardKeySelector::AddClassFilter(UObject*, const FName PropertyName, TSubclassOf<UObject> AllowedClass)
{
	UBlackboardKeyType_Class* FilterOb = UE::BlackboardKeySelector::CreateFilter<UBlackboardKeyType_Class>(PropertyName);
	FilterOb->BaseClass = AllowedClass;
	AllowedTypes.Add(FilterOb);
}

void FBlackboardKeySelector::AddEnumFilter(UObject*, const FName PropertyName, UEnum* AllowedEnum)
{
	UBlackboardKeyType_Enum* FilterOb = UE::BlackboardKeySelector::CreateFilter<UBlackboardKeyType_Enum>(PropertyName);
	FilterOb->EnumType = AllowedEnum;
	AllowedTypes.Add(FilterOb);
}

void FBlackboardKeySelector::AddNativeEnumFilter(UObject*, const FName PropertyName, const FString& AllowedEnumName)
{
	UBlackboardKeyType_NativeEnum* FilterOb = UE::BlackboardKeySelector::CreateFilter<UBlackboardKeyType_NativeEnum>(PropertyName);
	FilterOb->EnumName = AllowedEnumName;
	AllowedTypes.Add(FilterOb);
}

void FBlackboardKeySelector::AddIntFilter(UObject*, const FName PropertyName)
{
	AllowedTypes.Add(UE::BlackboardKeySelector::CreateFilter<UBlackboardKeyType_Int>(PropertyName));
}

void FBlackboardKeySelector::AddFloatFilter(UObject*, const FName PropertyName)
{
	AllowedTypes.Add(UE::BlackboardKeySelector::CreateFilter<UBlackboardKeyType_Float>(PropertyName));
}

void FBlackboardKeySelector::AddBoolFilter(UObject*, const FName PropertyName)
{
	AllowedTypes.Add(UE::BlackboardKeySelector::CreateFilter<UBlackboardKeyType_Bool>(PropertyName));
}

void FBlackboardKeySelector::AddVectorFilter(UObject*, const FName PropertyName)
{
	AllowedTypes.Add(UE::BlackboardKeySelector::CreateFilter<UBlackboardKeyType_Vector>(PropertyName));
}

void FBlackboardKeySelector::AddRotatorFilter(UObject*, const FName PropertyName)
{
	AllowedTypes.Add(UE::BlackboardKeySelector::CreateFilter<UBlackboardKeyType_Rotator>(PropertyName));
}

void FBlackboardKeySelector::AddStringFilter(UObject*, const FName PropertyName)
{
	AllowedTypes.Add(UE::BlackboardKeySelector::CreateFilter<UBlackboardKeyType_String>(PropertyName));
}

void FBlackboardKeySelector::AddNameFilter(UObject*, const FName PropertyName)
{
	AllowedTypes.Add(UE::BlackboardKeySelector::CreateFilter<UBlackboardKeyType_Name>(PropertyName));
}

void FBlackboardKeySelector::AddStructFilter(UObject*, const FName PropertyName, const UScriptStruct* AllowedStruct)
{
	UBlackboardKeyType_Struct* FilterOb = UE::BlackboardKeySelector::CreateFilter<UBlackboardKeyType_Struct>(PropertyName);
	FilterOb->DefaultValue.InitializeAs(AllowedStruct);
	AllowedTypes.Add(FilterOb);
}

#if WITH_EDITOR
void FBlackboardKeySelector::PreSave(const UObject* Outer, const UBlackboardData& Blackboard, FName PropertyName)
{
	if(!SelectedKeyName.IsNone())
	{
		const TCHAR* ResetValueMessage = bNoneIsAllowedValue ? TEXT("Resetting selected key to none.") : TEXT("Setting the key to first matching key.");
		const FBlackboard::FKey Id = Blackboard.GetKeyID(SelectedKeyName);
		if (Id != FBlackboard::InvalidKey)
		{
			if (const FBlackboardEntry* BlackboardEntry = Blackboard.GetKey(Id))
			{
				if(AllowedTypes.Num() > 0)
				{
					for (UBlackboardKeyType* AllowedType : AllowedTypes)
					{
						if (BlackboardEntry->KeyType->IsAllowedByFilter(AllowedType))
						{
							return;
						}
					}
				}
				else
				{
					return;
				}
				UE_LOGF(LogBehaviorTree, Warning, "%ls in node %ls in bt %ls is bound to key %ls in blackboard %ls but the key type isn't allowed. %ls", *PropertyName.ToString(), *GetNameSafe(Outer), Outer ? *GetNameSafe(Outer->GetTypedOuter<UBehaviorTree>()) : TEXT("None"), *SelectedKeyName.ToString(), *Blackboard.GetFullName(), ResetValueMessage);
			}
			else
			{
				UE_LOGF(LogBehaviorTree, Warning, "%ls in node %ls in bt %ls is bound to key %ls in blackboard %ls but the key doesn't exist. %ls", *PropertyName.ToString(), *GetNameSafe(Outer), Outer ? *GetNameSafe(Outer->GetTypedOuter<UBehaviorTree>()) : TEXT("None"), *SelectedKeyName.ToString(), *Blackboard.GetFullName(), ResetValueMessage);
			}
		}
		else
		{
			UE_LOGF(LogBehaviorTree, Warning, "%ls in node %ls in bt %ls is bound to key %ls in blackboard %ls but the key doesn't exist. %ls", *PropertyName.ToString(), *GetNameSafe(Outer), Outer ? *GetNameSafe(Outer->GetTypedOuter<UBehaviorTree>()) : TEXT("None"), *SelectedKeyName.ToString(), *Blackboard.GetFullName(), ResetValueMessage);
		}

		if (bNoneIsAllowedValue)
		{
			InitSelection(Blackboard);
		}
		else
		{
			SelectedKeyName = NAME_None;
		}
	}
}
#endif // WITH_EDITOR

//----------------------------------------------------------------------//
// UBehaviorTreeTypes
//----------------------------------------------------------------------//
FString UBehaviorTreeTypes::BTLoggingContext;

FString UBehaviorTreeTypes::DescribeNodeResult(EBTNodeResult::Type NodeResult)
{
	static FString ResultDesc[] = { TEXT("Succeeded"), TEXT("Failed"), TEXT("Aborted"), TEXT("InProgress") };
	return (NodeResult < UE_ARRAY_COUNT(ResultDesc)) ? ResultDesc[NodeResult] : FString();
}

FString UBehaviorTreeTypes::DescribeFlowAbortMode(EBTFlowAbortMode::Type AbortMode)
{
	static FString AbortModeDesc[] = { TEXT("None"), TEXT("Lower Priority"), TEXT("Self"), TEXT("Both") };
	return (AbortMode < UE_ARRAY_COUNT(AbortModeDesc)) ? AbortModeDesc[AbortMode] : FString();
}

FString UBehaviorTreeTypes::DescribeActiveNode(EBTActiveNode::Type ActiveNodeType)
{
	static FString ActiveDesc[] = { TEXT("Composite"), TEXT("ActiveTask"), TEXT("AbortingTask"), TEXT("InactiveTask") };
	return (ActiveNodeType < UE_ARRAY_COUNT(ActiveDesc)) ? ActiveDesc[ActiveNodeType] : FString();
}

FString UBehaviorTreeTypes::DescribeTaskStatus(EBTTaskStatus::Type TaskStatus)
{
	static FString TaskStatusDesc[] = { TEXT("Active"), TEXT("Aborting"), TEXT("Inactive") };
	return (TaskStatus < UE_ARRAY_COUNT(TaskStatusDesc)) ? TaskStatusDesc[TaskStatus] : FString();
}

FString UBehaviorTreeTypes::DescribeNodeUpdateMode(EBTNodeUpdateMode::Type UpdateMode)
{
	static FString UpdateModeDesc[] = { TEXT("Unknown"), TEXT("Add"), TEXT("Remove") };
	return (UpdateMode < UE_ARRAY_COUNT(UpdateModeDesc)) ? UpdateModeDesc[UpdateMode] : FString();
}

FString UBehaviorTreeTypes::DescribeNodeHelper(const UBTNode* Node)
{
	return Node ? FString::Printf(TEXT("%s::%s[%d]"), *GetNameSafe(Node->GetTreeAsset()), *Node->GetNodeName(), Node->GetExecutionIndex()) : FString();
}

FString UBehaviorTreeTypes::GetShortTypeName(const UObject* Ob)
{
	if ((Ob == nullptr) || (Ob->GetClass() == nullptr))
	{
		return TEXT("None");
	}

	if (Ob->GetClass()->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
	{
		return Ob->GetClass()->GetName().LeftChop(2);
	}

	FString TypeDesc = Ob->GetClass()->GetName();
	const int32 ShortNameIdx = TypeDesc.Find(TEXT("_"), ESearchCase::CaseSensitive);
	if (ShortNameIdx != INDEX_NONE)
	{
		TypeDesc.MidInline(ShortNameIdx + 1, MAX_int32, EAllowShrinking::No);
	}

	return TypeDesc;
}

void UBehaviorTreeTypes::SetBTLoggingContext(const UBTNode* NewBTLoggingContext)
{
	BTLoggingContext = DescribeNodeHelper(NewBTLoggingContext);
}

//----------------------------------------------------------------------//
// DEPRECATED
//----------------------------------------------------------------------//
