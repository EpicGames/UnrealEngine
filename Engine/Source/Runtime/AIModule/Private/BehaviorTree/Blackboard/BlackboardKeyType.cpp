// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Blackboard/BlackboardKeyType.h"
#include "BehaviorTree/BlackboardComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlackboardKeyType)

UBlackboardKeyType::UBlackboardKeyType(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	ValueSize = 0;
	SupportedOp = EBlackboardKeyOperation::Basic;
	bCreateKeyInstance = false;
	bIsInstanced = false;
}

void UBlackboardKeyType::PreInitialize(UBlackboardComponent& OwnerComp)
{
	// empty in base class
}

void UBlackboardKeyType::InitializeKey(UBlackboardComponent& OwnerComp, FBlackboard::FKey KeyID)
{
	uint8* RawData = OwnerComp.GetKeyRawData(KeyID);

	if (bCreateKeyInstance)
	{
		FBlackboardInstancedKeyMemory* MyMemory = (FBlackboardInstancedKeyMemory*)RawData;
		UBlackboardKeyType* KeyInstance = DuplicateObject<UBlackboardKeyType>(this, &OwnerComp);
		KeyInstance->bIsInstanced = true;
		MyMemory->KeyIdx = KeyID;
		OwnerComp.KeyInstances[KeyID] = KeyInstance;

		uint8* InstanceMemoryBlock = RawData + sizeof(FBlackboardInstancedKeyMemory);
		KeyInstance->InitializeMemory(OwnerComp, InstanceMemoryBlock);
	}
	else
	{
		InitializeMemory(OwnerComp, RawData);
	}
}

UBlackboardKeyType* UBlackboardKeyType::GetKeyInstance(UBlackboardComponent& OwnerComp, const uint8* MemoryBlock) const
{
	FBlackboardInstancedKeyMemory* MyMemory = (FBlackboardInstancedKeyMemory*)MemoryBlock;
	return MyMemory && OwnerComp.KeyInstances.IsValidIndex(MyMemory->KeyIdx) ?
		OwnerComp.KeyInstances[MyMemory->KeyIdx] : NULL;
}

const UBlackboardKeyType* UBlackboardKeyType::GetKeyInstance(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock) const
{
	FBlackboardInstancedKeyMemory* MyMemory = (FBlackboardInstancedKeyMemory*)MemoryBlock;
	return MyMemory && OwnerComp.KeyInstances.IsValidIndex(MyMemory->KeyIdx) ?
		OwnerComp.KeyInstances[MyMemory->KeyIdx] : NULL;
}

void UBlackboardKeyType::InitializeMemory(UBlackboardComponent& OwnerComp, uint8* MemoryBlock)
{
	// empty in base class
}

void UBlackboardKeyType::WrappedFree(UBlackboardComponent& OwnerComp, uint8* MemoryBlock)
{
	if (HasInstance())
	{
		UBlackboardKeyType* InstancedKey = GetKeyInstance(OwnerComp, MemoryBlock);
		uint8* InstanceMemoryBlock = MemoryBlock + sizeof(FBlackboardInstancedKeyMemory);
		return InstancedKey->FreeMemory(OwnerComp, InstanceMemoryBlock);
	}

	return FreeMemory(OwnerComp, MemoryBlock);
}

void UBlackboardKeyType::FreeMemory(UBlackboardComponent& OwnerComp, uint8* MemoryBlock)
{
	// empty in base class
}

bool UBlackboardKeyType::WrappedGetLocation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, FVector& Location) const
{
	if (HasInstance())
	{
		const UBlackboardKeyType* InstancedKey = GetKeyInstance(OwnerComp, MemoryBlock);
		const uint8* InstanceMemoryBlock = MemoryBlock + sizeof(FBlackboardInstancedKeyMemory);
		return InstancedKey->GetLocation(OwnerComp, InstanceMemoryBlock, Location);
	}

	return GetLocation(OwnerComp, MemoryBlock, Location);
}

bool UBlackboardKeyType::GetLocation(const UBlackboardComponent& OwnerComp, const uint8* RawData, FVector& Location) const
{
	return false;
}

bool UBlackboardKeyType::WrappedGetRotation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, FRotator& Rotation) const
{
	if (HasInstance())
	{
		const UBlackboardKeyType* InstancedKey = GetKeyInstance(OwnerComp, MemoryBlock);
		const uint8* InstanceMemoryBlock = MemoryBlock + sizeof(FBlackboardInstancedKeyMemory);
		return InstancedKey->GetRotation(OwnerComp, InstanceMemoryBlock, Rotation);
	}

	return GetRotation(OwnerComp, MemoryBlock, Rotation);
}

bool UBlackboardKeyType::GetRotation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, FRotator& Rotation) const
{
	return false;
}

void UBlackboardKeyType::WrappedClear(const UBlackboardComponent& OwnerComp, uint8* MemoryBlock) const
{
	UBlackboardComponent& MutableOwner = (UBlackboardComponent&)OwnerComp;
	if (HasInstance())
	{
		UBlackboardKeyType* InstancedKey = GetKeyInstance(MutableOwner, MemoryBlock);
		uint8* InstanceMemoryBlock = MemoryBlock + sizeof(FBlackboardInstancedKeyMemory);
		InstancedKey->Clear(MutableOwner, InstanceMemoryBlock);
	}
	else
	{
		UBlackboardKeyType* MutableKey = (UBlackboardKeyType*)this;
		MutableKey->Clear(MutableOwner, MemoryBlock);
	}
}

void UBlackboardKeyType::Clear(UBlackboardComponent& OwnerComp, uint8* MemoryBlock)
{
	FMemory::Memzero(MemoryBlock, GetValueSize());
}

bool UBlackboardKeyType::WrappedIsEmpty(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock) const
{
	if (HasInstance())
	{
		const UBlackboardKeyType* InstancedKey = GetKeyInstance(OwnerComp, MemoryBlock);
		const uint8* InstanceMemoryBlock = MemoryBlock + sizeof(FBlackboardInstancedKeyMemory);
		return InstancedKey->IsEmpty(OwnerComp, InstanceMemoryBlock);
	}

	return IsEmpty(OwnerComp, MemoryBlock);
}

bool UBlackboardKeyType::IsEmpty(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock) const
{
	for (int32 ByteIndex = 0; ByteIndex < GetValueSize(); ++ByteIndex)
	{
		if (MemoryBlock[ByteIndex] != uint8(0))
		{
			return false;
		}
	}

	return true;
}

bool UBlackboardKeyType::WrappedTestBasicOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, EBasicKeyOperation::Type Op) const
{
	if (HasInstance())
	{
		const UBlackboardKeyType* InstancedKey = GetKeyInstance(OwnerComp, MemoryBlock);
		const uint8* InstanceMemoryBlock = MemoryBlock + sizeof(FBlackboardInstancedKeyMemory);
		return InstancedKey->TestBasicOperation(OwnerComp, InstanceMemoryBlock, Op);
	}

	return TestBasicOperation(OwnerComp, MemoryBlock, Op);
}

bool UBlackboardKeyType::TestBasicOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, EBasicKeyOperation::Type Op) const
{
	return false;
}

bool UBlackboardKeyType::WrappedTestArithmeticOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, EArithmeticKeyOperation::Type Op, int32 OtherIntValue, float OtherFloatValue) const
{
	if (HasInstance())
	{
		const UBlackboardKeyType* InstancedKey = GetKeyInstance(OwnerComp, MemoryBlock);
		const uint8* InstanceMemoryBlock = MemoryBlock + sizeof(FBlackboardInstancedKeyMemory);
		return InstancedKey->TestArithmeticOperation(OwnerComp, InstanceMemoryBlock, Op, OtherIntValue, OtherFloatValue);
	}

	return TestArithmeticOperation(OwnerComp, MemoryBlock, Op, OtherIntValue, OtherFloatValue);
}

bool UBlackboardKeyType::TestArithmeticOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, EArithmeticKeyOperation::Type Op, int32 OtherIntValue, float OtherFloatValue) const
{
	return false;
}

bool UBlackboardKeyType::WrappedTestTextOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, ETextKeyOperation::Type Op, const FString& OtherString) const
{
	if (HasInstance())
	{
		const UBlackboardKeyType* InstancedKey = GetKeyInstance(OwnerComp, MemoryBlock);
		const uint8* InstanceMemoryBlock = MemoryBlock + sizeof(FBlackboardInstancedKeyMemory);
		return InstancedKey->TestTextOperation(OwnerComp, InstanceMemoryBlock, Op, OtherString);
	}

	return TestTextOperation(OwnerComp, MemoryBlock, Op, OtherString);
}

bool UBlackboardKeyType::TestTextOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, ETextKeyOperation::Type Op, const FString& OtherString) const
{
	return false;
}

FString UBlackboardKeyType::WrappedDescribeValue(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock) const
{
	if (HasInstance())
	{
		const UBlackboardKeyType* InstancedKey = GetKeyInstance(OwnerComp, MemoryBlock);
		const uint8* InstanceMemoryBlock = MemoryBlock + sizeof(FBlackboardInstancedKeyMemory);
		return InstancedKey->DescribeValue(OwnerComp, InstanceMemoryBlock);
	}

	return DescribeValue(OwnerComp, MemoryBlock);
}

FString UBlackboardKeyType::DescribeValue(const UBlackboardComponent& OwnerComp, const uint8* RawData) const
{
	FString DescBytes;
	for (int32 ValueIndex = 0; ValueIndex < ValueSize; ValueIndex++)
	{
		DescBytes += FString::Printf(TEXT("%X"), RawData[ValueIndex]);
	}

	return DescBytes.Len() ? (FString("0x") + DescBytes) : FString("empty");
}

EBlackboardCompare::Type UBlackboardKeyType::CompareValues(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock,
	const UBlackboardKeyType* OtherKeyOb, const uint8* OtherMemoryBlock) const
{
	const int32 MemCmp = FMemory::Memcmp(MemoryBlock, OtherMemoryBlock, GetValueSize());
	return MemCmp ? EBlackboardCompare::NotEqual : EBlackboardCompare::Equal;
}

void UBlackboardKeyType::CopyValues(UBlackboardComponent& OwnerComp, uint8* MemoryBlock, const UBlackboardKeyType* SourceKeyOb, const uint8* SourceBlock)
{
	FMemory::Memcpy(MemoryBlock, SourceBlock, GetValueSize());
}

bool UBlackboardKeyType::IsAllowedByFilter(UBlackboardKeyType* FilterOb) const
{
	return GetClass() == (FilterOb ? FilterOb->GetClass() : NULL);
}

FString UBlackboardKeyType::DescribeSelf() const
{
	return FString();
}

FString UBlackboardKeyType::DescribeArithmeticParam(int32 IntValue, float FloatValue) const
{
	return FString();
}

//----------------------------------------------------------------------//
// DEPRECATED
//----------------------------------------------------------------------//
UBlackboardKeyType* UBlackboardKeyType::UpdateDeprecatedKey()
{
	return nullptr;
}

