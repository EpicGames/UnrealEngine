// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Blackboard/BlackboardKeyType_Bool.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlackboardKeyType_Bool)

const UBlackboardKeyType_Bool::FDataType UBlackboardKeyType_Bool::InvalidValue = false;

UBlackboardKeyType_Bool::UBlackboardKeyType_Bool(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	ValueSize = sizeof(uint8);
	SupportedOp = EBlackboardKeyOperation::Basic;
}

bool UBlackboardKeyType_Bool::GetValue(const UBlackboardKeyType_Bool* KeyOb, const uint8* RawData)
{
	return GetValueFromMemory<uint8>(RawData) != 0;
}

bool UBlackboardKeyType_Bool::SetValue(UBlackboardKeyType_Bool* KeyOb, uint8* RawData, bool bValue)
{
	return SetValueInMemory<uint8>(RawData, bValue);
}

EBlackboardCompare::Type UBlackboardKeyType_Bool::CompareValues(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock,
	const UBlackboardKeyType* OtherKeyOb, const uint8* OtherMemoryBlock) const
{
	const bool MyValue = GetValue(this, MemoryBlock);
	const bool OtherValue = GetValue((UBlackboardKeyType_Bool*)OtherKeyOb, OtherMemoryBlock);

	return (MyValue == OtherValue) ? EBlackboardCompare::Equal : EBlackboardCompare::NotEqual;
}

void UBlackboardKeyType_Bool::InitializeMemory(UBlackboardComponent& OwnerComp, uint8* MemoryBlock)
{
	SetValue(this, MemoryBlock, bDefaultValue);
}

FString UBlackboardKeyType_Bool::DescribeValue(const UBlackboardComponent& OwnerComp, const uint8* RawData) const
{
	return GetValue(this, RawData) ? TEXT("true") : TEXT("false");
}

bool UBlackboardKeyType_Bool::TestBasicOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, EBasicKeyOperation::Type Op) const
{
	const bool Value = GetValue(this, MemoryBlock);
	return (Op == EBasicKeyOperation::Set) ? Value : !Value;
}

