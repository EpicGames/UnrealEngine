// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Blackboard/BlackboardKeyType_Class.h"
#include "UObject/WeakObjectPtr.h"
#include "Templates/Casts.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlackboardKeyType_Class)

const UBlackboardKeyType_Class::FDataType UBlackboardKeyType_Class::InvalidValue = nullptr;

UBlackboardKeyType_Class::UBlackboardKeyType_Class(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	ValueSize = sizeof(TWeakObjectPtr<UClass>);
	BaseClass = UObject::StaticClass();
	SupportedOp = EBlackboardKeyOperation::Basic;
}

UClass* UBlackboardKeyType_Class::GetValue(const UBlackboardKeyType_Class* KeyOb, const uint8* RawData)
{
	TWeakObjectPtr<UClass> WeakObjPtr = GetValueFromMemory< TWeakObjectPtr<UClass> >(RawData);
	return WeakObjPtr.Get();
}

bool UBlackboardKeyType_Class::SetValue(UBlackboardKeyType_Class* KeyOb, uint8* RawData, UClass* Value)
{
	TWeakObjectPtr<UClass> WeakObjPtr(Value);
	return SetWeakObjectInMemory<UClass>(RawData, WeakObjPtr);
}

EBlackboardCompare::Type UBlackboardKeyType_Class::CompareValues(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock,
	const UBlackboardKeyType* OtherKeyOb, const uint8* OtherMemoryBlock) const
{
	const UClass* MyValue = GetValue(this, MemoryBlock);
	const UClass* OtherValue = GetValue((UBlackboardKeyType_Class*)OtherKeyOb, OtherMemoryBlock);

	return (MyValue == OtherValue) ? EBlackboardCompare::Equal : EBlackboardCompare::NotEqual;
}

void UBlackboardKeyType_Class::InitializeMemory(UBlackboardComponent& OwnerComp, uint8* MemoryBlock)
{
	SetValue(this, MemoryBlock, DefaultValue);
}

FString UBlackboardKeyType_Class::DescribeValue(const UBlackboardComponent& OwnerComp, const uint8* RawData) const
{
	return *GetNameSafe(GetValue(this, RawData));
}

FString UBlackboardKeyType_Class::DescribeSelf() const
{
	return *GetNameSafe(BaseClass);
}

bool UBlackboardKeyType_Class::IsAllowedByFilter(UBlackboardKeyType* FilterOb) const
{
	UBlackboardKeyType_Class* FilterClass = Cast<UBlackboardKeyType_Class>(FilterOb);
	return (FilterClass && (FilterClass->BaseClass == BaseClass || BaseClass->IsChildOf(FilterClass->BaseClass)));
}

bool UBlackboardKeyType_Class::TestBasicOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, EBasicKeyOperation::Type Op) const
{
	TWeakObjectPtr<UClass> WeakObjPtr = GetValueFromMemory< TWeakObjectPtr<UClass> >(MemoryBlock);
	return (Op == EBasicKeyOperation::Set) ? WeakObjPtr.IsValid() : !WeakObjPtr.IsValid();
}

