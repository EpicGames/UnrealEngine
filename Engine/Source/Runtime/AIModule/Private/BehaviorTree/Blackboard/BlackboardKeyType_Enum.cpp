// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Blackboard/BlackboardKeyType_Enum.h"
#include "BehaviorTree/BTNode.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlackboardKeyType_Enum)

#if WITH_EDITOR
#include "Misc/MessageDialog.h"
#include <limits>
#endif // WITH_EDITOR

const UBlackboardKeyType_Enum::FDataType UBlackboardKeyType_Enum::InvalidValue = UBlackboardKeyType_Enum::FDataType(0);

UBlackboardKeyType_Enum::UBlackboardKeyType_Enum(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	ValueSize = sizeof(FDataType);
	SupportedOp = EBlackboardKeyOperation::Arithmetic;
}

UBlackboardKeyType_Enum::FDataType UBlackboardKeyType_Enum::GetValue(const UBlackboardKeyType_Enum* KeyOb, const uint8* RawData)
{
	return GetValueFromMemory<FDataType>(RawData);
}

bool UBlackboardKeyType_Enum::SetValue(UBlackboardKeyType_Enum* KeyOb, uint8* RawData, const FDataType Value)
{
	return SetValueInMemory<FDataType>(RawData, Value);
}

EBlackboardCompare::Type UBlackboardKeyType_Enum::CompareValues(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock,
	const UBlackboardKeyType* OtherKeyOb, const uint8* OtherMemoryBlock) const
{
	const FDataType MyValue = GetValue(this, MemoryBlock);
	const FDataType OtherValue = GetValue((UBlackboardKeyType_Enum*)OtherKeyOb, OtherMemoryBlock);

	return (MyValue > OtherValue) ? EBlackboardCompare::Greater :
		(MyValue < OtherValue) ? EBlackboardCompare::Less :
		EBlackboardCompare::Equal;
}

FString UBlackboardKeyType_Enum::DescribeValue(const UBlackboardComponent& OwnerComp, const uint8* RawData) const
{
	return EnumType ? EnumType->GetDisplayNameTextByValue(GetValue(this, RawData)).ToString() : FString("UNKNOWN!");
}

FString UBlackboardKeyType_Enum::DescribeSelf() const
{
	return *GetNameSafe(EnumType);
}

bool UBlackboardKeyType_Enum::IsAllowedByFilter(UBlackboardKeyType* FilterOb) const
{
	UBlackboardKeyType_Enum* FilterEnum = Cast<UBlackboardKeyType_Enum>(FilterOb);
	return (FilterEnum && FilterEnum->EnumType == EnumType);
}

bool UBlackboardKeyType_Enum::TestArithmeticOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, EArithmeticKeyOperation::Type Op, int32 OtherIntValue, float OtherFloatValue) const
{
	const FDataType Value = GetValue(this, MemoryBlock);
	switch (Op)
	{
	case EArithmeticKeyOperation::Equal:			return (Value == OtherIntValue);
	case EArithmeticKeyOperation::NotEqual:			return (Value != OtherIntValue);
	case EArithmeticKeyOperation::Less:				return (Value < OtherIntValue);
	case EArithmeticKeyOperation::LessOrEqual:		return (Value <= OtherIntValue);
	case EArithmeticKeyOperation::Greater:			return (Value > OtherIntValue);
	case EArithmeticKeyOperation::GreaterOrEqual:	return (Value >= OtherIntValue);
	default: break;
	}

	return false;
}

FString UBlackboardKeyType_Enum::DescribeArithmeticParam(int32 IntValue, float FloatValue) const
{
	return EnumType ? EnumType->GetDisplayNameTextByValue(IntValue).ToString() : FString("UNKNOWN!");
}

void UBlackboardKeyType_Enum::InitializeMemory(UBlackboardComponent& OwnerComp, uint8* MemoryBlock)
{
	SetValue(this, MemoryBlock, DefaultValue);
}

#if WITH_EDITOR
void UBlackboardKeyType_Enum::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property &&
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UBlackboardKeyType_Enum, EnumName))
	{
		EnumType = UClass::TryFindTypeSlow<UEnum>(EnumName, EFindFirstObjectOptions::ExactClass);

		if (EnumType != nullptr && !ValidateEnum(*EnumType))
		{
			EnumType = nullptr;
		}
	}

	bIsEnumNameValid = EnumType && !EnumName.IsEmpty();
}

bool UBlackboardKeyType_Enum::ValidateEnum(const UEnum& EnumType)
{
	bool bAllValid = true;

	// Do not test the max value (if present) since it is an internal value and users don't have access to it
	const int32 NumEnums = EnumType.ContainsExistingMax() ? EnumType.NumEnums() - 1 : EnumType.NumEnums();
	for (int32 i = 0; i < NumEnums; i++)
	{
		const int64 Value = EnumType.GetValueByIndex(i);
		if (Value < std::numeric_limits<FDataType>::min() || Value > std::numeric_limits<FDataType>::max())
		{
			UE_LOG(LogBehaviorTree, Error, TEXT("'%s' value %" INT64_FMT " is outside the range of supported key values for enum [%d, %d].")
				, *EnumType.GenerateFullEnumName(*EnumType.GetDisplayNameTextByIndex(i).ToString())
				, Value, std::numeric_limits<FDataType>::min(), std::numeric_limits<FDataType>::max());
			bAllValid = false;
		}
	}

	if (!bAllValid)
	{
		FMessageDialog::Open(EAppMsgType::Ok,
			NSLOCTEXT("BehaviorTree"
				, "Unsupported enumeration"
				, "Specified enumeration contains one or more values outside supported value range for enum keys and can not be used in the Blackboard. See log for details."));
	}

	return bAllValid;
}
#endif // WITH_EDITOR

