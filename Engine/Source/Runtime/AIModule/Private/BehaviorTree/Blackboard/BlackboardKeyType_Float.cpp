// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Blackboard/BlackboardKeyType_Float.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlackboardKeyType_Float)

const UBlackboardKeyType_Float::FDataType UBlackboardKeyType_Float::InvalidValue = UBlackboardKeyType_Float::FDataType(0);

UBlackboardKeyType_Float::UBlackboardKeyType_Float(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	ValueSize = sizeof(float);
	SupportedOp = EBlackboardKeyOperation::Arithmetic;
}

float UBlackboardKeyType_Float::GetValue(const UBlackboardKeyType_Float* KeyOb, const uint8* RawData)
{
	return GetValueFromMemory<float>(RawData);
}

bool UBlackboardKeyType_Float::SetValue(UBlackboardKeyType_Float* KeyOb, uint8* RawData, float Value)
{
	return SetValueInMemory<float>(RawData, Value);
}

EBlackboardCompare::Type UBlackboardKeyType_Float::CompareValues(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock,
	const UBlackboardKeyType* OtherKeyOb, const uint8* OtherMemoryBlock) const
{
	const float MyValue = GetValue(this, MemoryBlock);
	const float OtherValue = GetValue((UBlackboardKeyType_Float*)OtherKeyOb, OtherMemoryBlock);

	return (FMath::Abs(MyValue - OtherValue) < KINDA_SMALL_NUMBER) ? EBlackboardCompare::Equal :
		(MyValue > OtherValue) ? EBlackboardCompare::Greater :
		EBlackboardCompare::Less;
}

void UBlackboardKeyType_Float::InitializeMemory(UBlackboardComponent& OwnerComp, uint8* MemoryBlock)
{
	SetValue(this, MemoryBlock, DefaultValue);
}

FString UBlackboardKeyType_Float::DescribeValue(const UBlackboardComponent& OwnerComp, const uint8* RawData) const
{
	return FString::Printf(TEXT("%f"), GetValue(this, RawData));
}

bool UBlackboardKeyType_Float::TestArithmeticOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, EArithmeticKeyOperation::Type Op, int32 OtherIntValue, float OtherFloatValue) const
{
	const float Value = GetValue(this, MemoryBlock);
	switch (Op)
	{
	case EArithmeticKeyOperation::Equal:			return (Value == OtherFloatValue);
	case EArithmeticKeyOperation::NotEqual:			return (Value != OtherFloatValue);
	case EArithmeticKeyOperation::Less:				return (Value < OtherFloatValue);
	case EArithmeticKeyOperation::LessOrEqual:		return (Value <= OtherFloatValue);
	case EArithmeticKeyOperation::Greater:			return (Value > OtherFloatValue);
	case EArithmeticKeyOperation::GreaterOrEqual:	return (Value >= OtherFloatValue);
	default: break;
	}

	return false;
}

FString UBlackboardKeyType_Float::DescribeArithmeticParam(int32 IntValue, float FloatValue) const
{
	return FString::Printf(TEXT("%f"), FloatValue);
}

