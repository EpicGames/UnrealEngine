// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Blackboard/BlackboardKeyType_Int.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlackboardKeyType_Int)

const UBlackboardKeyType_Int::FDataType UBlackboardKeyType_Int::InvalidValue = UBlackboardKeyType_Int::FDataType(0);

UBlackboardKeyType_Int::UBlackboardKeyType_Int(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	ValueSize = sizeof(int32);
	SupportedOp = EBlackboardKeyOperation::Arithmetic;
}

int32 UBlackboardKeyType_Int::GetValue(const UBlackboardKeyType_Int* KeyOb, const uint8* RawData)
{
	return GetValueFromMemory<int32>(RawData);
}

bool UBlackboardKeyType_Int::SetValue(UBlackboardKeyType_Int* KeyOb, uint8* RawData, int32 Value)
{
	return SetValueInMemory<int32>(RawData, Value);
}

EBlackboardCompare::Type UBlackboardKeyType_Int::CompareValues(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock,
	const UBlackboardKeyType* OtherKeyOb, const uint8* OtherMemoryBlock) const
{
	const int32 MyValue = GetValue(this, MemoryBlock);
	const int32 OtherValue = GetValue((UBlackboardKeyType_Int*)OtherKeyOb, OtherMemoryBlock);

	return (MyValue > OtherValue) ? EBlackboardCompare::Greater :
		(MyValue < OtherValue) ? EBlackboardCompare::Less :
		EBlackboardCompare::Equal;
}

void UBlackboardKeyType_Int::InitializeMemory(UBlackboardComponent& OwnerComp, uint8* MemoryBlock)
{
	SetValue(this, MemoryBlock, DefaultValue);
}

FString UBlackboardKeyType_Int::DescribeValue(const UBlackboardComponent& OwnerComp, const uint8* RawData) const
{
	return FString::Printf(TEXT("%d"), GetValue(this, RawData));
}

bool UBlackboardKeyType_Int::TestArithmeticOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, EArithmeticKeyOperation::Type Op, int32 OtherIntValue, float OtherFloatValue) const
{
	const int32 Value = GetValue(this, MemoryBlock);
	switch (Op)
	{
	case EArithmeticKeyOperation::Equal:			return (Value == OtherIntValue);
	case EArithmeticKeyOperation::NotEqual:			return (Value != OtherIntValue);
	case EArithmeticKeyOperation::Less:				return (Value < OtherIntValue);
	case EArithmeticKeyOperation::LessOrEqual:		return (Value <= OtherIntValue);
	case EArithmeticKeyOperation::Greater:			return (Value > OtherIntValue);
	case EArithmeticKeyOperation::GreaterOrEqual:	return (Value >= OtherIntValue);
	default: break;
	}

	return false;
}

FString UBlackboardKeyType_Int::DescribeArithmeticParam(int32 IntValue, float FloatValue) const
{
	return FString::Printf(TEXT("%d"), IntValue);
}

