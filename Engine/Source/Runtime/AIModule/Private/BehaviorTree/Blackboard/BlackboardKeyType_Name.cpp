// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Blackboard/BlackboardKeyType_Name.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlackboardKeyType_Name)

const UBlackboardKeyType_Name::FDataType UBlackboardKeyType_Name::InvalidValue = NAME_None;

UBlackboardKeyType_Name::UBlackboardKeyType_Name(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	ValueSize = sizeof(FName);
	SupportedOp = EBlackboardKeyOperation::Text;
}

FName UBlackboardKeyType_Name::GetValue(const UBlackboardKeyType_Name* KeyOb, const uint8* RawData)
{
	return GetValueFromMemory<FName>(RawData);
}

bool UBlackboardKeyType_Name::SetValue(UBlackboardKeyType_Name* KeyOb, uint8* RawData, const FName& Value)
{
	return SetValueInMemory<FName>(RawData, Value);
}

EBlackboardCompare::Type UBlackboardKeyType_Name::CompareValues(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock,
	const UBlackboardKeyType* OtherKeyOb, const uint8* OtherMemoryBlock) const
{
	const FName MyValue = GetValue(this, MemoryBlock);
	const FName OtherValue = GetValue((UBlackboardKeyType_Name*)OtherKeyOb, OtherMemoryBlock);

	return (MyValue == OtherValue) ? EBlackboardCompare::Equal : EBlackboardCompare::NotEqual;
}

void UBlackboardKeyType_Name::InitializeMemory(UBlackboardComponent& OwnerComp, uint8* MemoryBlock)
{
	SetValue(this, MemoryBlock, DefaultValue);
}

FString UBlackboardKeyType_Name::DescribeValue(const UBlackboardComponent& OwnerComp, const uint8* RawData) const
{
	return GetValue(this, RawData).ToString();
}

bool UBlackboardKeyType_Name::TestTextOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, ETextKeyOperation::Type Op, const FString& OtherString) const
{
	const FString Value = GetValue(this, MemoryBlock).ToString();
	switch (Op)
	{
	case ETextKeyOperation::Equal:			return (Value == OtherString);
	case ETextKeyOperation::NotEqual:		return (Value != OtherString);
	case ETextKeyOperation::Contain:		return (Value.Contains(OtherString) == true);
	case ETextKeyOperation::NotContain:		return (Value.Contains(OtherString) == false);
	default: break;
	}

	return false;
}

