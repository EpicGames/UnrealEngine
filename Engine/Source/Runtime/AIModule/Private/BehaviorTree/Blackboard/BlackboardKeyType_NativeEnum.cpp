// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Blackboard/BlackboardKeyType_NativeEnum.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Enum.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlackboardKeyType_NativeEnum)

const UBlackboardKeyType_NativeEnum::FDataType UBlackboardKeyType_NativeEnum::InvalidValue = UBlackboardKeyType_NativeEnum::FDataType(0);

UBlackboardKeyType_NativeEnum::UBlackboardKeyType_NativeEnum(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

UBlackboardKeyType* UBlackboardKeyType_NativeEnum::UpdateDeprecatedKey()
{
	UBlackboardKeyType_Enum* KeyOb = NewObject<UBlackboardKeyType_Enum>(GetOuter());
	KeyOb->EnumName = EnumName;
	KeyOb->EnumType = EnumType;
	KeyOb->bIsEnumNameValid = EnumType && !EnumName.IsEmpty();

	return KeyOb;
}

uint8 UBlackboardKeyType_NativeEnum::GetValue(const UBlackboardKeyType_NativeEnum* KeyOb, const uint8* RawData)
{
	return GetValueFromMemory<uint8>(RawData);
}

bool UBlackboardKeyType_NativeEnum::SetValue(UBlackboardKeyType_NativeEnum* KeyOb, uint8* RawData, uint8 Value)
{
	return SetValueInMemory<uint8>(RawData, Value);
}

