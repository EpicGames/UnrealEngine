// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Blackboard/BlackboardKeyType_Object.h"
#include "UObject/WeakObjectPtr.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlackboardKeyType_Object)

const UBlackboardKeyType_Object::FDataType UBlackboardKeyType_Object::InvalidValue = nullptr;

UBlackboardKeyType_Object::UBlackboardKeyType_Object(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	ValueSize = sizeof(FWeakObjectPtr);
	BaseClass = UObject::StaticClass();
	SupportedOp = EBlackboardKeyOperation::Basic;
}

UObject* UBlackboardKeyType_Object::GetValue(const UBlackboardKeyType_Object* KeyOb, const uint8* RawData)
{
	// RawData can be NULL if the key is corrupted by bad data, such as if someone has duplicated a key in a derived
	// blackboard.  We must handle that bad data case gracefully.  It's likely that we need to handle this for
	// all blackboard key types, so possibly GetValueFromMemory should handle the NULL case instead.  But for now I'm
	// just fixing the case I came across.
	if (RawData == NULL)
	{
		return NULL;
	}

	FWeakObjectPtr WeakObjPtr = GetValueFromMemory<FWeakObjectPtr>(RawData);
	return WeakObjPtr.Get();
}

bool UBlackboardKeyType_Object::SetValue(UBlackboardKeyType_Object* KeyOb, uint8* RawData, UObject* Value)
{
	TWeakObjectPtr<UObject> WeakObjPtr(Value);
	return SetWeakObjectInMemory<UObject>(RawData, WeakObjPtr);
}

EBlackboardCompare::Type UBlackboardKeyType_Object::CompareValues(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock,
	const UBlackboardKeyType* OtherKeyOb, const uint8* OtherMemoryBlock) const
{
	const UObject* MyValue = GetValue(this, MemoryBlock);
	const UObject* OtherValue = GetValue((UBlackboardKeyType_Object*)OtherKeyOb, OtherMemoryBlock);

	return (MyValue == OtherValue) ? EBlackboardCompare::Equal : EBlackboardCompare::NotEqual;
}

FString UBlackboardKeyType_Object::DescribeValue(const UBlackboardComponent& OwnerComp, const uint8* RawData) const
{
	return *GetNameSafe(GetValue(this, RawData));
}

FString UBlackboardKeyType_Object::DescribeSelf() const
{
	return *GetNameSafe(BaseClass);
}

bool UBlackboardKeyType_Object::IsAllowedByFilter(UBlackboardKeyType* FilterOb) const
{
	UBlackboardKeyType_Object* FilterObject = Cast<UBlackboardKeyType_Object>(FilterOb);
	UE_CLOGF(BaseClass == nullptr, LogBlackboard, Warning, "Cannot assign %ls to a Blackboard Key with null base class. Should the base class be changed?", FilterObject ? *FilterObject->DescribeSelf() : TEXT("NULL"));
	return (FilterObject && (FilterObject->BaseClass == BaseClass || (BaseClass != nullptr && BaseClass->IsChildOf(FilterObject->BaseClass))));
}

bool UBlackboardKeyType_Object::GetLocation(const UBlackboardComponent& OwnerComp, const uint8* RawData, FVector& Location) const
{
	AActor* MyActor = Cast<AActor>(GetValue(this, RawData));
	if (MyActor)
	{
		Location = MyActor->GetActorLocation();
		return true;
	}

	return false;
}

bool UBlackboardKeyType_Object::GetRotation(const UBlackboardComponent& OwnerComp, const uint8* RawData, FRotator& Rotation) const
{
	AActor* MyActor = Cast<AActor>(GetValue(this, RawData));
	if (MyActor)
	{
		Rotation = MyActor->GetActorRotation();
		return true;
	}

	return false;
}

bool UBlackboardKeyType_Object::TestBasicOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, EBasicKeyOperation::Type Op) const
{
	if (MemoryBlock == NULL)
	{
		return false;
	}

	FWeakObjectPtr WeakObjPtr = GetValueFromMemory<FWeakObjectPtr>(MemoryBlock);
	return (Op == EBasicKeyOperation::Set) ? WeakObjPtr.IsValid() : !WeakObjPtr.IsValid();
}

void UBlackboardKeyType_Object::InitializeMemory(UBlackboardComponent& OwnerComp, uint8* MemoryBlock)
{
	SetValue(this, MemoryBlock, DefaultValue);
}
