// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Blackboard/BlackboardKeyType_Rotator.h"
#include "AITypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlackboardKeyType_Rotator)

const UBlackboardKeyType_Rotator::FDataType UBlackboardKeyType_Rotator::InvalidValue = FAISystem::InvalidRotation;

UBlackboardKeyType_Rotator::UBlackboardKeyType_Rotator(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	ValueSize = sizeof(FRotator);
	SupportedOp = EBlackboardKeyOperation::Basic;
}

FRotator UBlackboardKeyType_Rotator::GetValue(const UBlackboardKeyType_Rotator* KeyOb, const uint8* RawData)
{
	return GetValueFromMemory<FRotator>(RawData);
}

bool UBlackboardKeyType_Rotator::SetValue(UBlackboardKeyType_Rotator* KeyOb, uint8* RawData, const FRotator& Value)
{
	return SetValueInMemory<FRotator>(RawData, Value);
}

EBlackboardCompare::Type UBlackboardKeyType_Rotator::CompareValues(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock,
	const UBlackboardKeyType* OtherKeyOb, const uint8* OtherMemoryBlock) const
{
	const FRotator MyValue = GetValue(this, MemoryBlock);
	const FRotator OtherValue = GetValue((UBlackboardKeyType_Rotator*)OtherKeyOb, OtherMemoryBlock);

	return MyValue.Equals(OtherValue) ? EBlackboardCompare::Equal : EBlackboardCompare::NotEqual;
}

void UBlackboardKeyType_Rotator::Clear(UBlackboardComponent& OwnerComp, uint8* RawData)
{
	SetValueInMemory<FRotator>(RawData, FAISystem::InvalidRotation);
}

bool UBlackboardKeyType_Rotator::IsEmpty(const UBlackboardComponent& OwnerComp, const uint8* RawData) const
{
	const FRotator Rotation = GetValue(this, RawData);
	return !FAISystem::IsValidRotation(Rotation);
}

FString UBlackboardKeyType_Rotator::DescribeValue(const UBlackboardComponent& OwnerComp, const uint8* RawData) const
{
	const FRotator Rotation = GetValue(this, RawData);
	return FAISystem::IsValidRotation(Rotation) ? Rotation.ToString() : TEXT("(invalid)");
}

bool UBlackboardKeyType_Rotator::GetRotation(const UBlackboardComponent& OwnerComp, const uint8* RawData, FRotator& Rotation) const
{
	Rotation = GetValue(this, RawData);
	return FAISystem::IsValidRotation(Rotation);
}

void UBlackboardKeyType_Rotator::InitializeMemory(UBlackboardComponent& OwnerComp, uint8* RawData)
{
	if (bUseDefaultValue)
	{
		SetValue(this, RawData, DefaultValue);
	}
	else
	{
		SetValue(this,RawData, InvalidValue);
	}
}

bool UBlackboardKeyType_Rotator::TestBasicOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, EBasicKeyOperation::Type Op) const
{
	const FRotator Rotation = GetValue(this, MemoryBlock);
	return (Op == EBasicKeyOperation::Set) ? FAISystem::IsValidRotation(Rotation) : !FAISystem::IsValidRotation(Rotation);
}

