// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Blackboard/BlackboardKeyType_String.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlackboardKeyType_String)

const UBlackboardKeyType_String::FDataType UBlackboardKeyType_String::InvalidValue = FString();

UBlackboardKeyType_String::UBlackboardKeyType_String(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	ValueSize = 0;
	bCreateKeyInstance = true;

	SupportedOp = EBlackboardKeyOperation::Text;
}

FString UBlackboardKeyType_String::GetValue(const UBlackboardKeyType_String* KeyOb, const uint8* RawData)
{
	return KeyOb->StringValue;
}

bool UBlackboardKeyType_String::SetValue(UBlackboardKeyType_String* KeyOb, uint8* RawData, const FString& Value)
{
	const bool bChanged = !KeyOb->StringValue.Equals(Value);
	KeyOb->StringValue = Value;
	return bChanged;
}

EBlackboardCompare::Type UBlackboardKeyType_String::CompareValues(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock,
	const UBlackboardKeyType* OtherKeyOb, const uint8* OtherMemoryBlock) const
{
	const FString MyValue = GetValue(this, MemoryBlock);
	const FString OtherValue = GetValue((UBlackboardKeyType_String*)OtherKeyOb, OtherMemoryBlock);

	return MyValue.Equals(OtherValue) ? EBlackboardCompare::Equal : EBlackboardCompare::NotEqual;
}

void UBlackboardKeyType_String::CopyValues(UBlackboardComponent& OwnerComp, uint8* MemoryBlock, const UBlackboardKeyType* SourceKeyOb, const uint8* SourceBlock)
{
	StringValue = ((UBlackboardKeyType_String*)SourceKeyOb)->StringValue;
}

void UBlackboardKeyType_String::InitializeMemory(UBlackboardComponent& OwnerComp, uint8* MemoryBlock)
{
	SetValue(this, MemoryBlock, DefaultValue);
}

FString UBlackboardKeyType_String::DescribeValue(const UBlackboardComponent& OwnerComp, const uint8* RawData) const
{
	return StringValue;
}

bool UBlackboardKeyType_String::TestTextOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, ETextKeyOperation::Type Op, const FString& OtherString) const
{
	switch (Op)
	{
	case ETextKeyOperation::Equal:			return (StringValue == OtherString);
	case ETextKeyOperation::NotEqual:		return (StringValue != OtherString);
	case ETextKeyOperation::Contain:		return (StringValue.Contains(OtherString) == true);
	case ETextKeyOperation::NotContain:		return (StringValue.Contains(OtherString) == false);
	default: break;
	}

	return false;
}

void UBlackboardKeyType_String::Clear(UBlackboardComponent& OwnerComp, uint8* MemoryBlock)
{
	StringValue.Empty();
}

bool UBlackboardKeyType_String::IsEmpty(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock) const
{
	return StringValue.IsEmpty();
}

