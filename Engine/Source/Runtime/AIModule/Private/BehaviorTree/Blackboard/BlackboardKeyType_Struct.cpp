// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Blackboard/BlackboardKeyType_Struct.h"

#include "StructUtils/StructView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlackboardKeyType_Struct)

const UBlackboardKeyType_Struct::FDataType UBlackboardKeyType_Struct::InvalidValue;

UBlackboardKeyType_Struct::UBlackboardKeyType_Struct(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateKeyInstance = false;
	SupportedOp = EBlackboardKeyOperation::Basic;
}

FConstStructView UBlackboardKeyType_Struct::GetValue(UBlackboardKeyType_Struct* KeyOb, const uint8* RawData)
{
	if (KeyOb->bIsInstanced)
	{
		return FConstStructView(KeyOb->Value);
	}
	else
	{
		return FConstStructView(KeyOb->DefaultValue.GetScriptStruct(), RawData);
	}
}

bool UBlackboardKeyType_Struct::SetValue(UBlackboardKeyType_Struct* KeyOb, uint8* RawData, FConstStructView NewValue)
{
	if (KeyOb->bIsInstanced)
	{
		if (NewValue.GetScriptStruct() == KeyOb->DefaultValue.GetScriptStruct())
		{
			KeyOb->Value = NewValue;
			return true;
		}
	}
	else
	{
		if (const UScriptStruct* ScriptStruct = KeyOb->DefaultValue.GetScriptStruct())
		{
			if (NewValue.GetScriptStruct() == ScriptStruct)
			{
				ScriptStruct->CopyScriptStruct(RawData, NewValue.GetMemory());
				return true;
			}
		}
	}
	return false;
}

EBlackboardCompare::Type UBlackboardKeyType_Struct::CompareValues(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock,
	const UBlackboardKeyType* OtherKeyOb, const uint8* OtherMemoryBlock) const
{
	if (const UBlackboardKeyType_Struct* OtherKey = Cast<UBlackboardKeyType_Struct>(OtherKeyOb))
	{
		if (DefaultValue.GetScriptStruct() == OtherKey->DefaultValue.GetScriptStruct())
		{
			if (bIsInstanced)
			{
				return Value == OtherKey->Value ? EBlackboardCompare::Equal : EBlackboardCompare::NotEqual;
			}
			else
			{
				if (const UScriptStruct* ScriptStruct = DefaultValue.GetScriptStruct())
				{
					return ScriptStruct->CompareScriptStruct(MemoryBlock, OtherMemoryBlock, PPF_None) ? EBlackboardCompare::Equal : EBlackboardCompare::NotEqual;
				}
			}
		}
	}
	return EBlackboardCompare::NotEqual;
}

void UBlackboardKeyType_Struct::CopyValues(UBlackboardComponent& OwnerComp, uint8* MemoryBlock, const UBlackboardKeyType* SourceKeyOb, const uint8* SourceBlock)
{
	if (const UBlackboardKeyType_Struct* SourceKey = Cast<UBlackboardKeyType_Struct>(SourceKeyOb))
	{
		if (DefaultValue.GetScriptStruct() == SourceKey->DefaultValue.GetScriptStruct())
		{
			if (bIsInstanced)
			{
				Value = SourceKey->Value;
			}
			else
			{
				if (const UScriptStruct* ScriptStruct = DefaultValue.GetScriptStruct())
				{
					ScriptStruct->CopyScriptStruct(MemoryBlock, SourceBlock);
				}
			}
		}
	}
}

void UBlackboardKeyType_Struct::UpdateNeedsInstance()
{
	if (const UScriptStruct* ScriptStruct = DefaultValue.GetScriptStruct())
	{
		for (const FProperty* PropertyLink = ScriptStruct->PropertyLink; PropertyLink; PropertyLink = PropertyLink->PropertyLinkNext)
		{
			// Struct with strong object reference needs to have their key instantiated to be taken into account by the gc.
			// Otherwise struct can safely lives inside the blackboard buffer
			TArray<const FStructProperty*> EncounteredStructProps;
			if (PropertyLink->ContainsObjectReference(EncounteredStructProps, EPropertyObjectReferenceType::Strong))
			{
				bCreateKeyInstance = true;
				ValueSize = 0;
				return;
			}
		}

		bCreateKeyInstance = false;
		ValueSize = ScriptStruct->GetStructureSize();
	}
	else
	{
		bCreateKeyInstance = false;
		ValueSize = 0;
	}
}

FString UBlackboardKeyType_Struct::DescribeValue(const UBlackboardComponent& OwnerComp, const uint8* RawData) const
{
	const UScriptStruct* ScriptStruct = nullptr;
	const uint8* ValuePtr = nullptr;
	if (bIsInstanced)
	{
		ScriptStruct = Value.GetScriptStruct();
		ValuePtr = Value.GetMemory();
	}
	else
	{
		ScriptStruct = DefaultValue.GetScriptStruct();
		ValuePtr = RawData;
	}

	if (ScriptStruct && ValuePtr)
	{
		FString ExportedValue;
		ScriptStruct->ExportText(ExportedValue, ValuePtr, nullptr, const_cast<UBlackboardKeyType_Struct*>(this), PPF_None, nullptr);
		return ExportedValue;
	}
	return DescribeSelf();
}

FString UBlackboardKeyType_Struct::DescribeSelf() const
{
	return *GetNameSafe(DefaultValue.GetScriptStruct());
}

bool UBlackboardKeyType_Struct::IsAllowedByFilter(UBlackboardKeyType* FilterOb) const
{
	const UBlackboardKeyType_Struct* FilterClass = Cast<UBlackboardKeyType_Struct>(FilterOb);
	return FilterClass && (FilterClass->DefaultValue.GetScriptStruct() == DefaultValue.GetScriptStruct());
}

void UBlackboardKeyType_Struct::InitializeMemory(UBlackboardComponent& OwnerComp, uint8* MemoryBlock)
{
	SetValue(this, MemoryBlock, DefaultValue);
}

void UBlackboardKeyType_Struct::FreeMemory(UBlackboardComponent& OwnerComp, uint8* MemoryBlock)
{
	// Let normal object destruction flow take care of the instanced key.
	if (!bIsInstanced)
	{
		if (const UScriptStruct* ScriptStruct = DefaultValue.GetScriptStruct())
		{
			ScriptStruct->DestroyStruct(MemoryBlock);
		}
	}
}

void UBlackboardKeyType_Struct::Clear(UBlackboardComponent& OwnerComp, uint8* MemoryBlock)
{
	InitializeMemory(OwnerComp, MemoryBlock);
}

void UBlackboardKeyType_Struct::PostLoad()
{
	Super::PostLoad();

	UpdateNeedsInstance();
}

#if WITH_EDITOR
void UBlackboardKeyType_Struct::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	UpdateNeedsInstance();
}
#endif // WITH_EDITOR

bool UBlackboardKeyType_Struct::TestBasicOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, EBasicKeyOperation::Type Op) const
{
	if (bIsInstanced)
	{
		return (Op == EBasicKeyOperation::Set) ? Value != DefaultValue : Value == DefaultValue;
	}
	else
	{
		if (const UScriptStruct* ScriptStruct = DefaultValue.GetScriptStruct())
		{
			const bool Identical = ScriptStruct->CompareScriptStruct(MemoryBlock, DefaultValue.GetMemory(), PPF_None);
			return Op == EBasicKeyOperation::Set ? !Identical : Identical;
		}
	}
	return Op == EBasicKeyOperation::NotSet;
}
