// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Blackboard/BlackboardKeyType_Vector.h"
#include "AITypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlackboardKeyType_Vector)

const UBlackboardKeyType_Vector::FDataType UBlackboardKeyType_Vector::InvalidValue = FAISystem::InvalidLocation;

UBlackboardKeyType_Vector::UBlackboardKeyType_Vector(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	ValueSize = sizeof(FVector);
	SupportedOp = EBlackboardKeyOperation::Basic;
}

FVector UBlackboardKeyType_Vector::GetValue(const UBlackboardKeyType_Vector* KeyOb, const uint8* RawData)
{
	return GetValueFromMemory<FVector>(RawData);
}

bool UBlackboardKeyType_Vector::SetValue(UBlackboardKeyType_Vector* KeyOb, uint8* RawData, const FVector& Value)
{
	return SetValueInMemory<FVector>(RawData, Value);
}

EBlackboardCompare::Type UBlackboardKeyType_Vector::CompareValues(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock,
	const UBlackboardKeyType* OtherKeyOb, const uint8* OtherMemoryBlock) const
{
	const FVector MyValue = GetValue(this, MemoryBlock);
	const FVector OtherValue = GetValue((UBlackboardKeyType_Vector*)OtherKeyOb, OtherMemoryBlock);

	return MyValue.Equals(OtherValue) ? EBlackboardCompare::Equal : EBlackboardCompare::NotEqual;
}

void UBlackboardKeyType_Vector::Clear(UBlackboardComponent& OwnerComp, uint8* RawData)
{
	SetValueInMemory<FVector>(RawData, FAISystem::InvalidLocation);
}

bool UBlackboardKeyType_Vector::IsEmpty(const UBlackboardComponent& OwnerComp, const uint8* RawData) const
{
	const FVector Location = GetValue(this, RawData);
	return !FAISystem::IsValidLocation(Location);
}

FString UBlackboardKeyType_Vector::DescribeValue(const UBlackboardComponent& OwnerComp, const uint8* RawData) const
{
	const FVector Location = GetValue(this, RawData);
	return FAISystem::IsValidLocation(Location) ? Location.ToString() : TEXT("(invalid)");
}

bool UBlackboardKeyType_Vector::GetLocation(const UBlackboardComponent& OwnerComp, const uint8* RawData, FVector& Location) const
{
	Location = GetValue(this, RawData);
	return FAISystem::IsValidLocation(Location);
}

void UBlackboardKeyType_Vector::InitializeMemory(UBlackboardComponent& OwnerComp, uint8* RawData)
{
	if (bUseDefaultValue)
	{
		SetValue(this, RawData, DefaultValue);
	}
	else
	{
		SetValue(this, RawData, InvalidValue);
	}
}

bool UBlackboardKeyType_Vector::TestBasicOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, EBasicKeyOperation::Type Op) const
{
	const FVector Location = GetValue(this, MemoryBlock);
	return (Op == EBasicKeyOperation::Set) ? FAISystem::IsValidLocation(Location) : !FAISystem::IsValidLocation(Location);
}

