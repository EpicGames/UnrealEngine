// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/BlackboardAssetProvider.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlackboardAssetProvider)


//----------------------------------------------------------------------//
// UBlackboardAssetProvider
//----------------------------------------------------------------------//

#if WITH_EDITOR
IBlackboardAssetProvider::FBlackboardOwnerChanged IBlackboardAssetProvider::OnBlackboardOwnerChanged;
#endif

UBlackboardAssetProvider::UBlackboardAssetProvider(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

