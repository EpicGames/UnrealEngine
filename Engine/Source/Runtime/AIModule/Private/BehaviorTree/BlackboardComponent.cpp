// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/BlackboardComponent.h"
#include "BrainComponent.h"
#include "AIController.h"
#include "BehaviorTree/BTNode.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Enum.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Object.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Vector.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Rotator.h"
#include "VisualLogger/VisualLogger.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Bool.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Class.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Float.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Int.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Name.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_String.h"
#include "Misc/RuntimeErrors.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlackboardComponent)

DEFINE_LOG_CATEGORY(LogBlackboard);

UBlackboardComponent::UBlackboardComponent(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = false;
	bWantsInitializeComponent = true;
	bPausedNotifies = false;
	bSynchronizedKeyPopulated = false;
}

void UBlackboardComponent::InitializeComponent()
{
	Super::InitializeComponent();

	// cache blackboard component if owner has one
	// note that it's a valid scenario for this component to not have an owner at all (at least in terms of unittesting)
	AActor* Owner = GetOwner();
	if (Owner)
	{
		BrainComp = GetOwner()->FindComponentByClass<UBrainComponent>();
		if (BrainComp)
		{
			BrainComp->CacheBlackboardComponent(this);
		}
	}

	if (!BlackboardAsset && DefaultBlackboardAsset)
	{
		InitializeBlackboard(*DefaultBlackboardAsset);
	}
	else
	{
		UE_CVLOG(BlackboardAsset, this, LogBlackboard, Log, TEXT("InitializeComponent called while BlackboardAsset is not null (set to %s). Assuming it's been set via InitializeBlackboard prior to RegisterComponent call. Ignoring DefaultBlackboardAsset %s")
			, *GetNameSafe(BlackboardAsset), *GetNameSafe(DefaultBlackboardAsset));
	}
}

void UBlackboardComponent::UninitializeComponent()
{
	if (BlackboardAsset && BlackboardAsset->HasSynchronizedKeys())
	{
		UAISystem* AISystem = UAISystem::GetCurrentSafe(GetWorld());
		if (AISystem)
		{
			AISystem->UnregisterBlackboardComponent(*BlackboardAsset, *this);
		}
	}

	DestroyValues();
	Super::UninitializeComponent();
}

void UBlackboardComponent::CacheBrainComponent(UBrainComponent& BrainComponent)
{
	if (&BrainComponent != BrainComp)
	{
		BrainComp = &BrainComponent;
	}
}

struct FBlackboardInitializationData
{
	FBlackboard::FKey KeyID;
	uint16 DataSize;

	FBlackboardInitializationData() {}
	FBlackboardInitializationData(FBlackboard::FKey InKeyID, uint16 InDataSize) : KeyID(InKeyID)
	{
		DataSize = (InDataSize <= 2) ? InDataSize : ((InDataSize + 3) & ~3);
	}

	struct FMemorySort
	{
		FORCEINLINE bool operator()(const FBlackboardInitializationData& A, const FBlackboardInitializationData& B) const
		{
			return A.DataSize > B.DataSize;
		}
	};
};

void UBlackboardComponent::InitializeParentChain(UBlackboardData* NewAsset)
{
	if (NewAsset)
	{
		InitializeParentChain(NewAsset->Parent);
		NewAsset->UpdateKeyIDs();
	}
}

bool UBlackboardComponent::InitializeBlackboard(UBlackboardData& NewAsset)
{
	// if we re-initialize with the same asset then there's no point
	// in resetting, since we'd lose all the accumulated knowledge
	if (&NewAsset == BlackboardAsset)
	{
		return true;
	}

	UAISystem* AISystem = UAISystem::GetCurrentSafe(GetWorld());
	if (AISystem == nullptr)
	{
		return false;
	}

	if (BlackboardAsset && BlackboardAsset->HasSynchronizedKeys())
	{
		AISystem->UnregisterBlackboardComponent(*BlackboardAsset, *this);
		DestroyValues();
	}

	BlackboardAsset = &NewAsset;
	ValueMemory.Reset();
	ValueOffsets.Reset();
	bBlackboardMemoryInitialized = false;
	bSynchronizedKeyPopulated = false;

	if (BlackboardAsset->IsValid())
	{
		InitializeParentChain(BlackboardAsset);

		TArray<FBlackboardInitializationData> InitList;
		const int32 NumKeys = BlackboardAsset->GetNumKeys();
		InitList.Reserve(NumKeys);
		ValueOffsets.AddZeroed(NumKeys);

		for (UBlackboardData* It = BlackboardAsset; It; It = It->Parent)
		{
			for (int32 KeyIndex = 0; KeyIndex < It->Keys.Num(); KeyIndex++)
			{
				UBlackboardKeyType* KeyType = It->Keys[KeyIndex].KeyType;
				if (KeyType)
				{
					KeyType->PreInitialize(*this);

					const uint16 KeyMemory = KeyType->GetValueSize() + (KeyType->HasInstance() ? sizeof(FBlackboardInstancedKeyMemory) : 0);
					const int32 OffsetKey = KeyIndex + (int32)It->GetFirstKeyID();
					check(FBlackboard::FKey(OffsetKey) != FBlackboard::InvalidKey);

					InitList.Add(FBlackboardInitializationData(FBlackboard::FKey(OffsetKey), KeyMemory));
				}
			}
		}

		// sort key values by memory size, so they can be packed better
		// it still won't protect against structures, that are internally misaligned (-> uint8, uint32)
		// but since all Engine level keys are good... 
		InitList.Sort(FBlackboardInitializationData::FMemorySort());
		uint16 MemoryOffset = 0;
		for (int32 Index = 0; Index < InitList.Num(); Index++)
		{
			ValueOffsets[InitList[Index].KeyID] = MemoryOffset;
			MemoryOffset += InitList[Index].DataSize;
		}

		ValueMemory.AddZeroed(MemoryOffset);

		// initialize memory
		KeyInstances.AddZeroed(InitList.Num());
		for (int32 Index = 0; Index < InitList.Num(); Index++)
		{
			const FBlackboardEntry* KeyData = BlackboardAsset->GetKey(InitList[Index].KeyID);
			if (ensureMsgf(KeyData && KeyData->KeyType, TEXT("Too many elements in blackboard %s (%u)"),*GetNameSafe(BlackboardAsset), InitList.Num()))
			{
				KeyData->KeyType->InitializeKey(*this, InitList[Index].KeyID);
			}
		}

		// naive initial synchronization with one of already instantiated blackboards using the same BB asset
		if (BlackboardAsset->HasSynchronizedKeys())
		{
			PopulateSynchronizedKeys();
		}
		
		bBlackboardMemoryInitialized = true;
	}
	else
	{
		UE_LOGF(LogBlackboard, Error, "Blackboard asset (%ls) has errors and can't be used!", *GetNameSafe(BlackboardAsset));
	}

	return bBlackboardMemoryInitialized;
}

void UBlackboardComponent::DestroyValues()
{
	if (!bBlackboardMemoryInitialized || 
		!ensureMsgf(ValueMemory.Num() > 0 && ValueOffsets.Num() > 0, TEXT("The Blackboard is initialized but no memory was allocated to store values for Blackboard asset: %s"), *GetNameSafe(BlackboardAsset)))
	{
		return;
	}
	
	for (UBlackboardData* It = BlackboardAsset; It; It = It->Parent)
	{
		for (int32 KeyIndex = 0; KeyIndex < It->Keys.Num(); KeyIndex++)
		{
			if (UBlackboardKeyType* KeyType = It->Keys[KeyIndex].KeyType)
			{
				const int32 OffsetKey = KeyIndex + static_cast<int32>(It->GetFirstKeyID());
				check(FBlackboard::FKey(OffsetKey) != FBlackboard::InvalidKey);

				uint8* KeyMemory = GetKeyRawData(FBlackboard::FKey(OffsetKey));
				if (KeyMemory)
				{
					KeyType->WrappedFree(*this, KeyMemory);
				}
				else
				{
					UE_LOGF(LogBlackboard, Warning, 
						   "DestroyValues skipping key '%ls' - no memory allocated. Blackboard asset (%ls) may have failed initialization.", 
						   *It->Keys[KeyIndex].EntryName.ToString(), *GetNameSafe(BlackboardAsset));
				}
			}
		}
	}

	ValueOffsets.Reset();
	ValueMemory.Reset();

	bBlackboardMemoryInitialized = false;
}

void UBlackboardComponent::PopulateSynchronizedKeys()
{
	ensure(bSynchronizedKeyPopulated == false);
	
	UAISystem* AISystem = UAISystem::GetCurrentSafe(GetWorld());
	check(AISystem);
	AISystem->RegisterBlackboardComponent(*BlackboardAsset, *this);

	for (auto Iter = AISystem->CreateBlackboardDataToComponentsIterator(*BlackboardAsset); Iter; ++Iter)
	{
		UBlackboardComponent* OtherBlackboard = Iter.Value();
		if (OtherBlackboard != nullptr && ShouldSyncWithBlackboard(*OtherBlackboard))
		{
			for (const FBlackboardEntry& Key : BlackboardAsset->Keys)
			{
				if (Key.bInstanceSynced)
				{
					UBlackboardData* const OtherBlackboardAsset = OtherBlackboard->GetBlackboardAsset();
					const FBlackboard::FKey OtherKeyID = OtherBlackboardAsset ? OtherBlackboardAsset->GetKeyID(Key.EntryName) : FBlackboard::InvalidKey;
					if (OtherKeyID != FBlackboard::InvalidKey)
					{
						const FBlackboardEntry* const OtherKey = OtherBlackboard->GetBlackboardAsset()->GetKey(OtherKeyID);
						check(Key.EntryName == OtherKey->EntryName);
						check(Key.KeyType && OtherKey->KeyType && Key.KeyType->GetClass() == OtherKey->KeyType->GetClass());

						const bool bKeyHasInstance = Key.KeyType->HasInstance();
						const uint16 DataOffset = bKeyHasInstance ? sizeof(FBlackboardInstancedKeyMemory) : 0;

						const FBlackboard::FKey KeyID = BlackboardAsset->GetKeyID(Key.EntryName);
						uint8* RawData = GetKeyRawData(KeyID) + DataOffset;
						uint8* RawSource = OtherBlackboard->GetKeyRawData(OtherKeyID) + DataOffset;

						UBlackboardKeyType* KeyOb = bKeyHasInstance ? ToRawPtr(KeyInstances[KeyID]) : ToRawPtr(Key.KeyType);
						const UBlackboardKeyType* SourceKeyOb = bKeyHasInstance ? ToRawPtr(OtherBlackboard->KeyInstances[OtherKeyID]) : ToRawPtr(Key.KeyType);

						KeyOb->CopyValues(*this, RawData, SourceKeyOb, RawSource);
					}
				}
			}
			break;
		}
	}

	bSynchronizedKeyPopulated = true;
}

bool UBlackboardComponent::ShouldSyncWithBlackboard(UBlackboardComponent& OtherBlackboardComponent) const
{
	return &OtherBlackboardComponent != this && (
		(BrainComp == nullptr || (BrainComp->GetAIOwner() != nullptr && BrainComp->GetAIOwner()->ShouldSyncBlackboardWith(OtherBlackboardComponent) == true))
		|| (OtherBlackboardComponent.BrainComp == nullptr || (OtherBlackboardComponent.BrainComp->GetAIOwner() != nullptr && OtherBlackboardComponent.BrainComp->GetAIOwner()->ShouldSyncBlackboardWith(*this) == true)));
}

UBrainComponent* UBlackboardComponent::GetBrainComponent() const
{
	return BrainComp;
}

UBlackboardData* UBlackboardComponent::GetBlackboardAsset() const
{
	return BlackboardAsset;
}

FName UBlackboardComponent::GetKeyName(FBlackboard::FKey KeyID) const
{
	return BlackboardAsset ? BlackboardAsset->GetKeyName(KeyID) : NAME_None;
}

FBlackboard::FKey UBlackboardComponent::GetKeyID(const FName& KeyName) const
{
	return BlackboardAsset ? BlackboardAsset->GetKeyID(KeyName) : FBlackboard::InvalidKey;
}

TSubclassOf<UBlackboardKeyType> UBlackboardComponent::GetKeyType(FBlackboard::FKey KeyID) const
{
	return BlackboardAsset ? BlackboardAsset->GetKeyType(KeyID) : nullptr;
}

bool UBlackboardComponent::IsKeyInstanceSynced(FBlackboard::FKey KeyID) const
{
	return BlackboardAsset ? BlackboardAsset->IsKeyInstanceSynced(KeyID) : false;
}

int32 UBlackboardComponent::GetNumKeys() const
{
	return BlackboardAsset ? BlackboardAsset->GetNumKeys() : 0;
}

FDelegateHandle UBlackboardComponent::RegisterObserver(FBlackboard::FKey KeyID, const UObject* NotifyOwner, FOnBlackboardChangeNotification ObserverDelegate)
{
	for (auto It = Observers.CreateConstKeyIterator(KeyID); It; ++It)
	{
		// If the pair's value matches, return a pointer to it.
		if (It.Value().GetHandle() == ObserverDelegate.GetHandle())
		{
			return It.Value().GetHandle();
		}
	}

	FDelegateHandle Handle = Observers.Add(KeyID, ObserverDelegate).GetHandle();
	ObserverHandles.Add(NotifyOwner, Handle);

	return Handle;
}

void UBlackboardComponent::UnregisterObserver(FBlackboard::FKey KeyID, FDelegateHandle ObserverHandle)
{
	for (auto It = Observers.CreateKeyIterator(KeyID); It; ++It)
	{
		if (It.Value().GetHandle() == ObserverHandle)
		{
			for (auto HandleIt = ObserverHandles.CreateIterator(); HandleIt; ++HandleIt)
			{
				if (HandleIt.Value() == ObserverHandle)
				{
					HandleIt.RemoveCurrent();
					break;
				}
			}

			// Make sure to not remove observer while iterating through them in the notification code
			if (NotifyObserversRecursionCount == 0)
			{
				It.RemoveCurrent();
			}
			else if (!It.Value().bToBeRemoved)
			{
				It.Value().bToBeRemoved = true;
				++ObserversToRemoveCount;
			}
			break;
		}
	}
}

void UBlackboardComponent::UnregisterObserversFrom(const UObject* NotifyOwner)
{
	for (auto It = ObserverHandles.CreateKeyIterator(NotifyOwner); It; ++It)
	{
		for (auto ObsIt = Observers.CreateIterator(); ObsIt; ++ObsIt)
		{
			if (ObsIt.Value().GetHandle() == It.Value())
			{
				// Make sure to not remove observer while iterating through them in the notification code
				if (NotifyObserversRecursionCount == 0)
				{
					ObsIt.RemoveCurrent();
				}
				else if (!ObsIt.Value().bToBeRemoved)
				{
					ObsIt.Value().bToBeRemoved = true;
					++ObserversToRemoveCount;
				}
				break;
			}
		}

		It.RemoveCurrent();
		// check other delegates from NotifyOwner as well
	}
}

void UBlackboardComponent::PauseObserverNotifications()
{
	bPausedNotifies = true;
}


void UBlackboardComponent::ResumeObserverNotifications(bool bSendQueuedObserverNotifications)
{
	bPausedNotifies = false;

	if (bSendQueuedObserverNotifications)
	{
		for (int32 UpdateIndex = 0; UpdateIndex < QueuedUpdates.Num(); UpdateIndex++)
		{
			NotifyObservers(QueuedUpdates[UpdateIndex]);
		}
	}

	QueuedUpdates.Empty();
}

void UBlackboardComponent::NotifyObservers(FBlackboard::FKey KeyID) const
{
	TMultiMap<FBlackboard::FKey, FOnBlackboardChangeNotificationInfo>::TKeyIterator KeyIt(Observers, KeyID);

	// checking it here mostly to avoid storing this update in QueuedUpdates while
	// at this point no one observes it, and there can be someone added before QueuedUpdates
	// gets processed 
	if (KeyIt)
	{
		if (bPausedNotifies)
		{
			QueuedUpdates.AddUnique(KeyID);
		}
		else
		{
			++NotifyObserversRecursionCount;
			for (; KeyIt; ++KeyIt)
			{
				FOnBlackboardChangeNotificationInfo& ObserverDelegateInfo = KeyIt.Value();
				if (ObserverDelegateInfo.bToBeRemoved)
				{
					continue;
				}

				const FOnBlackboardChangeNotification& ObserverDelegate = ObserverDelegateInfo.DelegateHandle;
				const bool bWantsToContinueObserving = ObserverDelegate.IsBound() && (ObserverDelegate.Execute(*this, KeyID) == EBlackboardNotificationResult::ContinueObserving);

				if (bWantsToContinueObserving == false)
				{
					// Remove from the ObserverHandle map, if not already removed
					if (!ObserverDelegateInfo.bToBeRemoved)
					{
						for (auto HandleIt = ObserverHandles.CreateIterator(); HandleIt; ++HandleIt)
						{
							if (HandleIt.Value() == ObserverDelegate.GetHandle())
							{
								HandleIt.RemoveCurrent();
								break;
							}
						}
					}

					if (NotifyObserversRecursionCount == 1)
					{
						KeyIt.RemoveCurrent();

						// If we are removing it and it was already queued up for removal, remove it from that count
						if (ObserverDelegateInfo.bToBeRemoved)
						{
							--ObserversToRemoveCount;
						}
					}
					else if(!ObserverDelegateInfo.bToBeRemoved)
					{
						ObserverDelegateInfo.bToBeRemoved = true;
						++ObserversToRemoveCount;
					}
				}
			}
			--NotifyObserversRecursionCount;

			if (NotifyObserversRecursionCount == 0 && ObserversToRemoveCount > 0)
			{
				for (auto ObsIt = Observers.CreateIterator(); ObsIt; ++ObsIt)
				{
					if (ObsIt.Value().bToBeRemoved)
					{
						ObsIt.RemoveCurrent();
						if (--ObserversToRemoveCount == 0)
                        {
                            break;
                        }
					}
				}
				ObserversToRemoveCount = 0;
			}
		}
	}
}

bool UBlackboardComponent::IsCompatibleWith(const UBlackboardData* TestAsset) const
{
	for (UBlackboardData* It = BlackboardAsset; It; It = It->Parent)
	{
		if (It == TestAsset)
		{
			return true;
		}

		if (It->Keys == TestAsset->Keys)
		{
			return true;
		}
	}

	return false;
}

EBlackboardCompare::Type UBlackboardComponent::CompareKeyValues(TSubclassOf<UBlackboardKeyType> KeyType, FBlackboard::FKey KeyA, FBlackboard::FKey KeyB) const
{	
	const uint8* KeyAMemory = GetKeyRawData(KeyA) + (KeyInstances[KeyA] ? sizeof(FBlackboardInstancedKeyMemory) : 0);
	const uint8* KeyBMemory = GetKeyRawData(KeyB) + (KeyInstances[KeyB] ? sizeof(FBlackboardInstancedKeyMemory) : 0);

	const UBlackboardKeyType* KeyAOb = KeyInstances[KeyA] ? ToRawPtr(KeyInstances[KeyA]) : KeyType->GetDefaultObject<UBlackboardKeyType>();
	return KeyAOb->CompareValues(*this, KeyAMemory, KeyInstances[KeyB], KeyBMemory);
}

FString UBlackboardComponent::GetDebugInfoString(EBlackboardDescription::Type Mode) const 
{
	FString DebugString = FString::Printf(TEXT("Blackboard (asset: %s)\n"), *GetNameSafe(BlackboardAsset));

	TArray<FString> KeyDesc;
	int32 Offset = 0;
	for (UBlackboardData* It = BlackboardAsset; It; It = It->Parent)
	{
		for (int32 KeyIndex = 0; KeyIndex < It->Keys.Num(); KeyIndex++)
		{
			const int32 OffsetKey = KeyIndex + Offset;
			check(FBlackboard::FKey(OffsetKey) != FBlackboard::InvalidKey);

			KeyDesc.Add(DescribeKeyValue(FBlackboard::FKey(OffsetKey), Mode));
		}
		Offset += It->Keys.Num();
	}
	
	KeyDesc.Sort();
	for (int32 KeyDescIndex = 0; KeyDescIndex < KeyDesc.Num(); KeyDescIndex++)
	{
		DebugString += TEXT("  ");
		DebugString += KeyDesc[KeyDescIndex];
		DebugString += TEXT('\n');
	}

	if (Mode == EBlackboardDescription::Full && BlackboardAsset)
	{
		DebugString += TEXT("Observed Keys:\n");

		TArray<FBlackboard::FKey> ObserversKeys;
		if (Observers.Num() > 0)
		{
			Observers.GetKeys(ObserversKeys);

			for (FBlackboard::FKey KeyID : ObserversKeys)
			{
				//@todo shouldn't be using a localized value?; GetKeyName() [10/11/2013 justin.sargent]
				DebugString += FString::Printf(TEXT("  %s:\n"), *BlackboardAsset->GetKeyName(KeyID).ToString());
			}
		}
		else
		{
			DebugString += TEXT("  NONE\n");
		}
	}

	return DebugString;
}

FString UBlackboardComponent::DescribeKeyValue(const FName& KeyName, EBlackboardDescription::Type Mode) const
{
	return DescribeKeyValue(GetKeyID(KeyName), Mode);
}

FString UBlackboardComponent::DescribeKeyValue(FBlackboard::FKey KeyID, EBlackboardDescription::Type Mode) const
{
	FString Description;

	const FBlackboardEntry* Key = BlackboardAsset ? BlackboardAsset->GetKey(KeyID) : NULL;
	if (Key)
	{
		const uint8* ValueData = GetKeyRawData(KeyID);
		FString ValueDesc = Key->KeyType && ValueData ? *(Key->KeyType->WrappedDescribeValue(*this, ValueData)) : TEXT("empty");

		if (Mode == EBlackboardDescription::OnlyValue)
		{
			Description = ValueDesc;
		}
		else if (Mode == EBlackboardDescription::KeyWithValue)
		{
			Description = FString::Printf(TEXT("%s: %s"), *(Key->EntryName.ToString()), *ValueDesc);
		}
		else
		{
			const FString CommonTypePrefix = UBlackboardKeyType::StaticClass()->GetName().AppendChar(TEXT('_'));
			const FString FullKeyType = Key->KeyType ? GetNameSafe(Key->KeyType->GetClass()) : FString();
			const FString DescKeyType = FullKeyType.StartsWith(CommonTypePrefix) ? FullKeyType.RightChop(CommonTypePrefix.Len()) : FullKeyType;

			Description = FString::Printf(TEXT("%s [%s]: %s"), *(Key->EntryName.ToString()), *DescKeyType, *ValueDesc);
		}
	}

	return Description;
}

#if ENABLE_VISUAL_LOG

void UBlackboardComponent::DescribeSelfToVisLog(FVisualLogEntry* Snapshot) const
{
	FVisualLogStatusCategory Category;
	Category.Category = FString::Printf(TEXT("Blackboard (asset: %s)"), *GetNameSafe(BlackboardAsset));

	// describe only when values are initialized
	if (ValueMemory.Num())
	{
		for (UBlackboardData* It = BlackboardAsset; It; It = It->Parent)
		{
			for (int32 KeyIndex = 0; KeyIndex < It->Keys.Num(); KeyIndex++)
			{
				const FBlackboardEntry& Key = It->Keys[KeyIndex];

				const int32 OffsetKey = (int32)It->GetFirstKeyID() + KeyIndex;
				check(FBlackboard::FKey(OffsetKey) != FBlackboard::InvalidKey);

				const uint8* ValueData = GetKeyRawData(FBlackboard::FKey(OffsetKey));
				FString ValueDesc = Key.KeyType ? *(Key.KeyType->WrappedDescribeValue(*this, ValueData)) : TEXT("empty");

				Category.Add(Key.EntryName.ToString(), ValueDesc);
			}
		}

		Category.Data.Sort();
	}

	Snapshot->Status.Add(Category);
}

#endif

UObject* UBlackboardComponent::GetValueAsObject(const FName& KeyName) const
{
	return GetValue<UBlackboardKeyType_Object>(KeyName);
}

UClass* UBlackboardComponent::GetValueAsClass(const FName& KeyName) const
{
	return GetValue<UBlackboardKeyType_Class>(KeyName);
}

uint8 UBlackboardComponent::GetValueAsEnum(const FName& KeyName) const
{
	return GetValue<UBlackboardKeyType_Enum>(KeyName);
}

int32 UBlackboardComponent::GetValueAsInt(const FName& KeyName) const
{
	return GetValue<UBlackboardKeyType_Int>(KeyName);
}

float UBlackboardComponent::GetValueAsFloat(const FName& KeyName) const
{
	return GetValue<UBlackboardKeyType_Float>(KeyName);
}

bool UBlackboardComponent::GetValueAsBool(const FName& KeyName) const
{
	return GetValue<UBlackboardKeyType_Bool>(KeyName);
}

FString UBlackboardComponent::GetValueAsString(const FName& KeyName) const
{
	return GetValue<UBlackboardKeyType_String>(KeyName);
}

FName UBlackboardComponent::GetValueAsName(const FName& KeyName) const
{
	return GetValue<UBlackboardKeyType_Name>(KeyName);
}

FVector UBlackboardComponent::GetValueAsVector(const FName& KeyName) const
{
	return GetValue<UBlackboardKeyType_Vector>(KeyName);
}

FRotator UBlackboardComponent::GetValueAsRotator(const FName& KeyName) const
{
	return GetValue<UBlackboardKeyType_Rotator>(KeyName);
}

void UBlackboardComponent::SetValueAsObject(const FName& KeyName, UObject* ObjectValue)
{
	const FBlackboard::FKey KeyID = GetKeyID(KeyName);
	SetValue<UBlackboardKeyType_Object>(KeyID, ObjectValue);
}

void UBlackboardComponent::SetValueAsClass(const FName& KeyName, UClass* ClassValue)
{
	const FBlackboard::FKey KeyID = GetKeyID(KeyName);
	SetValue<UBlackboardKeyType_Class>(KeyID, ClassValue);
}

void UBlackboardComponent::SetValueAsEnum(const FName& KeyName, uint8 EnumValue)
{
	const FBlackboard::FKey KeyID = GetKeyID(KeyName);
	SetValue<UBlackboardKeyType_Enum>(KeyID, EnumValue);
}

void UBlackboardComponent::SetValueAsInt(const FName& KeyName, int32 IntValue)
{
	const FBlackboard::FKey KeyID = GetKeyID(KeyName);
	SetValue<UBlackboardKeyType_Int>(KeyID, IntValue);
}

void UBlackboardComponent::SetValueAsFloat(const FName& KeyName, float FloatValue)
{
	const FBlackboard::FKey KeyID = GetKeyID(KeyName);
	SetValue<UBlackboardKeyType_Float>(KeyID, FloatValue);
}

void UBlackboardComponent::SetValueAsBool(const FName& KeyName, bool BoolValue)
{
	const FBlackboard::FKey KeyID = GetKeyID(KeyName);
	SetValue<UBlackboardKeyType_Bool>(KeyID, BoolValue);
}

void UBlackboardComponent::SetValueAsString(const FName& KeyName, FString StringValue)
{
	const FBlackboard::FKey KeyID = GetKeyID(KeyName);
	SetValue<UBlackboardKeyType_String>(KeyID, StringValue);
}

void UBlackboardComponent::SetValueAsName(const FName& KeyName, FName NameValue)
{
	const FBlackboard::FKey KeyID = GetKeyID(KeyName);
	SetValue<UBlackboardKeyType_Name>(KeyID, NameValue);
}

void UBlackboardComponent::SetValueAsVector(const FName& KeyName, FVector VectorValue)
{
	const FBlackboard::FKey KeyID = GetKeyID(KeyName);
	SetValue<UBlackboardKeyType_Vector>(KeyID, VectorValue);
}

void UBlackboardComponent::SetValueAsRotator(const FName& KeyName, FRotator RotatorValue)
{
	const FBlackboard::FKey KeyID = GetKeyID(KeyName);
	SetValue<UBlackboardKeyType_Rotator>(KeyID, RotatorValue);
}

bool UBlackboardComponent::IsVectorValueSet(const FName& KeyName) const
{
	const FBlackboard::FKey KeyID = GetKeyID(KeyName);
	return IsVectorValueSet(KeyID);
}

bool UBlackboardComponent::IsVectorValueSet(FBlackboard::FKey KeyID) const
{
	FVector VectorValue = GetValue<UBlackboardKeyType_Vector>(KeyID);
	return (VectorValue != FAISystem::InvalidLocation);
}

void UBlackboardComponent::ClearValue(const FName& KeyName)
{
	const FBlackboard::FKey KeyID = GetKeyID(KeyName);
	ClearValue(KeyID);
}

void UBlackboardComponent::ClearValue(FBlackboard::FKey KeyID)
{
	if (!ensureAsRuntimeWarning(BlackboardAsset != nullptr))
	{
		return;
	}

	const FBlackboardEntry* EntryInfo = BlackboardAsset->GetKey(KeyID);

	uint8* RawData = GetKeyRawData(KeyID);
	if (RawData)
	{
		const bool bHasData = (EntryInfo->KeyType->WrappedIsEmpty(*this, RawData) == false);
		if (bHasData)
		{
			EntryInfo->KeyType->WrappedClear(*this, RawData);
			NotifyObservers(KeyID);

			if (BlackboardAsset->HasSynchronizedKeys() && IsKeyInstanceSynced(KeyID))
			{
				const bool bKeyHasInstance = EntryInfo->KeyType->HasInstance();

				UBlackboardKeyType* KeyOb = bKeyHasInstance ? KeyInstances[KeyID] : EntryInfo->KeyType;
				const uint16 DataOffset = bKeyHasInstance ? sizeof(FBlackboardInstancedKeyMemory) : 0;
				uint8* InstancedRawData = RawData + DataOffset;

				// grab the value set and apply the same to synchronized keys
				// to avoid virtual function call overhead
				UAISystem* AISystem = UAISystem::GetCurrentSafe(GetWorld());
				for (auto Iter = AISystem->CreateBlackboardDataToComponentsIterator(*BlackboardAsset); Iter; ++Iter)
				{
					UBlackboardComponent* OtherBlackboard = Iter.Value();
					if (OtherBlackboard != nullptr && ShouldSyncWithBlackboard(*OtherBlackboard))
					{
						UBlackboardData* const OtherBlackboardAsset = OtherBlackboard->GetBlackboardAsset();
						const FBlackboard::FKey OtherKeyID = OtherBlackboardAsset ? OtherBlackboardAsset->GetKeyID(EntryInfo->EntryName) : FBlackboard::InvalidKey;
						if (OtherKeyID != FBlackboard::InvalidKey)
						{
							const FBlackboardEntry* OtherEntryInfo = OtherBlackboard->BlackboardAsset->GetKey(OtherKeyID);
							UBlackboardKeyType* OtherKeyOb = bKeyHasInstance ? OtherBlackboard->KeyInstances[OtherKeyID] : EntryInfo->KeyType;
							uint8* OtherRawData = OtherBlackboard->GetKeyRawData(OtherKeyID) + DataOffset;

							OtherKeyOb->CopyValues(*OtherBlackboard, OtherRawData, KeyOb, InstancedRawData);
							OtherBlackboard->NotifyObservers(OtherKeyID);
						}
					}
				}
			}
		}
	}
}

bool UBlackboardComponent::CopyKeyValue(FBlackboard::FKey SourceKeyID, FBlackboard::FKey DestinationKeyID)
{
	UBlackboardData* BBAsset = GetBlackboardAsset();
	if (BBAsset == nullptr)
	{
		return false;
	}

	// copy only when values are initialized
	if (ValueMemory.Num() == 0)
	{
		return false;
	}

	const FBlackboardEntry* SourceValueEntryInfo = BBAsset->GetKey(SourceKeyID);
	const FBlackboardEntry* DestinationValueEntryInfo = BBAsset->GetKey(DestinationKeyID);

	if (SourceValueEntryInfo == nullptr || DestinationValueEntryInfo == nullptr || SourceValueEntryInfo->KeyType == nullptr || DestinationValueEntryInfo->KeyType == nullptr)
	{
		return false;
	}

	if (SourceValueEntryInfo->KeyType->GetClass() != DestinationValueEntryInfo->KeyType->GetClass())
	{
		return false;
	}

	const bool bKeyHasInstance = SourceValueEntryInfo->KeyType->HasInstance();

	const uint16 MemDataOffset = bKeyHasInstance ? sizeof(FBlackboardInstancedKeyMemory) : 0;
	
	const uint8* SourceValueMem = GetKeyRawData(SourceKeyID) + MemDataOffset;
	uint8* DestinationValueMem = GetKeyRawData(DestinationKeyID) + MemDataOffset;

	const UBlackboardKeyType* SourceKeyOb = bKeyHasInstance ? KeyInstances[SourceKeyID] : SourceValueEntryInfo->KeyType;
	UBlackboardKeyType*	DestKeyOb = bKeyHasInstance ? KeyInstances[DestinationKeyID] : DestinationValueEntryInfo->KeyType;

	DestKeyOb->CopyValues(*this, DestinationValueMem, SourceKeyOb, SourceValueMem);

	return true;
}

bool UBlackboardComponent::GetLocationFromEntry(const FName& KeyName, FVector& ResultLocation) const
{
	const FBlackboard::FKey KeyID = GetKeyID(KeyName);
	return GetLocationFromEntry(KeyID, ResultLocation);
}

bool UBlackboardComponent::GetLocationFromEntry(FBlackboard::FKey KeyID, FVector& ResultLocation) const
{
	if (BlackboardAsset && ValueOffsets.IsValidIndex(KeyID))
	{
		const FBlackboardEntry* EntryInfo = BlackboardAsset->GetKey(KeyID);
		if (EntryInfo && EntryInfo->KeyType)
		{
			const uint8* ValueData = ValueMemory.GetData() + ValueOffsets[KeyID];
			return EntryInfo->KeyType->WrappedGetLocation(*this, ValueData, ResultLocation);
		}
	}

	return false;
}

bool UBlackboardComponent::GetRotationFromEntry(const FName& KeyName, FRotator& ResultRotation) const
{
	const FBlackboard::FKey KeyID = GetKeyID(KeyName);
	return GetRotationFromEntry(KeyID, ResultRotation);
}

bool UBlackboardComponent::GetRotationFromEntry(FBlackboard::FKey KeyID, FRotator& ResultRotation) const
{
	if (BlackboardAsset && ValueOffsets.IsValidIndex(KeyID))
	{
		const FBlackboardEntry* EntryInfo = BlackboardAsset->GetKey(KeyID);
		if (EntryInfo && EntryInfo->KeyType)
		{
			const uint8* ValueData = ValueMemory.GetData() + ValueOffsets[KeyID];
			return EntryInfo->KeyType->WrappedGetRotation(*this, ValueData, ResultRotation);
		}
	}

	return false;
}

