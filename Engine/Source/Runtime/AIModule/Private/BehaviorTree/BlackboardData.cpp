// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/BlackboardData.h"
#include "GameFramework/Actor.h"
#include "BehaviorTree/BTNode.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Object.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "AISystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlackboardData)

UBlackboardData::FKeyUpdate UBlackboardData::OnUpdateKeys;

#if WITH_EDITOR
#include "Async/Async.h"
#include "Editor/EditorEngine.h"

extern UNREALED_API UEditorEngine* GEditor;
UBlackboardData::FBlackboardDataChanged UBlackboardData::OnBlackboardDataChanged;
#endif

static void UpdatePersistentKeys(UBlackboardData& Asset)
{
	if (GET_AI_CONFIG_VAR(bAddBlackboardSelfKey))
	{
		// note that UpdatePersistentKey will return non-null only if a given key gets newly created 
		UBlackboardKeyType_Object* SelfKeyType = Asset.UpdatePersistentKey<UBlackboardKeyType_Object>(FBlackboard::KeySelf);
		if (SelfKeyType)
		{
			SelfKeyType->BaseClass = AActor::StaticClass();
#if WITH_EDITOR
			// MarkPackageDirty returning false means marking wasn't possible at this moment. Give it one more try in a moment
			if (GEditor != nullptr && Asset.MarkPackageDirty() == false)
			{
				TWeakObjectPtr<UBlackboardData> WeakAsset = &Asset;
				AsyncTask(ENamedThreads::GameThread, [WeakAsset](){
					if (UBlackboardData* AssetPtr = WeakAsset.Get())
					{
						AssetPtr->MarkPackageDirty();
					}
				});
			}
#endif // WITH_EDITOR
		}
	}
}

bool FBlackboardEntry::operator==(const FBlackboardEntry& Other) const
{
	return (EntryName == Other.EntryName) &&
		((KeyType && Other.KeyType && KeyType->GetClass() == Other.KeyType->GetClass()) || (KeyType == NULL && Other.KeyType == NULL));
}

UBlackboardData::UBlackboardData(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer), FirstKeyID(0)
{
}

FBlackboard::FKey UBlackboardData::GetKeyID(const FName& KeyName) const
{
	return InternalGetKeyID(KeyName, CheckParentKeys);
}

FName UBlackboardData::GetKeyName(FBlackboard::FKey KeyID) const
{
	const FBlackboardEntry* KeyEntry = GetKey(KeyID);
	return KeyEntry ? KeyEntry->EntryName : NAME_None;
}

TSubclassOf<UBlackboardKeyType> UBlackboardData::GetKeyType(FBlackboard::FKey KeyID) const
{
	const FBlackboardEntry* KeyEntry = GetKey(KeyID);
	return KeyEntry && KeyEntry->KeyType ? KeyEntry->KeyType->GetClass() : NULL;
}

bool UBlackboardData::IsKeyInstanceSynced(FBlackboard::FKey KeyID) const
{
	const FBlackboardEntry* KeyEntry = GetKey(KeyID);
	return KeyEntry ? KeyEntry->bInstanceSynced : false;
}

const FBlackboardEntry* UBlackboardData::GetKey(FBlackboard::FKey KeyID) const
{
	if (KeyID != FBlackboard::InvalidKey)
	{
		if ((int32)KeyID >= (int32)FirstKeyID)
		{
			return &Keys[(int32)KeyID - (int32)FirstKeyID];
		}
		else if (Parent)
		{
			return Parent->GetKey(KeyID);
		}
	}

	return NULL;
}

int32 UBlackboardData::GetNumKeys() const
{
	return (int32)FirstKeyID + Keys.Num();
}

FBlackboard::FKey UBlackboardData::InternalGetKeyID(const FName& KeyName, EKeyLookupMode LookupMode) const
{
	for (int32 KeyIndex = 0; KeyIndex < Keys.Num(); KeyIndex++)
	{
		if (Keys[KeyIndex].EntryName == KeyName)
		{
			const int32 OffsetKey = KeyIndex + (int32)FirstKeyID;
			check(FBlackboard::FKey(OffsetKey) != FBlackboard::InvalidKey);

			return FBlackboard::FKey(OffsetKey);
		}
	}

	return Parent && (LookupMode == CheckParentKeys) ? Parent->InternalGetKeyID(KeyName, LookupMode) : FBlackboard::InvalidKey;
}

bool UBlackboardData::IsValid() const
{
	if (Parent)
	{
		for (int32 KeyIndex = 0; KeyIndex < Keys.Num(); KeyIndex++)
		{
			const FBlackboard::FKey KeyID = Parent->InternalGetKeyID(Keys[KeyIndex].EntryName, CheckParentKeys);
			if (KeyID != FBlackboard::InvalidKey)
			{
				UE_LOGF(LogBehaviorTree, Error, "Blackboard asset (%ls) has duplicated key (%ls) in parent chain!",
					*GetName(), *Keys[KeyIndex].EntryName.ToString());

				return false;
			}
		}
	}

	return true;
}

void UBlackboardData::UpdateIfHasSynchronizedKeys()
{
	bHasSynchronizedKeys = Parent != nullptr && Parent->bHasSynchronizedKeys;
	for (int32 KeyIndex = 0; KeyIndex < Keys.Num() && bHasSynchronizedKeys == false; ++KeyIndex)
	{
		bHasSynchronizedKeys |= Keys[KeyIndex].bInstanceSynced;
	}
}

void UBlackboardData::PostInitProperties()
{
	Super::PostInitProperties();
	if (HasAnyFlags(RF_NeedPostLoad | RF_ClassDefaultObject) == false)
	{
		UpdatePersistentKeys(*this);
	}
}

void UBlackboardData::PostLoad()
{
	Super::PostLoad();

	// we cache some information based on Parent asset
	// but while UnrealEngine guarantees the Parent is already loaded
	// it does not guarantee that it's PostLoad has been called
	// Following is a little hack that's widely used in the
	// engine to address this
	if (Parent)
	{
		Parent->ConditionalPostLoad();
	}

	UpdateParentKeys();
	UpdateIfHasSynchronizedKeys();
}

#if WITH_EDITOR
void UBlackboardData::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName NAME_Parent = GET_MEMBER_NAME_CHECKED(UBlackboardData, Parent);
	static const FName NAME_InstanceSynced = GET_MEMBER_NAME_CHECKED(FBlackboardEntry, bInstanceSynced);
	static const FName NAME_Keys = GET_MEMBER_NAME_CHECKED(UBlackboardData, Keys);

	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		if (PropertyChangedEvent.Property->GetFName() == NAME_Parent)
		{
			// look for cycles
			if (Parent && Parent->IsChildOf(*this))
			{
				UE_LOGF(LogBehaviorTree, Warning, "Blackboard asset (%ls) has (%ls) in parent chain! Clearing value to avoid cycle.",
					*GetNameSafe(Parent), *GetNameSafe(this));

				Parent = NULL;
			}

			UpdateParentKeys();
		}

		if (PropertyChangedEvent.Property->GetFName() == NAME_InstanceSynced || PropertyChangedEvent.Property->GetFName() == NAME_Parent)
		{
			UpdateIfHasSynchronizedKeys();
		}
	}
	if (PropertyChangedEvent.MemberProperty)
	{
		if (PropertyChangedEvent.MemberProperty->GetFName() == NAME_Keys)
		{
			// look for BB assets using this one as a parent and update them as well
			PropagateKeyChangesToDerivedBlackboardAssets();
		}
	}

	UBlackboardData::OnBlackboardDataChanged.Broadcast(this);
}
#endif // WITH_EDITOR

void UBlackboardData::PropagateKeyChangesToDerivedBlackboardAssets()
{
	for (TObjectIterator<UBlackboardData> It; It; ++It)
	{
		if (It->Parent == this)
		{
			It->UpdateParentKeys();
			It->UpdateIfHasSynchronizedKeys();
			It->PropagateKeyChangesToDerivedBlackboardAssets();
		}
	}
}

static bool ContainsKeyName(FName KeyName, const TArray<FBlackboardEntry>& Keys, const TArray<FBlackboardEntry>& ParentKeys)
{
	for (int32 KeyIndex = 0; KeyIndex < Keys.Num(); KeyIndex++)
	{
		if (Keys[KeyIndex].EntryName == KeyName)
		{
			return true;
		}
	}

	for (int32 KeyIndex = 0; KeyIndex < ParentKeys.Num(); KeyIndex++)
	{
		if (ParentKeys[KeyIndex].EntryName == KeyName)
		{
			return true;
		}
	}

	return false;
}

void UBlackboardData::UpdateParentKeys()
{
	if (Parent == this)
	{
		Parent = NULL;
	}

	UpdateKeyIDs();
	UpdatePersistentKeys(*this);

#if WITH_EDITORONLY_DATA
	// note that we need to gather ParentKeys only once UpdatePersistentKeys was called since that will remove 
	// this BB asset's persistent keys that double the ones already present in the parent asset.
	ParentKeys.Reset();

	for (UBlackboardData* It = Parent; It; It = It->Parent)
	{
		for (int32 KeyIndex = 0; KeyIndex < It->Keys.Num(); KeyIndex++)
		{
			const bool bAlreadyExist = ContainsKeyName(It->Keys[KeyIndex].EntryName, Keys, ParentKeys);
			if (!bAlreadyExist)
			{
				ParentKeys.Add(It->Keys[KeyIndex]);
			}
		}
	}
#endif // WITH_EDITORONLY_DATA

	OnUpdateKeys.Broadcast(this);
}

void UBlackboardData::UpdateKeyIDs()
{
	const int32 FirstKeyIDInt = Parent ? Parent->GetNumKeys() : 0;
	check(FBlackboard::FKey(FirstKeyIDInt) != FBlackboard::InvalidKey);

	FirstKeyID = FBlackboard::FKey(FirstKeyIDInt);
}

void UBlackboardData::UpdateDeprecatedKeys()
{
	for (int32 KeyIndex = 0; KeyIndex < Keys.Num(); KeyIndex++)
	{
		FBlackboardEntry& Entry = Keys[KeyIndex];
		if (Entry.KeyType)
		{
			UBlackboardKeyType* UpdatedKey = Entry.KeyType->UpdateDeprecatedKey();
			if (UpdatedKey)
			{
				Entry.KeyType = UpdatedKey;
			}
		}
	}
}

bool UBlackboardData::IsChildOf(const UBlackboardData& OtherAsset) const
{
	const UBlackboardData* TmpParent = Parent;
	
	// rewind
	while (TmpParent != nullptr && TmpParent != &OtherAsset)
	{
		TmpParent = TmpParent->Parent;
	}

	return (TmpParent == &OtherAsset);
}


