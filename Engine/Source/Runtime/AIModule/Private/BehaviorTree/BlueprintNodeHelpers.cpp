// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintNodeHelpers.h"
#include "UObject/UnrealType.h"
#include "UObject/PropertyPortFlags.h"
#include "TimerManager.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Pawn.h"
#include "Engine/World.h"
#include "GameFramework/Controller.h"
#include "BehaviorTree/BehaviorTreeTypes.h"
#include "BehaviorTree/BTNode.h"
#include "BehaviorTree/ValueOrBBKey.h"
#include "GameplayTagContainer.h"
#include "UObject/TextProperty.h"

namespace BlueprintNodeHelpers
{
	uint16 GetPropertiesMemorySize(const TArray<FProperty*>& PropertyData)
	{
		int32 TotalMem = 0;
		for (int32 PropertyIndex = 0; PropertyIndex < PropertyData.Num(); PropertyIndex++)
		{
			TotalMem += PropertyData[PropertyIndex]->GetSize();
		}

		if (TotalMem > MAX_uint16)
		{
			TotalMem = 0;
		}

		return IntCastChecked<uint16>(TotalMem);
	}

#define GET_STRUCT_NAME_CHECKED(StructName) \
	((void)sizeof(StructName), TEXT(#StructName))

	bool CanUsePropertyType(const FProperty* TestProperty)
	{
		if (TestProperty->IsA(FNumericProperty::StaticClass()) ||
			TestProperty->IsA(FBoolProperty::StaticClass()) ||
			TestProperty->IsA(FNameProperty::StaticClass()) ||
			TestProperty->IsA(FStrProperty::StaticClass()) ||
			TestProperty->IsA(FTextProperty::StaticClass()) )
		{
			return true;
		}

		const FStructProperty* StructProp = CastField<const FStructProperty>(TestProperty);
		if (StructProp)
		{
			FString CPPType = StructProp->GetCPPType(NULL, CPPF_None);
			if (CPPType.Contains(GET_STRUCT_NAME_CHECKED(FVector)) ||
				CPPType.Contains(GET_STRUCT_NAME_CHECKED(FRotator)))
			{
				return true;
			}
		}

		return false;
	}

	void CollectPropertyData(const UObject* Ob, const UClass* StopAtClass, TArray<FProperty*>& PropertyData)
	{
		UE_LOGF(LogBehaviorTree, Verbose, "Looking for runtime properties of class: %ls", *GetNameSafe(Ob->GetClass()));

		PropertyData.Reset();
		for (FProperty* TestProperty = Ob->GetClass()->PropertyLink; TestProperty; TestProperty = TestProperty->PropertyLinkNext)
		{
			// stop when reaching base class
			if (TestProperty->GetOwner<UObject>() == StopAtClass)
			{
				break;
			}

			// skip properties without any setup data
			if (TestProperty->HasAnyPropertyFlags(CPF_Transient) ||
				TestProperty->HasAnyPropertyFlags(CPF_DisableEditOnInstance) == false)
			{
				continue;
			}

			// serialize only simple types
			if (CanUsePropertyType(TestProperty))
			{
				UE_LOGF(LogBehaviorTree, Verbose, "> name: '%ls'", *GetNameSafe(TestProperty));
				PropertyData.Add(TestProperty);
			}
		}
	}

	FString DescribeProperty(const FProperty* Prop, const uint8* PropertyAddr)
	{
		FString ExportedStringValue;
		const FStructProperty* StructProp = CastField<const FStructProperty>(Prop);
		const FFloatProperty* FloatProp = CastField<const FFloatProperty>(Prop);

		if (StructProp && StructProp->GetCPPType(NULL, CPPF_None).Contains(GET_STRUCT_NAME_CHECKED(FBlackboardKeySelector)))
		{
			// special case for blackboard key selectors
			const FBlackboardKeySelector* PropertyValue = (const FBlackboardKeySelector*)PropertyAddr;
			ExportedStringValue = PropertyValue->SelectedKeyName.ToString();
		}
		else if (StructProp && StructProp->Struct && StructProp->Struct->IsChildOf(TBaseStructure<FGameplayTag>::Get()))
		{
			ExportedStringValue = ((const FGameplayTag*)PropertyAddr)->ToString();

#if WITH_EDITOR
			const FString CategoryLimit = StructProp->GetMetaData(TEXT("Categories"));
			if (!CategoryLimit.IsEmpty() && ExportedStringValue.StartsWith(CategoryLimit))
			{
				ExportedStringValue.MidInline(CategoryLimit.Len(), MAX_int32, EAllowShrinking::No);
			}
#endif
		}
		else if (StructProp && StructProp->Struct->IsChildOf(FValueOrBlackboardKeyBase::StaticStruct()))
		{
			ExportedStringValue = reinterpret_cast<const FValueOrBlackboardKeyBase*>(PropertyAddr)->ToString();
		}
		else if (FloatProp)
		{
			// special case for floats to remove unnecessary zeros
			const float FloatValue = FloatProp->GetPropertyValue(PropertyAddr);
			ExportedStringValue = FString::SanitizeFloat(FloatValue);
		}
		else
		{
			Prop->ExportTextItem_Direct(ExportedStringValue, PropertyAddr, nullptr, nullptr, PPF_PropertyWindow | PPF_SimpleObjectText, nullptr);
		}

		const bool bIsBool = Prop->IsA(FBoolProperty::StaticClass());
		return FString::Printf(TEXT("%s: %s"), *FName::NameToDisplayString(Prop->GetName(), bIsBool), *ExportedStringValue);
	}

	void CollectBlackboardSelectors(const UObject* Ob, const UClass* StopAtClass, TArray<FName>& KeyNames)
	{
		for (FProperty* TestProperty = Ob->GetClass()->PropertyLink; TestProperty; TestProperty = TestProperty->PropertyLinkNext)
		{
			// stop when reaching base class
			if (TestProperty->GetOwner<UObject>() == StopAtClass)
			{
				break;
			}

			// skip properties without any setup data	
			if (TestProperty->HasAnyPropertyFlags(CPF_Transient) ||
				TestProperty->HasAnyPropertyFlags(CPF_DisableEditOnInstance))
			{
				continue;
			}

			const FStructProperty* StructProp = CastField<const FStructProperty>(TestProperty);
			if (StructProp && StructProp->GetCPPType(NULL, CPPF_None).Contains(GET_STRUCT_NAME_CHECKED(FBlackboardKeySelector)))
			{
				const FBlackboardKeySelector* PropData = TestProperty->ContainerPtrToValuePtr<FBlackboardKeySelector>(Ob);
				KeyNames.AddUnique(PropData->SelectedKeyName);
			}
		}
	}

	void ResolveBlackboardSelectors(UObject& Ob, const UClass& StopAtClass, const UBlackboardData& BlackboardAsset)
	{
		for (FProperty* TestProperty = Ob.GetClass()->PropertyLink; TestProperty; TestProperty = TestProperty->PropertyLinkNext)
		{
			// stop when reaching base class
			if (TestProperty->GetOwner<UObject>() == &StopAtClass)
			{
				break;
			}

			FStructProperty* StructProp = CastField<FStructProperty>(TestProperty);
			if (StructProp && StructProp->GetCPPType(NULL, CPPF_None).Contains(GET_STRUCT_NAME_CHECKED(FBlackboardKeySelector)))
			{
				FBlackboardKeySelector* PropData = const_cast<FBlackboardKeySelector*>(TestProperty->ContainerPtrToValuePtr<FBlackboardKeySelector>(&Ob));
				PropData->ResolveSelectedKey(BlackboardAsset);
			}
		}
	}

	bool HasAnyBlackboardSelectors(const UObject* Ob, const UClass* StopAtClass)
	{
		bool bResult = false;

		for (FProperty* TestProperty = Ob->GetClass()->PropertyLink; TestProperty; TestProperty = TestProperty->PropertyLinkNext)
		{
			// stop when reaching base class
			if (TestProperty->GetOwner<UObject>() == StopAtClass)
			{
				break;
			}

			// skip properties without any setup data	
			if (TestProperty->HasAnyPropertyFlags(CPF_Transient) ||
				TestProperty->HasAnyPropertyFlags(CPF_DisableEditOnInstance))
			{
				continue;
			}

			const FStructProperty* StructProp = CastField<const FStructProperty>(TestProperty);
			if (StructProp && StructProp->GetCPPType(NULL, CPPF_None).Contains(GET_STRUCT_NAME_CHECKED(FBlackboardKeySelector)))
			{
				bResult = true;
				break;
			}
		}

		return bResult;
	}

#undef GET_STRUCT_NAME_CHECKED

	FString CollectPropertyDescription(const UObject* Ob, const UClass* StopAtClass, const TArray<FProperty*>& InPropertiesToSkip)
	{
		return CollectPropertyDescription(Ob, StopAtClass, [&InPropertiesToSkip](FProperty* InTestProperty){ return InPropertiesToSkip.Contains(InTestProperty); });
	}

	FString CollectPropertyDescription(const UObject* Ob, const UClass* StopAtClass, TFunctionRef<bool(FProperty* /*TestProperty*/)> ShouldSkipProperty)
	{
		FString RetString;
		for (FProperty* TestProperty = Ob->GetClass()->PropertyLink; TestProperty; TestProperty = TestProperty->PropertyLinkNext)
		{
			// stop when reaching base class
			if (TestProperty->GetOwner<UObject>() == StopAtClass)
			{
				break;
			}

			// skip properties without any setup data	
			if (TestProperty->HasAnyPropertyFlags(CPF_Transient) ||
				TestProperty->HasAnyPropertyFlags(CPF_DisableEditOnInstance) ||
				ShouldSkipProperty(TestProperty))
			{
				continue;
			}

			if (TestProperty->IsA(FSoftClassProperty::StaticClass()) ||
				TestProperty->IsA(FClassProperty::StaticClass()) ||
				TestProperty->IsA(FStructProperty::StaticClass()) ||
				CanUsePropertyType(TestProperty))
			{
				if (RetString.Len())
				{
					RetString.AppendChar(TEXT('\n'));
				}

				const uint8* PropData = TestProperty->ContainerPtrToValuePtr<uint8>(Ob);
				RetString += DescribeProperty(TestProperty, PropData);
			}
		}

		return RetString;
	}

	void DescribeRuntimeValues(const UObject* Ob, const TArray<FProperty*>& PropertyData, TArray<FString>& RuntimeValues)
	{
		for (int32 PropertyIndex = 0; PropertyIndex < PropertyData.Num(); PropertyIndex++)
		{
			FProperty* TestProperty = PropertyData[PropertyIndex];
			const uint8* PropAddr = TestProperty->ContainerPtrToValuePtr<uint8>(Ob);

			RuntimeValues.Add(DescribeProperty(TestProperty, PropAddr));
		}
	}

	bool FindNodeOwner(AActor* OwningActor, UBTNode* Node, UBehaviorTreeComponent*& OwningComp, int32& OwningInstanceIdx)
	{
		bool bFound = false;

		APawn* OwningPawn = Cast<APawn>(OwningActor);
		if (OwningPawn && OwningPawn->GetController())
		{
			bFound = FindNodeOwner(OwningPawn->GetController(), Node, OwningComp, OwningInstanceIdx);
		}

		if (OwningActor && !bFound)
		{
			for (UActorComponent* Component : OwningActor->GetComponents())
			{
				if (UBehaviorTreeComponent* BTComp = Cast<UBehaviorTreeComponent>(Component))
				{
					const int32 InstanceIdx = BTComp->FindInstanceContainingNode(Node);
					if (InstanceIdx != INDEX_NONE)
					{
						OwningComp = BTComp;
						OwningInstanceIdx = InstanceIdx;
						bFound = true;
						break;
					}
				}
			}
		}

		return bFound;
	}

	void AbortLatentActions(UActorComponent& OwnerOb, const UObject& Ob)
	{
		if (!OwnerOb.HasAnyFlags(RF_BeginDestroyed) && OwnerOb.GetOwner())
		{
			UWorld* MyWorld = OwnerOb.GetOwner()->GetWorld();
			if (MyWorld)
			{
				MyWorld->GetLatentActionManager().RemoveActionsForObject(MakeWeakObjectPtr(const_cast<UObject*>(&Ob)));
				MyWorld->GetTimerManager().ClearAllTimersForObject(&Ob);
			}
		}
	}
}

