// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Composites/BTComposite_Selector.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTComposite_Selector)

UBTComposite_Selector::UBTComposite_Selector(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Selector";
}

int32 UBTComposite_Selector::GetNextChildHandler(FBehaviorTreeSearchData& SearchData, int32 PrevChild, EBTNodeResult::Type LastResult) const
{
	// success = quit
	int32 NextChildIdx = BTSpecialChild::ReturnToParent;

	if (PrevChild == BTSpecialChild::NotInitialized)
	{
		// newly activated: start from first
		NextChildIdx = 0;
	}
	else if (LastResult == EBTNodeResult::Failed && (PrevChild + 1) < GetChildrenNum())
	{
		// failed = choose next child
		NextChildIdx = PrevChild + 1;
	}

	return NextChildIdx;
}

#if WITH_EDITOR

FName UBTComposite_Selector::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Composite.Selector.Icon");
}

#endif

