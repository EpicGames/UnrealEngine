// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Composites/BTComposite_Sequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTComposite_Sequence)

UBTComposite_Sequence::UBTComposite_Sequence(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Sequence";
}

int32 UBTComposite_Sequence::GetNextChildHandler(FBehaviorTreeSearchData& SearchData, int32 PrevChild, EBTNodeResult::Type LastResult) const
{
	// failure = quit
	int32 NextChildIdx = BTSpecialChild::ReturnToParent;

	if (PrevChild == BTSpecialChild::NotInitialized)
	{
		// newly activated: start from first
		NextChildIdx = 0;
	}
	else if (LastResult == EBTNodeResult::Succeeded && (PrevChild + 1) < GetChildrenNum())
	{
		// success = choose next child
		NextChildIdx = PrevChild + 1;
	}

	return NextChildIdx;
}

#if WITH_EDITOR

bool UBTComposite_Sequence::CanAbortLowerPriority() const
{
	// don't allow aborting lower priorities, as it breaks sequence order and doesn't makes sense
	return false;
}

FName UBTComposite_Sequence::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Composite.Sequence.Icon");
}

#endif

