// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Composites/BTComposite_SimpleParallel.h"
#include "GameFramework/Actor.h"
#include "VisualLogger/VisualLogger.h"
#include "BehaviorTree/BTTaskNode.h"
#include "BehaviorTree/BTAuxiliaryNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTComposite_SimpleParallel)

UBTComposite_SimpleParallel::UBTComposite_SimpleParallel(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Simple Parallel";
	INIT_COMPOSITE_NODE_NOTIFY_FLAGS();
	bApplyDecoratorScope = true;
}

int32 UBTComposite_SimpleParallel::GetNextChildHandler(FBehaviorTreeSearchData& SearchData, int32 PrevChild, EBTNodeResult::Type LastResult) const
{
	FBTParallelMemory* MyMemory = GetNodeMemory<FBTParallelMemory>(SearchData);
	int32 NextChildIdx = BTSpecialChild::ReturnToParent;

	if (PrevChild == BTSpecialChild::NotInitialized)
	{
		NextChildIdx = EBTParallelChild::MainTask;
		MyMemory->MainTaskResult = EBTNodeResult::Failed;
		MyMemory->bRepeatMainTask = false;
	}
	else if ((MyMemory->bMainTaskIsActive || MyMemory->bForceBackgroundTree) && !SearchData.OwnerComp.IsRestartPending())
	{
		// if main task is running - always pick background tree
		// unless search is from abort from background tree - return to parent (and break from search when node gets deactivated)
		NextChildIdx = EBTParallelChild::BackgroundTree;
		MyMemory->bForceBackgroundTree = false;
	}
	else if (MyMemory->bRepeatMainTask)
	{
		UE_VLOG(SearchData.OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT("Repeating main task of %s"), *UBehaviorTreeTypes::DescribeNodeHelper(this));
		NextChildIdx = EBTParallelChild::MainTask;
		MyMemory->bRepeatMainTask = false;
	}

	if ((PrevChild == NextChildIdx) && (MyMemory->LastSearchId == SearchData.SearchId))
	{
		// retrying the same branch again within the same search - possible infinite loop
		SearchData.bPostponeSearch = true;
	}

	MyMemory->LastSearchId = SearchData.SearchId;
	return NextChildIdx;
}

void UBTComposite_SimpleParallel::NotifyChildExecution(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, int32 ChildIdx, EBTNodeResult::Type& NodeResult) const
{
	FBTParallelMemory* MyMemory = (FBTParallelMemory*)NodeMemory;
	if (ChildIdx == EBTParallelChild::MainTask)
	{
		MyMemory->MainTaskResult = NodeResult;

		if (NodeResult == EBTNodeResult::InProgress)
		{
			EBTTaskStatus::Type Status = OwnerComp.GetTaskStatus(Children[EBTParallelChild::MainTask].ChildTask);
			if (Status == EBTTaskStatus::Active)
			{
				// can't register if task is currently being aborted (latent abort returns in progress)

				MyMemory->bMainTaskIsActive = true;
				MyMemory->bForceBackgroundTree = false;
				
				OwnerComp.RegisterParallelTask(Children[EBTParallelChild::MainTask].ChildTask);
				RequestDelayedExecution(OwnerComp, EBTNodeResult::Succeeded);
			}
		}
		else if (MyMemory->bMainTaskIsActive)
		{
			MyMemory->bMainTaskIsActive = false;
			
			// notify decorators on main task, ignore observers updates in FakeSearchData - they are not allowed by parallel composite
			FBehaviorTreeSearchData FakeSearchData(OwnerComp);
			NotifyDecoratorsOnDeactivation(FakeSearchData, ChildIdx, NodeResult, true /*bIsRequestInSameInstance*/);

			const int32 MyInstanceIdx = OwnerComp.FindInstanceContainingNode(this);

			// use check() here not IntCastChecked() as MyInstanceIdx can be INDEX_NONE!
			check(MyInstanceIdx < MAX_uint16);
			OwnerComp.UnregisterParallelTask(Children[EBTParallelChild::MainTask].ChildTask, static_cast<uint16>(MyInstanceIdx));
			if (NodeResult != EBTNodeResult::Aborted && !MyMemory->bRepeatMainTask)
			{
				// check if subtree should be aborted when task finished with success/failed result
				if (FinishMode == EBTParallelMode::AbortBackground)
				{
					OwnerComp.RequestExecution((UBTCompositeNode*)this, MyInstanceIdx,
						Children[EBTParallelChild::MainTask].ChildTask, EBTParallelChild::MainTask,
						NodeResult);
				}
			}
		}
		else if (NodeResult == EBTNodeResult::Succeeded && FinishMode == EBTParallelMode::WaitForBackground)
		{
			// special case: if main task finished instantly, but composite is supposed to wait for background tree,
			// request single run of background tree anyway

			MyMemory->bForceBackgroundTree = true;

			RequestDelayedExecution(OwnerComp, EBTNodeResult::Succeeded);
		}
	}
}

void UBTComposite_SimpleParallel::NotifyNodeDeactivation(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type& NodeResult) const
{
	FBTParallelMemory* MyMemory = GetNodeMemory<FBTParallelMemory>(SearchData);
	const uint16 ActiveInstanceIdx = SearchData.OwnerComp.GetActiveInstanceIdx();

	// modify node result if main task has finished
	if (!MyMemory->bMainTaskIsActive)
	{
		NodeResult = MyMemory->MainTaskResult;
	}

	// remove main task
	if (Children.IsValidIndex(EBTParallelChild::MainTask))
	{
		SearchData.AddUniqueUpdate(FBehaviorTreeSearchUpdate(Children[EBTParallelChild::MainTask].ChildTask, ActiveInstanceIdx, EBTNodeUpdateMode::Remove));
	}
}

bool UBTComposite_SimpleParallel::CanNotifyDecoratorsOnDeactivation(FBehaviorTreeSearchData& SearchData, int32 ChildIdx, EBTNodeResult::Type& NodeResult) const
{
	// don't pass deactivation to decorators when parallel is just switching to background tree
	// decorators on running main task will be notified when their task finishes execution
	if (ChildIdx == EBTParallelChild::MainTask)
	{
		FBTParallelMemory* MyMemory = GetNodeMemory<FBTParallelMemory>(SearchData);
		if (MyMemory->bMainTaskIsActive)
		{
			return false;
		}
	}

	return true;
}

uint16 UBTComposite_SimpleParallel::GetInstanceMemorySize() const
{
	return sizeof(FBTParallelMemory);
}

void UBTComposite_SimpleParallel::InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const
{
	InitializeNodeMemory<FBTParallelMemory>(NodeMemory, InitType);
}

void UBTComposite_SimpleParallel::CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const
{
	CleanupNodeMemory<FBTParallelMemory>(NodeMemory, CleanupType);
}

bool UBTComposite_SimpleParallel::CanPushSubtree(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, int32 ChildIdx) const
{
	return (ChildIdx != EBTParallelChild::MainTask);
}

void UBTComposite_SimpleParallel::SetChildOverride(FBehaviorTreeSearchData& SearchData, int8 Index) const
{
	// store information about repeating main task and use it when possible
	// ignore changes to other child indices = background tree, as it's already being auto repeated
	if (Index == EBTParallelChild::MainTask)
	{
		FBTParallelMemory* MyMemory = GetNodeMemory<FBTParallelMemory>(SearchData);
		MyMemory->bRepeatMainTask = true;

		UE_VLOG(SearchData.OwnerComp.GetOwner(), LogBehaviorTree, Log, TEXT("Main task of %s will be repeated."), *UBehaviorTreeTypes::DescribeNodeHelper(this));
	}
}

FString UBTComposite_SimpleParallel::DescribeFinishMode(EBTParallelMode::Type Mode)
{
	static FString FinishDesc[] = { TEXT("AbortBackground"), TEXT("WaitForBackground") };
	return FinishDesc[Mode];
}

FString UBTComposite_SimpleParallel::GetStaticDescription() const
{
	static FString FinishDesc[] = { TEXT("finish with main task"), TEXT("wait for subtree") };

	return FString::Printf(TEXT("%s: %s"), *Super::GetStaticDescription(),
		FinishMode < 2 ? *FinishDesc[FinishMode] : TEXT(""));
}

void UBTComposite_SimpleParallel::DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const
{
	Super::DescribeRuntimeValues(OwnerComp, NodeMemory, Verbosity, Values);

	if (Verbosity == EBTDescriptionVerbosity::Detailed)
	{
		FBTParallelMemory* MyMemory = (FBTParallelMemory*)NodeMemory;
		Values.Add(FString::Printf(TEXT("main task: %s"), MyMemory->bMainTaskIsActive ? TEXT("active") : TEXT("not active")));

		if (MyMemory->bForceBackgroundTree)
		{
			Values.Add(TEXT("force background tree"));
		}
	}
}

#if WITH_EDITOR

bool UBTComposite_SimpleParallel::CanAbortLowerPriority() const
{
	return false;
}

bool UBTComposite_SimpleParallel::CanAbortSelf() const
{
	return false;
}

FName UBTComposite_SimpleParallel::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Composite.SimpleParallel.Icon");
}

#endif // WITH_EDITOR

