// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Decorators/BTDecorator_Blackboard.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Enum.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_NativeEnum.h"
#include "BehaviorTree/BTCompositeNode.h"
#include "BehaviorTree/BlackboardComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTDecorator_Blackboard)

UBTDecorator_Blackboard::UBTDecorator_Blackboard(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Blackboard Based Condition";
	NotifyObserver = EBTBlackboardRestart::ResultChange;
}

bool UBTDecorator_Blackboard::CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	const UBlackboardComponent* BlackboardComp = OwnerComp.GetBlackboardComponent();
	// note that this may produce unexpected logical results. FALSE is a valid return value here as well
	// @todo signal it
	return BlackboardComp && EvaluateOnBlackboard(*BlackboardComp);
}

bool UBTDecorator_Blackboard::EvaluateOnBlackboard(const UBlackboardComponent& BlackboardComp) const
{
	bool bResult = false;
	if (BlackboardKey.SelectedKeyType)
	{
		UBlackboardKeyType* KeyCDO = BlackboardKey.SelectedKeyType->GetDefaultObject<UBlackboardKeyType>();
		const uint8* KeyMemory = BlackboardComp.GetKeyRawData(BlackboardKey.GetSelectedKeyID());

		// KeyMemory can be NULL if the blackboard has its data setup wrong, so we must conditionally handle that case.
		if (ensure(KeyCDO != NULL) && (KeyMemory != NULL))
		{
			const EBlackboardKeyOperation::Type Op = KeyCDO->GetTestOperation();
			switch (Op)
			{
			case EBlackboardKeyOperation::Basic:
				bResult = KeyCDO->WrappedTestBasicOperation(BlackboardComp, KeyMemory, (EBasicKeyOperation::Type)OperationType);
				break;

			case EBlackboardKeyOperation::Arithmetic:
				bResult = KeyCDO->WrappedTestArithmeticOperation(BlackboardComp, KeyMemory, (EArithmeticKeyOperation::Type)OperationType, IntValue, FloatValue);
				break;

			case EBlackboardKeyOperation::Text:
				bResult = KeyCDO->WrappedTestTextOperation(BlackboardComp, KeyMemory, (ETextKeyOperation::Type)OperationType, StringValue);
				break;

			default:
				break;
			}
		}
	}

	return bResult;
}

EBlackboardNotificationResult UBTDecorator_Blackboard::OnBlackboardKeyValueChange(const UBlackboardComponent& Blackboard, FBlackboard::FKey ChangedKeyID)
{
	UBehaviorTreeComponent* BehaviorComp = (UBehaviorTreeComponent*)Blackboard.GetBrainComponent();
	if (BehaviorComp == nullptr)
	{
		return EBlackboardNotificationResult::RemoveObserver;
	}

	if (BlackboardKey.GetSelectedKeyID() == ChangedKeyID)
	{
		// can't simply use BehaviorComp->RequestExecution(this) here, we need to support condition/value change modes

		const EBTDecoratorAbortRequest RequestMode = (NotifyObserver == EBTBlackboardRestart::ValueChange) ? EBTDecoratorAbortRequest::ConditionPassing : EBTDecoratorAbortRequest::ConditionResultChanged;
		ConditionalFlowAbort(*BehaviorComp, RequestMode);
	}

	return EBlackboardNotificationResult::ContinueObserving;
}

void UBTDecorator_Blackboard::DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const
{
	Super::DescribeRuntimeValues(OwnerComp, NodeMemory, Verbosity, Values);

	const UBlackboardComponent* BlackboardComp = OwnerComp.GetBlackboardComponent();
	FString DescKeyValue;

	if (BlackboardComp)
	{
		DescKeyValue = BlackboardComp->DescribeKeyValue(BlackboardKey.GetSelectedKeyID(), EBlackboardDescription::OnlyValue);
	}

	const bool bResult = BlackboardComp && EvaluateOnBlackboard(*BlackboardComp);
	Values.Add(FString::Printf(TEXT("value: %s (%s)"), *DescKeyValue, bResult ? TEXT("pass") : TEXT("fail")));
}

FString UBTDecorator_Blackboard::GetStaticDescription() const
{
	return FString::Printf(TEXT("%s: %s"), *Super::GetStaticDescription(), *CachedDescription);
}

#if WITH_EDITOR
static UEnum* BasicOpEnum = NULL;
static UEnum* ArithmeticOpEnum = NULL;
static UEnum* TextOpEnum = NULL;

static void CacheOperationEnums()
{
	if (BasicOpEnum == NULL)
	{
		BasicOpEnum = StaticEnum<EBasicKeyOperation::Type>();
		check(BasicOpEnum);
	}

	if (ArithmeticOpEnum == NULL)
	{
		ArithmeticOpEnum = StaticEnum<EArithmeticKeyOperation::Type>();
		check(ArithmeticOpEnum);
	}

	if (TextOpEnum == NULL)
	{
		TextOpEnum = StaticEnum<ETextKeyOperation::Type>();
		check(TextOpEnum);
	}
}

void UBTDecorator_Blackboard::BuildDescription()
{
	const UBlackboardData* BlackboardAsset = GetBlackboardAsset();
	const FBlackboardEntry* EntryInfo = BlackboardAsset ? BlackboardAsset->GetKey(BlackboardKey.GetSelectedKeyID()) : NULL;

	FString BlackboardDesc = "invalid";
	if (EntryInfo)
	{
		// safety feature to not crash when changing couple of properties on a bunch
		// while "post edit property" triggers for every each of them
		if (EntryInfo->KeyType && EntryInfo->KeyType->GetClass() == BlackboardKey.SelectedKeyType)
		{
			RefreshEnumBasedDecorator(*EntryInfo);

			const FString KeyName = EntryInfo->EntryName.ToString();
			CacheOperationEnums();		

			const EBlackboardKeyOperation::Type Op = EntryInfo->KeyType->GetTestOperation();
			switch (Op)
			{
			case EBlackboardKeyOperation::Basic:
				BlackboardDesc = FString::Printf(TEXT("%s is %s"), *KeyName, *BasicOpEnum->GetDisplayNameTextByValue(OperationType).ToString());
				break;

			case EBlackboardKeyOperation::Arithmetic:
				BlackboardDesc = FString::Printf(TEXT("%s %s %s"), *KeyName, *ArithmeticOpEnum->GetDisplayNameTextByValue(OperationType).ToString(),
					*EntryInfo->KeyType->DescribeArithmeticParam(IntValue, FloatValue));
				break;

			case EBlackboardKeyOperation::Text:
				BlackboardDesc = FString::Printf(TEXT("%s %s [%s]"), *KeyName, *TextOpEnum->GetDisplayNameTextByValue(OperationType).ToString(), *StringValue);
				break;

			default: break;
			}
		}
	}

	CachedDescription = BlackboardDesc;
}

void UBTDecorator_Blackboard::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (PropertyChangedEvent.Property == NULL)
	{
		return;
	}

	const FName ChangedPropName = PropertyChangedEvent.Property->GetFName();

	const FName SelectedKeyName = GET_MEMBER_NAME_CHECKED(FBlackboardKeySelector, SelectedKeyName);
	const FName IntValueName = GET_MEMBER_NAME_CHECKED(UBTDecorator_Blackboard, IntValue);
	const FName StringValueName = GET_MEMBER_NAME_CHECKED(UBTDecorator_Blackboard, StringValue);

	const bool bIsEnumKey = BlackboardKey.SelectedKeyType == UBlackboardKeyType_Enum::StaticClass()
		|| BlackboardKey.SelectedKeyType == UBlackboardKeyType_NativeEnum::StaticClass();
	
	if (bIsEnumKey && (ChangedPropName == SelectedKeyName || ChangedPropName == IntValueName || ChangedPropName == StringValueName))
	{
		const UBlackboardData* BlackboardAsset = GetBlackboardAsset();
		const FBlackboardEntry* EntryInfo = BlackboardAsset ? BlackboardAsset->GetKey(BlackboardKey.GetSelectedKeyID()) : nullptr;
		if (EntryInfo != nullptr)
		{
			const UEnum* EnumType = (BlackboardKey.SelectedKeyType == UBlackboardKeyType_Enum::StaticClass())
									   ? ((UBlackboardKeyType_Enum*)(EntryInfo->KeyType))->EnumType
									   : ((UBlackboardKeyType_NativeEnum*)(EntryInfo->KeyType))->EnumType;

			if (ChangedPropName == SelectedKeyName)
			{
				// Set both properties to match the first valid value or invalidate them
				if (EnumType != nullptr && EnumType->NumEnums())
				{
					IntValue = IntCastChecked<int32>(EnumType->GetValueByIndex(0));
					StringValue = EnumType->GetNameStringByIndex(0);
				}
				else
				{
					IntValue = INDEX_NONE;
					StringValue.Reset();
				}
			}
			else if (ChangedPropName == IntValueName)
			{
				StringValue = (EnumType != nullptr) ? EnumType->GetNameStringByValue(IntValue) : TEXT("");
			}
			else if (ChangedPropName == StringValueName)
			{
				IntValue = (EnumType != nullptr) ? IntCastChecked<int32>(EnumType->GetValueByNameString(StringValue)) : INDEX_NONE;
			}
		}
	}

	const bool bKeyTypeChanged = (ChangedPropName == GET_MEMBER_NAME_CHECKED(FBlackboardKeySelector, SelectedKeyType));

	const UBlackboardKeyType* KeyCDO = BlackboardKey.SelectedKeyType ? BlackboardKey.SelectedKeyType->GetDefaultObject<UBlackboardKeyType>() : NULL;
	if (bKeyTypeChanged || ChangedPropName == GET_MEMBER_NAME_CHECKED(UBTDecorator_Blackboard, BasicOperation))
	{
		if (KeyCDO && KeyCDO->GetTestOperation() == EBlackboardKeyOperation::Basic)
		{
			OperationType = BasicOperation.GetIntValue();
		}
	}
	if (bKeyTypeChanged || ChangedPropName == GET_MEMBER_NAME_CHECKED(UBTDecorator_Blackboard, ArithmeticOperation))
	{
		if (KeyCDO && KeyCDO->GetTestOperation() == EBlackboardKeyOperation::Arithmetic)
		{
			OperationType = ArithmeticOperation.GetIntValue();
		}
	}
	if (bKeyTypeChanged || ChangedPropName == GET_MEMBER_NAME_CHECKED(UBTDecorator_Blackboard, TextOperation))
	{
		if (KeyCDO && KeyCDO->GetTestOperation() == EBlackboardKeyOperation::Text)
		{
			OperationType = TextOperation.GetIntValue();
		}
	}

	BuildDescription();
}

#endif	// WITH_EDITOR

void UBTDecorator_Blackboard::InitializeFromAsset(UBehaviorTree& Asset)
{
	Super::InitializeFromAsset(Asset);

#if WITH_EDITOR
	// BuildDescription will take care of refreshing enum based decorator
	BuildDescription();
#else
	const UBlackboardData* BlackboardAsset = GetBlackboardAsset();
	if (const FBlackboardEntry* EntryInfo = BlackboardAsset ? BlackboardAsset->GetKey(BlackboardKey.GetSelectedKeyID()) : nullptr)
	{
		RefreshEnumBasedDecorator(*EntryInfo);
	}
#endif // WITH_EDITOR
}

void UBTDecorator_Blackboard::RefreshEnumBasedDecorator(const FBlackboardEntry& Entry)
{
	if (BlackboardKey.SelectedKeyType != UBlackboardKeyType_Enum::StaticClass() &&
		BlackboardKey.SelectedKeyType != UBlackboardKeyType_NativeEnum::StaticClass())
	{
		return;
	}

	const UEnum* Enum = (BlackboardKey.SelectedKeyType == UBlackboardKeyType_Enum::StaticClass())
							? ((UBlackboardKeyType_Enum*)(Entry.KeyType))->EnumType
							: ((UBlackboardKeyType_NativeEnum*)(Entry.KeyType))->EnumType;
	if (Enum == nullptr)
	{
		return;
	}

	// Enum implementation relies on 'StringValue' to store enum name but also
	// synchronizes 'IntValue' as the enumeration value for runtime evaluation
	if (!StringValue.IsEmpty())
	{
		// Pass string as FName to use fast search path whenever possible 
		const FName Name = *StringValue;
		IntValue = IntCastChecked<int32>(Enum->GetValueByName(Name));
	}
	else if (IntValue >= 0 && IntValue < Enum->NumEnums())
	{
		// Previous implementation was using only 'IntValue' to store the enumeration
		// index so we need to extract associated name and update `IntValue` to store the value.
		StringValue = Enum->GetNameByIndex(IntValue).ToString();
		IntValue = IntCastChecked<int32>(Enum->GetValueByIndex(IntValue));
	}
}

