// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Decorators/BTDecorator_BlackboardBase.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "AISystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTDecorator_BlackboardBase)

UBTDecorator_BlackboardBase::UBTDecorator_BlackboardBase(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "BlackboardBase";

	INIT_DECORATOR_NODE_NOTIFY_FLAGS();

	// empty KeySelector = allow everything

	BlackboardKey.AllowNoneAsValue(GET_AI_CONFIG_VAR(bBlackboardKeyDecoratorAllowsNoneAsValue));
}

void UBTDecorator_BlackboardBase::InitializeFromAsset(UBehaviorTree& Asset)
{
	Super::InitializeFromAsset(Asset);

	if (const UBlackboardData* BBAsset = GetBlackboardAsset())
	{
		BlackboardKey.ResolveSelectedKey(*BBAsset);
	}
	else
	{
		BlackboardKey.InvalidateResolvedKey();
	}
}

void UBTDecorator_BlackboardBase::OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	UBlackboardComponent* BlackboardComp = OwnerComp.GetBlackboardComponent();
	if (BlackboardComp)
	{
		auto KeyID = BlackboardKey.GetSelectedKeyID();
		BlackboardComp->RegisterObserver(KeyID, this, FOnBlackboardChangeNotification::CreateUObject(this, &UBTDecorator_BlackboardBase::OnBlackboardKeyValueChange));
	}
}

void UBTDecorator_BlackboardBase::OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	UBlackboardComponent* BlackboardComp = OwnerComp.GetBlackboardComponent();
	if (BlackboardComp)
	{
		BlackboardComp->UnregisterObserversFrom(this);
	}
}

EBlackboardNotificationResult UBTDecorator_BlackboardBase::OnBlackboardKeyValueChange(const UBlackboardComponent& Blackboard, FBlackboard::FKey ChangedKeyID)
{
	UBehaviorTreeComponent* BehaviorComp = (UBehaviorTreeComponent*)Blackboard.GetBrainComponent();
	if (BehaviorComp == nullptr)
	{
		return EBlackboardNotificationResult::RemoveObserver;
	}

	if (BlackboardKey.GetSelectedKeyID() == ChangedKeyID)
	{
		BehaviorComp->RequestExecution(this);		
	}
	return EBlackboardNotificationResult::ContinueObserving;
}

#if WITH_EDITOR

FName UBTDecorator_BlackboardBase::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Decorator.Blackboard.Icon");
}

FString UBTDecorator_BlackboardBase::GetErrorMessage() const
{
	if (GetBlackboardAsset() == nullptr)
	{
		return UE::BehaviorTree::Messages::BlackboardNotSet.ToString();
	}
	return Super::GetErrorMessage();
}

#endif	// WITH_EDITOR

