// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Decorators/BTDecorator_BlueprintBase.h"
#include "AIController.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "BehaviorTree/BTFunctionLibrary.h"
#include "BlueprintNodeHelpers.h"
#include "BehaviorTree/BehaviorTree.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTDecorator_BlueprintBase)

UBTDecorator_BlueprintBase::UBTDecorator_BlueprintBase(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer)
{
	UClass* StopAtClass = UBTDecorator_BlueprintBase::StaticClass();
	ReceiveTickImplementations = FBTNodeBPImplementationHelper::CheckEventImplementationVersion(TEXT("ReceiveTick"), TEXT("ReceiveTickAI"), *this, *StopAtClass);
	ReceiveExecutionStartImplementations = FBTNodeBPImplementationHelper::CheckEventImplementationVersion(TEXT("ReceiveExecutionStart"), TEXT("ReceiveExecutionStartAI"), *this, *StopAtClass);
	ReceiveExecutionFinishImplementations = FBTNodeBPImplementationHelper::CheckEventImplementationVersion(TEXT("ReceiveExecutionFinish"), TEXT("ReceiveExecutionFinishAI"), *this, *StopAtClass);
	ReceiveObserverActivatedImplementations = FBTNodeBPImplementationHelper::CheckEventImplementationVersion(TEXT("ReceiveObserverActivated"), TEXT("ReceiveObserverActivatedAI"), *this, *StopAtClass);
	ReceiveObserverDeactivatedImplementations = FBTNodeBPImplementationHelper::CheckEventImplementationVersion(TEXT("ReceiveObserverDeactivated"), TEXT("ReceiveObserverDeactivatedAI"), *this, *StopAtClass);
	PerformConditionCheckImplementations = FBTNodeBPImplementationHelper::CheckEventImplementationVersion(TEXT("PerformConditionCheck"), TEXT("PerformConditionCheckAI"), *this, *StopAtClass);

	INIT_DECORATOR_NODE_NOTIFY_FLAGS();
	bNotifyBecomeRelevant = ReceiveObserverActivatedImplementations != 0;
	bNotifyCeaseRelevant = ReceiveObserverDeactivatedImplementations != 0;
	bNotifyTick = ReceiveTickImplementations != 0;
	bNotifyActivation = ReceiveExecutionStartImplementations != 0;
	bNotifyDeactivation = ReceiveExecutionFinishImplementations != 0;
	bShowPropertyDetails = true;
	bCheckConditionOnlyBlackBoardChanges = false;
	bIsObservingBB = false;

	// all blueprint based nodes must create instances
	bCreateNodeInstance = true;
}

void UBTDecorator_BlueprintBase::InitializeProperties()
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		UClass* StopAtClass = UBTDecorator_BlueprintBase::StaticClass();
		BlueprintNodeHelpers::CollectPropertyData(this, StopAtClass, PropertyData);

		bIsObservingBB = BlueprintNodeHelpers::HasAnyBlackboardSelectors(this, StopAtClass);
	}
	else
	{
		// Make sure if the users started to observe a BB key after we instantiated this node, that this value get propagated correctly
		bIsObservingBB = GetClass()->GetDefaultObject<UBTDecorator_BlueprintBase>()->bIsObservingBB;
	}
}

void UBTDecorator_BlueprintBase::PostInitProperties()
{
	Super::PostInitProperties();
		
	InitializeProperties();
	
	if (PerformConditionCheckImplementations || bIsObservingBB)
	{
		bNotifyBecomeRelevant = true;
		bNotifyCeaseRelevant = true;
	}
}

void UBTDecorator_BlueprintBase::PostLoad()
{
	Super::PostLoad();

	if (GetFlowAbortMode() != EBTFlowAbortMode::None && bIsObservingBB)
	{
		ObservedKeyNames.Reset();
		UClass* StopAtClass = UBTDecorator_BlueprintBase::StaticClass();
		BlueprintNodeHelpers::CollectBlackboardSelectors(this, StopAtClass, ObservedKeyNames);
		ensure(ObservedKeyNames.Num() > 0);
	}
}

void UBTDecorator_BlueprintBase::InitializeFromAsset(UBehaviorTree& Asset)
{
	Super::InitializeFromAsset(Asset);
	if (Asset.BlackboardAsset)
	{
		BlueprintNodeHelpers::ResolveBlackboardSelectors(*this, *StaticClass(), *Asset.BlackboardAsset);
	}
}

void UBTDecorator_BlueprintBase::SetOwner(AActor* InActorOwner)
{
	ActorOwner = InActorOwner;
	AIOwner = Cast<AAIController>(InActorOwner);
}

void UBTDecorator_BlueprintBase::OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	if (AIOwner != nullptr && (ReceiveObserverActivatedImplementations & FBTNodeBPImplementationHelper::AISpecific))
	{
		ReceiveObserverActivatedAI(AIOwner, AIOwner->GetPawn());
	}
	else if (ReceiveObserverActivatedImplementations & FBTNodeBPImplementationHelper::Generic)
	{
		ReceiveObserverActivated(ActorOwner);
	}

	if (GetNeedsTickForConditionChecking())
	{
		// if set up as observer, and has a condition check, we want to check the condition every frame
		// highly inefficient, but hopefully people will use it only for prototyping.
		bNotifyTick = true;
	}

	UBlackboardComponent* BlackboardComp = OwnerComp.GetBlackboardComponent();
	if (BlackboardComp)
	{
		for (int32 NameIndex = 0; NameIndex < ObservedKeyNames.Num(); NameIndex++)
		{
			const FBlackboard::FKey KeyID = BlackboardComp->GetKeyID(ObservedKeyNames[NameIndex]);
			if (KeyID != FBlackboard::InvalidKey)
			{
				BlackboardComp->RegisterObserver(KeyID, this, FOnBlackboardChangeNotification::CreateUObject(this, &UBTDecorator_BlueprintBase::OnBlackboardKeyValueChange));
			}
		}
	}
}

void UBTDecorator_BlueprintBase::OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	UBlackboardComponent* BlackboardComp = OwnerComp.GetBlackboardComponent();
	if (BlackboardComp)
	{
		BlackboardComp->UnregisterObserversFrom(this);
	}
		
	if (AIOwner != nullptr && (ReceiveObserverDeactivatedImplementations & FBTNodeBPImplementationHelper::AISpecific))
	{
		ReceiveObserverDeactivatedAI(AIOwner, AIOwner->GetPawn());
	}
	else if (ReceiveObserverDeactivatedImplementations & FBTNodeBPImplementationHelper::Generic)
	{
		ReceiveObserverDeactivated(ActorOwner);
	}

	if (GetNeedsTickForConditionChecking() == true && ReceiveTickImplementations == 0)
	{
		// clean up the tick request if no longer "active"
		bNotifyTick = false;
	}
}

void UBTDecorator_BlueprintBase::OnNodeActivation(FBehaviorTreeSearchData& SearchData)
{
	if (AIOwner != nullptr && (ReceiveExecutionStartImplementations & FBTNodeBPImplementationHelper::AISpecific))
	{
		ReceiveExecutionStartAI(AIOwner, AIOwner->GetPawn());
	}
	else if (ReceiveExecutionStartImplementations & FBTNodeBPImplementationHelper::Generic)
	{
		ReceiveExecutionStart(ActorOwner);
	}
}

void UBTDecorator_BlueprintBase::OnNodeDeactivation(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type NodeResult)
{
	if (AIOwner != nullptr && (ReceiveExecutionFinishImplementations & FBTNodeBPImplementationHelper::AISpecific))
	{
		ReceiveExecutionFinishAI(AIOwner, AIOwner->GetPawn(), NodeResult);
	}
	else if (ReceiveExecutionFinishImplementations & FBTNodeBPImplementationHelper::Generic)
	{
		ReceiveExecutionFinish(ActorOwner, NodeResult);
	}
}

void UBTDecorator_BlueprintBase::TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds)
{
	if (AIOwner != nullptr && (ReceiveTickImplementations & FBTNodeBPImplementationHelper::AISpecific))
	{
		ReceiveTickAI(AIOwner, AIOwner->GetPawn(), DeltaSeconds);
	}
	else if (ReceiveTickImplementations & FBTNodeBPImplementationHelper::Generic)
	{
		ReceiveTick(ActorOwner, DeltaSeconds);
	}
		
	// possible this got ticked due to the decorator being configured as an observer
	if (GetNeedsTickForConditionChecking())
	{
		RequestAbort(OwnerComp, EvaluateAbortType(OwnerComp));
	}
}

UBTDecorator_BlueprintBase::EAbortType  UBTDecorator_BlueprintBase::EvaluateAbortType(UBehaviorTreeComponent& OwnerComp) const
{
	if (PerformConditionCheckImplementations == 0)
	{
		return EAbortType::Unknown;
	}

	if (FlowAbortMode == EBTFlowAbortMode::None)
	{
		return EAbortType::NoAbort;
	}

	const bool bIsOnActiveBranch = OwnerComp.IsExecutingBranch(GetMyNode(), GetChildIndex());

	EAbortType AbortType = EAbortType::NoAbort;
	if (bIsOnActiveBranch)
	{
		if ((FlowAbortMode == EBTFlowAbortMode::Self || FlowAbortMode == EBTFlowAbortMode::Both) && CalculateRawConditionValue(OwnerComp, /*NodeMemory*/nullptr) == IsInversed())
		{
			AbortType = EAbortType::DeactivateBranch;
		}
	}
	else 
	{
		if ((FlowAbortMode == EBTFlowAbortMode::LowerPriority || FlowAbortMode == EBTFlowAbortMode::Both) && CalculateRawConditionValue(OwnerComp, /*NodeMemory*/nullptr) != IsInversed())
	    {
			AbortType = EAbortType::ActivateBranch;
	    }
    }

	return AbortType;
}

void UBTDecorator_BlueprintBase::RequestAbort(UBehaviorTreeComponent& OwnerComp, const EAbortType Type)
{
	switch (Type)
	{
	case EAbortType::NoAbort:
		// Nothing to abort, continue
		break;
	case EAbortType::ActivateBranch:
		OwnerComp.RequestBranchActivation(*this, false);
		break;
	case EAbortType::DeactivateBranch:
		OwnerComp.RequestBranchDeactivation(*this);
		break;
	case EAbortType::Unknown:
		checkf(false, TEXT("If decorator is active, it must know whether it needs to abort or not"))
		break;
	}
}

bool UBTDecorator_BlueprintBase::CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	bool CurrentCallResult = false;
	if (PerformConditionCheckImplementations != 0)
	{
		// can't use const functions with blueprints
		UBTDecorator_BlueprintBase* MyNode = (UBTDecorator_BlueprintBase*)this;

		if (AIOwner != nullptr && (PerformConditionCheckImplementations & FBTNodeBPImplementationHelper::AISpecific))
		{
			CurrentCallResult = MyNode->PerformConditionCheckAI(MyNode->AIOwner, MyNode->AIOwner->GetPawn());
		}
		else if (PerformConditionCheckImplementations & FBTNodeBPImplementationHelper::Generic)
		{
			CurrentCallResult = MyNode->PerformConditionCheck(MyNode->ActorOwner);
		}
	}

	return CurrentCallResult;
}

bool UBTDecorator_BlueprintBase::IsDecoratorExecutionActive() const
{
	UBehaviorTreeComponent* OwnerComp = Cast<UBehaviorTreeComponent>(GetOuter());
	return OwnerComp && OwnerComp->IsExecutingBranch(GetMyNode(), GetChildIndex());
}

bool UBTDecorator_BlueprintBase::IsDecoratorObserverActive() const
{
	UBehaviorTreeComponent* OwnerComp = Cast<UBehaviorTreeComponent>(GetOuter());
	return OwnerComp && OwnerComp->IsAuxNodeActive(this);
}

FString UBTDecorator_BlueprintBase::GetStaticDescription() const
{
	FString ReturnDesc =
#if WITH_EDITORONLY_DATA
		CustomDescription.Len() ? CustomDescription :
#endif // WITH_EDITORONLY_DATA
		Super::GetStaticDescription();

	UBTDecorator_BlueprintBase* CDO = (UBTDecorator_BlueprintBase*)(GetClass()->GetDefaultObject());
	if (bShowPropertyDetails && CDO)
	{
		UClass* StopAtClass = UBTDecorator_BlueprintBase::StaticClass();
		FString PropertyDesc = BlueprintNodeHelpers::CollectPropertyDescription(this, StopAtClass, CDO->PropertyData);
		if (PropertyDesc.Len())
		{
			ReturnDesc += TEXT(":\n\n");
			ReturnDesc += PropertyDesc;
		}
	}

	return ReturnDesc;
}

void UBTDecorator_BlueprintBase::DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const
{
	UBTDecorator_BlueprintBase* CDO = (UBTDecorator_BlueprintBase*)(GetClass()->GetDefaultObject());
	if (CDO && CDO->PropertyData.Num())
	{
		BlueprintNodeHelpers::DescribeRuntimeValues(this, CDO->PropertyData, Values);
	}
}

EBlackboardNotificationResult UBTDecorator_BlueprintBase::OnBlackboardKeyValueChange(const UBlackboardComponent& Blackboard, FBlackboard::FKey ChangedKeyID)
{
	UBehaviorTreeComponent* BehaviorComp = Cast<UBehaviorTreeComponent>(Blackboard.GetBrainComponent());
	if (BehaviorComp)
	{
		const EAbortType Type = EvaluateAbortType(*BehaviorComp);
		if (Type != EAbortType::Unknown)
		{
			RequestAbort(*BehaviorComp, Type);
		}
	}
	return BehaviorComp ? EBlackboardNotificationResult::ContinueObserving : EBlackboardNotificationResult::RemoveObserver;
}

void UBTDecorator_BlueprintBase::OnInstanceDestroyed(UBehaviorTreeComponent& OwnerComp)
{
	// force dropping all pending latent actions associated with this blueprint
	BlueprintNodeHelpers::AbortLatentActions(OwnerComp, *this);
}

#if WITH_EDITOR

FName UBTDecorator_BlueprintBase::GetNodeIconName() const
{
	if(PerformConditionCheckImplementations != 0)
	{
		return FName("BTEditor.Graph.BTNode.Decorator.Conditional.Icon");
	}
	else
	{
		return FName("BTEditor.Graph.BTNode.Decorator.NonConditional.Icon");
	}
}

bool UBTDecorator_BlueprintBase::UsesBlueprint() const
{
	return true;
}

#endif // WITH_EDITOR

