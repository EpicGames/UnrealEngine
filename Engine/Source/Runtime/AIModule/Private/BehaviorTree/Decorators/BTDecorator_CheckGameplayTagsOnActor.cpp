// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Decorators/BTDecorator_CheckGameplayTagsOnActor.h"
#include "GameFramework/Actor.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Object.h"
#include "GameplayTagAssetInterface.h"
#include "BehaviorTree/BlackboardComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTDecorator_CheckGameplayTagsOnActor)

UBTDecorator_CheckGameplayTagsOnActor::UBTDecorator_CheckGameplayTagsOnActor(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Gameplay Tag Condition";

	// Accept only actors
	ActorToCheck.AddObjectFilter(this, GET_MEMBER_NAME_CHECKED(UBTDecorator_CheckGameplayTagsOnActor, ActorToCheck), AActor::StaticClass());

	// Default to using Self Actor
	ActorToCheck.SelectedKeyName = FBlackboard::KeySelf;

	// For now, don't allow users to select any "Abort Observers", because it's currently not supported.
	bAllowAbortNone = false;
	bAllowAbortLowerPri = false;
	bAllowAbortChildNodes = false;
}

bool UBTDecorator_CheckGameplayTagsOnActor::CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	const UBlackboardComponent* BlackboardComp = OwnerComp.GetBlackboardComponent();
	if (BlackboardComp == NULL)
	{
		return false;
	}

	IGameplayTagAssetInterface* GameplayTagAssetInterface = Cast<IGameplayTagAssetInterface>(BlackboardComp->GetValue<UBlackboardKeyType_Object>(ActorToCheck.GetSelectedKeyID()));
	if (GameplayTagAssetInterface == NULL)
	{
		return false;
	}
	
	switch (TagsToMatch)
	{
		case EGameplayContainerMatchType::All:
			return GameplayTagAssetInterface->HasAllMatchingGameplayTags(GameplayTags);

		case EGameplayContainerMatchType::Any:
			return GameplayTagAssetInterface->HasAnyMatchingGameplayTags(GameplayTags);

		default:
		{
			UE_LOGF(LogBehaviorTree, Warning, "Invalid value for TagsToMatch (EGameplayContainerMatchType) %d.  Should only be Any or All.", static_cast<int32>(TagsToMatch));
			return false;
		}
	}
}

void UBTDecorator_CheckGameplayTagsOnActor::DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const
{
	Super::DescribeRuntimeValues(OwnerComp, NodeMemory, Verbosity, Values);
}

FString UBTDecorator_CheckGameplayTagsOnActor::GetStaticDescription() const
{
	return FString::Printf(TEXT("%s: %s"), *Super::GetStaticDescription(), *CachedDescription);
}

#if WITH_EDITOR

void UBTDecorator_CheckGameplayTagsOnActor::BuildDescription()
{
	CachedDescription = GameplayTags.ToMatchingText(TagsToMatch, IsInversed()).ToString();
}

void UBTDecorator_CheckGameplayTagsOnActor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (PropertyChangedEvent.Property == NULL)
	{
		return;
	}

	BuildDescription();
}

FString UBTDecorator_CheckGameplayTagsOnActor::GetErrorMessage() const
{
	if (GetBlackboardAsset() == nullptr)
	{
		return UE::BehaviorTree::Messages::BlackboardNotSet.ToString();
	}
	return Super::GetErrorMessage();
}

#endif	// WITH_EDITOR

void UBTDecorator_CheckGameplayTagsOnActor::InitializeFromAsset(UBehaviorTree& Asset)
{
	Super::InitializeFromAsset(Asset);

	if (const UBlackboardData* BBAsset = GetBlackboardAsset())
	{
		ActorToCheck.ResolveSelectedKey(*BBAsset);
	}
	else
	{
		ActorToCheck.InvalidateResolvedKey();
	}

#if WITH_EDITOR
	BuildDescription();
#endif	// WITH_EDITOR
}

