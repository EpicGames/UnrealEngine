// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Decorators/BTDecorator_CompareBBEntries.h"
#include "BehaviorTree/BlackboardComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTDecorator_CompareBBEntries)

UBTDecorator_CompareBBEntries::UBTDecorator_CompareBBEntries(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Compare Blackboard entries";

	Operator = EBlackBoardEntryComparison::Equal;
	INIT_DECORATOR_NODE_NOTIFY_FLAGS();
}

void UBTDecorator_CompareBBEntries::InitializeFromAsset(UBehaviorTree& Asset)
{
	Super::InitializeFromAsset(Asset);

	if (const UBlackboardData* BBAsset = GetBlackboardAsset())
	{
		BlackboardKeyA.ResolveSelectedKey(*BBAsset);
		BlackboardKeyB.ResolveSelectedKey(*BBAsset);
	}
	else
	{
		BlackboardKeyA.InvalidateResolvedKey();
		BlackboardKeyB.InvalidateResolvedKey();
	}
}

// @note I know it's ugly to have "return" statements in many places inside a function, but the way 
// around was very awkward here 
bool UBTDecorator_CompareBBEntries::CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	// first of all require same type
	// @todo this could be checked statically (i.e. in editor, asset creation time)!
	if (BlackboardKeyA.SelectedKeyType != BlackboardKeyB.SelectedKeyType)
	{
		return false;
	}
	
	const UBlackboardComponent* BlackboardComp = OwnerComp.GetBlackboardComponent();
	if (BlackboardComp)
	{
		const EBlackboardCompare::Type Result = BlackboardComp->CompareKeyValues(BlackboardKeyA.SelectedKeyType, BlackboardKeyA.GetSelectedKeyID(), BlackboardKeyB.GetSelectedKeyID());

		return ((Result == EBlackboardCompare::Equal) == (Operator == EBlackBoardEntryComparison::Equal));
	}

	return false;
}

FString UBTDecorator_CompareBBEntries::GetStaticDescription() const 
{
	return FString::Printf(TEXT("%s:\n%s and %s\ncontain %s values")
		, *Super::GetStaticDescription()
		, *BlackboardKeyA.SelectedKeyName.ToString()
		, *BlackboardKeyB.SelectedKeyName.ToString()
		, Operator == EBlackBoardEntryComparison::Equal ? TEXT("EQUAL") : TEXT("NOT EQUAL"));
}

void UBTDecorator_CompareBBEntries::OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	UBlackboardComponent* BlackboardComp = OwnerComp.GetBlackboardComponent();
	if (BlackboardComp)
	{
		BlackboardComp->RegisterObserver(BlackboardKeyA.GetSelectedKeyID(), this, FOnBlackboardChangeNotification::CreateUObject(this, &UBTDecorator_CompareBBEntries::OnBlackboardKeyValueChange));
		BlackboardComp->RegisterObserver(BlackboardKeyB.GetSelectedKeyID(), this, FOnBlackboardChangeNotification::CreateUObject(this, &UBTDecorator_CompareBBEntries::OnBlackboardKeyValueChange));
	}
}

void UBTDecorator_CompareBBEntries::OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	UBlackboardComponent* BlackboardComp = OwnerComp.GetBlackboardComponent();
	if (BlackboardComp)
	{
		BlackboardComp->UnregisterObserversFrom(this);
	}
}

EBlackboardNotificationResult UBTDecorator_CompareBBEntries::OnBlackboardKeyValueChange(const UBlackboardComponent& Blackboard, FBlackboard::FKey ChangedKeyID)
{
	UBehaviorTreeComponent* BehaviorComp = static_cast<UBehaviorTreeComponent*>(Blackboard.GetBrainComponent());
	if (BehaviorComp == nullptr)
	{
		return EBlackboardNotificationResult::RemoveObserver;
	}
	else if (BlackboardKeyA.GetSelectedKeyID() == ChangedKeyID || BlackboardKeyB.GetSelectedKeyID() == ChangedKeyID)
	{
		BehaviorComp->RequestExecution(this);		
	}

	return EBlackboardNotificationResult::ContinueObserving;
}

#if WITH_EDITOR

FName UBTDecorator_CompareBBEntries::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Decorator.CompareBlackboardEntries.Icon");
}

FString UBTDecorator_CompareBBEntries::GetErrorMessage() const
{
	if (GetBlackboardAsset() == nullptr)
	{
		return UE::BehaviorTree::Messages::BlackboardNotSet.ToString();
	}
	return Super::GetErrorMessage();
}

#endif	// WITH_EDITOR

