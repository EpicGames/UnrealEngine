// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Decorators/BTDecorator_ConditionalLoop.h"
#include "GameFramework/Actor.h"
#include "VisualLogger/VisualLogger.h"
#include "BehaviorTree/BTCompositeNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTDecorator_ConditionalLoop)

UBTDecorator_ConditionalLoop::UBTDecorator_ConditionalLoop(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Conditional Loop";
	INIT_DECORATOR_NODE_NOTIFY_FLAGS();

	bAllowAbortNone = false;
	bAllowAbortLowerPri = false;
	bAllowAbortChildNodes = false;
}

bool UBTDecorator_ConditionalLoop::CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	// always allows execution
	return true;
}

EBlackboardNotificationResult UBTDecorator_ConditionalLoop::OnBlackboardKeyValueChange(const UBlackboardComponent& Blackboard, FBlackboard::FKey ChangedKeyID)
{
	// empty, don't react to blackboard value changes
	return EBlackboardNotificationResult::RemoveObserver;
}

void UBTDecorator_ConditionalLoop::OnNodeDeactivation(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type NodeResult)
{
	FBTConditionalLoopDecoratorMemory* DecoratorMemory = GetNodeMemory<FBTConditionalLoopDecoratorMemory>(SearchData);
	checkf(DecoratorMemory, TEXT("Expecting to always have decorator memory available"));

	// protect from infinite loop within single search
	if (DecoratorMemory->SearchId != SearchData.SearchId)
	{
		DecoratorMemory->SearchId = SearchData.SearchId;

		if (NodeResult != EBTNodeResult::Aborted)
		{
			const UBlackboardComponent* BlackboardComp = SearchData.OwnerComp.GetBlackboardComponent();
			const bool bEvalResult = BlackboardComp && EvaluateOnBlackboard(*BlackboardComp);
			UE_VLOG(SearchData.OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT("Loop condition: %s -> %s"),
				bEvalResult ? TEXT("true") : TEXT("false"), (bEvalResult != IsInversed()) ? TEXT("run again!") : TEXT("break"));

			if (bEvalResult != IsInversed())
			{
				GetParentNode()->SetChildOverride(SearchData, GetChildIndex());
			}
		}
	}
}

uint16 UBTDecorator_ConditionalLoop::GetInstanceMemorySize() const
{
	return sizeof(FBTConditionalLoopDecoratorMemory);
}

void UBTDecorator_ConditionalLoop::InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const
{
	InitializeNodeMemory<FBTConditionalLoopDecoratorMemory>(NodeMemory, InitType);
}

void UBTDecorator_ConditionalLoop::CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const
{
	CleanupNodeMemory<FBTConditionalLoopDecoratorMemory>(NodeMemory, CleanupType);
}

#if WITH_EDITOR

FName UBTDecorator_ConditionalLoop::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Decorator.Loop.Icon");
}

#endif	// WITH_EDITOR

