// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Decorators/BTDecorator_ConeCheck.h"
#include "GameFramework/Actor.h"
#include "BehaviorTree/BlackboardComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTDecorator_ConeCheck)

UBTDecorator_ConeCheck::UBTDecorator_ConeCheck(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Cone Check";

	// accept only actors and vectors
	ConeOrigin.AddObjectFilter(this, GET_MEMBER_NAME_CHECKED(UBTDecorator_ConeCheck, ConeOrigin), AActor::StaticClass());
	ConeOrigin.AddVectorFilter(this, GET_MEMBER_NAME_CHECKED(UBTDecorator_ConeCheck, ConeOrigin));
	ConeDirection.AddObjectFilter(this, GET_MEMBER_NAME_CHECKED(UBTDecorator_ConeCheck, ConeDirection), AActor::StaticClass());
	ConeDirection.AddVectorFilter(this, GET_MEMBER_NAME_CHECKED(UBTDecorator_ConeCheck, ConeDirection));
	ConeDirection.AllowNoneAsValue(true);
	Observed.AddObjectFilter(this, GET_MEMBER_NAME_CHECKED(UBTDecorator_ConeCheck, Observed), AActor::StaticClass());
	Observed.AddVectorFilter(this, GET_MEMBER_NAME_CHECKED(UBTDecorator_ConeCheck, Observed));

	INIT_DECORATOR_NODE_NOTIFY_FLAGS();

	// KeepInCone always abort current branch
	FlowAbortMode = EBTFlowAbortMode::None;
	
	ConeHalfAngle = 45.0f;
}

void UBTDecorator_ConeCheck::InitializeFromAsset(UBehaviorTree& Asset)
{
	Super::InitializeFromAsset(Asset);

	if (const UBlackboardData* BBAsset = GetBlackboardAsset())
	{
		ConeOrigin.ResolveSelectedKey(*BBAsset);
		ConeDirection.ResolveSelectedKey(*BBAsset);
		Observed.ResolveSelectedKey(*BBAsset);
	}
	else
	{
		ConeOrigin.InvalidateResolvedKey();
		ConeDirection.InvalidateResolvedKey();
		Observed.InvalidateResolvedKey();
	}
}

bool UBTDecorator_ConeCheck::CalculateDirection(const UBlackboardComponent* BlackboardComp, const FBlackboardKeySelector& Origin, const FBlackboardKeySelector& End, FVector& Direction) const
{
	FVector PointA = FVector::ZeroVector;
	FVector PointB = FVector::ZeroVector;
	FRotator Rotation = FRotator::ZeroRotator;

	if (BlackboardComp)
	{
		if (End.IsNone())
		{
			if (BlackboardComp->GetRotationFromEntry(Origin.GetSelectedKeyID(), Rotation))
			{
				Direction = Rotation.Vector();
				return true;
			}
		}
		else if (BlackboardComp->GetLocationFromEntry(Origin.GetSelectedKeyID(), PointA) && BlackboardComp->GetLocationFromEntry(End.GetSelectedKeyID(), PointB))
		{
			Direction = (PointB - PointA).GetSafeNormal();
			return true;
		}
	}

	return false;
}

FORCEINLINE bool UBTDecorator_ConeCheck::CalcConditionImpl(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	const UBlackboardComponent* BBComponent = OwnerComp.GetBlackboardComponent();

	FVector ConeDir;
	FVector DirectionToObserve;

	return CalculateDirection(BBComponent, ConeOrigin, Observed, DirectionToObserve)
		&& CalculateDirection(BBComponent, ConeOrigin, ConeDirection, ConeDir)
		&& ConeDir.CosineAngle2D(DirectionToObserve) > GetConeHalfAngleDot(OwnerComp);
}

float UBTDecorator_ConeCheck::GetConeHalfAngleDot(const UBehaviorTreeComponent& OwnerComp) const
{
	return FMath::Cos(FMath::DegreesToRadians(ConeHalfAngle.GetValue(OwnerComp)));
}

bool UBTDecorator_ConeCheck::CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	return CalcConditionImpl(OwnerComp, NodeMemory);
}

void UBTDecorator_ConeCheck::OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	TNodeInstanceMemory* DecoratorMemory = CastInstanceNodeMemory<TNodeInstanceMemory>(NodeMemory);
	DecoratorMemory->bLastRawResult = CalcConditionImpl(OwnerComp, NodeMemory);
}

void UBTDecorator_ConeCheck::OnBlackboardChange(const UBlackboardComponent& Blackboard, FBlackboard::FKey ChangedKeyID)
{
	check(false);
}

void UBTDecorator_ConeCheck::TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds)
{
	TNodeInstanceMemory* DecoratorMemory = CastInstanceNodeMemory<TNodeInstanceMemory>(NodeMemory);
	
	const bool bResult = CalcConditionImpl(OwnerComp, NodeMemory);
	if (bResult != DecoratorMemory->bLastRawResult)
	{
		DecoratorMemory->bLastRawResult = bResult;
		OwnerComp.RequestExecution(this);
	}
}

FString UBTDecorator_ConeCheck::GetStaticDescription() const
{
	return FString::Printf(TEXT("%s: is %s in +- %s degrees %s-%s cone")
		, *Super::GetStaticDescription(), *Observed.SelectedKeyName.ToString()
		, *ConeHalfAngle.ToString(), *ConeOrigin.SelectedKeyName.ToString(), *ConeDirection.SelectedKeyName.ToString());
}

void UBTDecorator_ConeCheck::DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const
{
	const UBlackboardComponent* BBComponent = OwnerComp.GetBlackboardComponent();
			
	FVector ConeDir;
	FVector DirectionToObserved;

	if (CalculateDirection(BBComponent, ConeOrigin, Observed, DirectionToObserved)
		&& CalculateDirection(BBComponent, ConeOrigin, ConeDirection, ConeDir))
	{
		const FVector::FReal CurrentAngleDot = ConeDir.CosineAngle2D(DirectionToObserved);
		const FVector::FReal CurrentAngleRad = FMath::Acos(CurrentAngleDot);

		Values.Add(FString::Printf(TEXT("Angle: %.0f (%s cone)"),
			FMath::RadiansToDegrees(CurrentAngleRad),
			CurrentAngleDot < GetConeHalfAngleDot(OwnerComp) ? TEXT("outside") : TEXT("inside")
			));

	}
}

uint16 UBTDecorator_ConeCheck::GetInstanceMemorySize() const
{
	return sizeof(TNodeInstanceMemory);
}

void UBTDecorator_ConeCheck::InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const
{
	InitializeNodeMemory<TNodeInstanceMemory>(NodeMemory, InitType);
}

void UBTDecorator_ConeCheck::CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const
{
	CleanupNodeMemory<TNodeInstanceMemory>(NodeMemory, CleanupType);
}

#if WITH_EDITOR

FName UBTDecorator_ConeCheck::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Decorator.ConeCheck.Icon");
}

FString UBTDecorator_ConeCheck::GetErrorMessage() const
{
	if (GetBlackboardAsset() == nullptr)
	{
		return UE::BehaviorTree::Messages::BlackboardNotSet.ToString();
	}
	return Super::GetErrorMessage();
}

#endif	// WITH_EDITOR

