// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Decorators/BTDecorator_Cooldown.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTDecorator_Cooldown)

UBTDecorator_Cooldown::UBTDecorator_Cooldown(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Cooldown";
	INIT_DECORATOR_NODE_NOTIFY_FLAGS();
	CoolDownTime = 5.0f;
	
	// aborting child nodes doesn't makes sense, cooldown starts after leaving this branch
	bAllowAbortChildNodes = false;
}

void UBTDecorator_Cooldown::PostLoad()
{
	Super::PostLoad();
	bNotifyTick = (FlowAbortMode != EBTFlowAbortMode::None);
}

bool UBTDecorator_Cooldown::CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	FBTCooldownDecoratorMemory* DecoratorMemory = CastInstanceNodeMemory<FBTCooldownDecoratorMemory>(NodeMemory);
	const double RecalcTime = (OwnerComp.GetWorld()->GetTimeSeconds() - CoolDownTime.GetValue(OwnerComp));
	return RecalcTime >= DecoratorMemory->LastUseTimestamp;
}

void UBTDecorator_Cooldown::OnNodeDeactivation(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type NodeResult)
{
	FBTCooldownDecoratorMemory* DecoratorMemory = GetNodeMemory<FBTCooldownDecoratorMemory>(SearchData);
	DecoratorMemory->LastUseTimestamp = SearchData.OwnerComp.GetWorld()->GetTimeSeconds();
	DecoratorMemory->bRequestedRestart = false;
}

void UBTDecorator_Cooldown::TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds)
{
	FBTCooldownDecoratorMemory* DecoratorMemory = CastInstanceNodeMemory<FBTCooldownDecoratorMemory>(NodeMemory);
	if (!DecoratorMemory->bRequestedRestart)
	{
		const double RecalcTime = (OwnerComp.GetWorld()->GetTimeSeconds() - CoolDownTime.GetValue(OwnerComp));
		if (RecalcTime >= DecoratorMemory->LastUseTimestamp)
		{
			DecoratorMemory->bRequestedRestart = true;
			OwnerComp.RequestExecution(this);
		}
	}
}

FString UBTDecorator_Cooldown::GetStaticDescription() const
{
	// basic info: result after time
	return FString::Printf(TEXT("%s: lock for %s s after execution and return %s"), *Super::GetStaticDescription(),
		*CoolDownTime.ToString(), *UBehaviorTreeTypes::DescribeNodeResult(EBTNodeResult::Failed));
}

void UBTDecorator_Cooldown::DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const
{
	Super::DescribeRuntimeValues(OwnerComp, NodeMemory, Verbosity, Values);

	FBTCooldownDecoratorMemory* DecoratorMemory = CastInstanceNodeMemory<FBTCooldownDecoratorMemory>(NodeMemory);
	const double TimePassed = OwnerComp.GetWorld()->GetTimeSeconds() - DecoratorMemory->LastUseTimestamp;
	
	if (TimePassed < CoolDownTime.GetValue(OwnerComp))
	{
		Values.Add(FString::Printf(TEXT("%s in %ss"),
			(FlowAbortMode == EBTFlowAbortMode::None) ? TEXT("unlock") : TEXT("restart"),
			*FString::SanitizeFloat(CoolDownTime.GetValue(OwnerComp) - TimePassed)));
	}
}

uint16 UBTDecorator_Cooldown::GetInstanceMemorySize() const
{
	return sizeof(FBTCooldownDecoratorMemory);
}

void UBTDecorator_Cooldown::InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const
{
	FBTCooldownDecoratorMemory* DecoratorMemory = InitializeNodeMemory<FBTCooldownDecoratorMemory>(NodeMemory, InitType);
	if (InitType == EBTMemoryInit::Initialize)
	{
		DecoratorMemory->LastUseTimestamp = TNumericLimits<double>::Lowest();
	}

	DecoratorMemory->bRequestedRestart = false; ;
}

void UBTDecorator_Cooldown::CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const
{
	CleanupNodeMemory<FBTCooldownDecoratorMemory>(NodeMemory, CleanupType);
}

#if WITH_EDITOR

FName UBTDecorator_Cooldown::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Decorator.Cooldown.Icon");
}

#endif	// WITH_EDITOR

