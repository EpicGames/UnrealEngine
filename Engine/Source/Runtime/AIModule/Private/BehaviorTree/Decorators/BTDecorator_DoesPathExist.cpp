// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Decorators/BTDecorator_DoesPathExist.h"
#include "UObject/Package.h"
#include "GameFramework/Actor.h"
#include "NavigationSystem.h"
#include "NavFilters/NavigationQueryFilter.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "AIController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTDecorator_DoesPathExist)

UBTDecorator_DoesPathExist::UBTDecorator_DoesPathExist(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Does path exist";

	// accept only actors and vectors
	BlackboardKeyA.AddObjectFilter(this, GET_MEMBER_NAME_CHECKED(UBTDecorator_DoesPathExist, BlackboardKeyA), AActor::StaticClass());
	BlackboardKeyA.AddVectorFilter(this, GET_MEMBER_NAME_CHECKED(UBTDecorator_DoesPathExist, BlackboardKeyA));
	BlackboardKeyB.AddObjectFilter(this, GET_MEMBER_NAME_CHECKED(UBTDecorator_DoesPathExist, BlackboardKeyB), AActor::StaticClass());
	BlackboardKeyB.AddVectorFilter(this, GET_MEMBER_NAME_CHECKED(UBTDecorator_DoesPathExist, BlackboardKeyB));

	bAllowAbortLowerPri = false;
	bAllowAbortNone = true;
	bAllowAbortChildNodes = false;
	FlowAbortMode = EBTFlowAbortMode::None;

	BlackboardKeyA.SelectedKeyName = FBlackboard::KeySelf;
	PathQueryType = EPathExistanceQueryType::HierarchicalQuery;
}

void UBTDecorator_DoesPathExist::InitializeFromAsset(UBehaviorTree& Asset)
{
	Super::InitializeFromAsset(Asset);

	if (bUseSelf)
	{
		BlackboardKeyA.SelectedKeyName = FBlackboard::KeySelf;
		bUseSelf = false;
	}

	if (const UBlackboardData* BBAsset = GetBlackboardAsset())
	{
		BlackboardKeyA.ResolveSelectedKey(*BBAsset);
		BlackboardKeyB.ResolveSelectedKey(*BBAsset);
	}
	else
	{
		BlackboardKeyA.InvalidateResolvedKey();
		BlackboardKeyB.InvalidateResolvedKey();
	}
}

bool UBTDecorator_DoesPathExist::CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	const UBlackboardComponent* BlackboardComp = OwnerComp.GetBlackboardComponent();
	if (BlackboardComp == NULL)
	{
		return false;
	}

	FVector PointA = FVector::ZeroVector;
	FVector PointB = FVector::ZeroVector;	
	const bool bHasPointA = BlackboardComp->GetLocationFromEntry(BlackboardKeyA.GetSelectedKeyID(), PointA);
	const bool bHasPointB = BlackboardComp->GetLocationFromEntry(BlackboardKeyB.GetSelectedKeyID(), PointB);
	
	bool bHasPath = false;

	const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(OwnerComp.GetWorld());
	if (NavSys && bHasPointA && bHasPointB)
	{
		const AAIController* AIOwner = OwnerComp.GetAIOwner();
		const ANavigationData* NavData = AIOwner ? NavSys->GetNavDataForProps(AIOwner->GetNavAgentPropertiesRef(), AIOwner->GetNavAgentLocation()) : NULL;
		if (NavData)
		{
			FSharedConstNavQueryFilter QueryFilter = UNavigationQueryFilter::GetQueryFilter(*NavData, AIOwner, FilterClass.GetValue(*BlackboardComp));

			const EPathExistanceQueryType::Type QueryType = PathQueryType.GetValue<EPathExistanceQueryType::Type>(OwnerComp);
			if (QueryType == EPathExistanceQueryType::NavmeshRaycast2D)
			{
				FVector HitLocation;
				FNavigationRaycastAdditionalResults AdditionalResults;
				const bool bDidHit = NavData->Raycast(PointA, PointB, HitLocation, &AdditionalResults, QueryFilter);
				bHasPath = !bDidHit && AdditionalResults.bIsRayEndInCorridor;
			}
			else
			{
				EPathFindingMode::Type TestMode = (QueryType == EPathExistanceQueryType::HierarchicalQuery) ? EPathFindingMode::Hierarchical : EPathFindingMode::Regular;
				bHasPath = NavSys->TestPathSync(FPathFindingQuery(AIOwner, *NavData, PointA, PointB, QueryFilter), TestMode);
			}
		}
	}

	return bHasPath;
}

FString UBTDecorator_DoesPathExist::GetStaticDescription() const 
{
	return FString::Printf(TEXT("%s: Find path from %s to %s (mode:%s)"),
		*Super::GetStaticDescription(),
		*BlackboardKeyA.SelectedKeyName.ToString(),
		*BlackboardKeyB.SelectedKeyName.ToString(),
		*PathQueryType.ToString());
}

#if WITH_EDITOR

FName UBTDecorator_DoesPathExist::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Decorator.DoesPathExist.Icon");
}

FString UBTDecorator_DoesPathExist::GetErrorMessage() const
{
	if (GetBlackboardAsset() == nullptr)
	{
		return UE::BehaviorTree::Messages::BlackboardNotSet.ToString();
	}
	return Super::GetErrorMessage();
}

#endif	// WITH_EDITOR

