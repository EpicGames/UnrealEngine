// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Decorators/BTDecorator_ForceSuccess.h"

#include "BehaviorTree/BTCompositeNode.h"
#include "GameFramework/Actor.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTDecorator_ForceSuccess)

UBTDecorator_ForceSuccess::UBTDecorator_ForceSuccess(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = TEXT("Force Success");
	INIT_DECORATOR_NODE_NOTIFY_FLAGS();

	bAllowAbortNone = false;
	bAllowAbortLowerPri = false;
	bAllowAbortChildNodes = false;
}

void UBTDecorator_ForceSuccess::OnNodeProcessed(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type& NodeResult)
{
	bool bCanForceSuccess = true;

	// Decorator is always allowed to force success during the search to ignore an optional branch in a sequence.
	// But when used to override the node result on failure we only modify result if search originates
	// from our parent node (abort self) or on our associated node (failure).
	if (!SearchData.bSearchInProgress)
	{
		const int32 InstanceIdx = SearchData.OwnerComp.GetActiveInstanceIdx();
		const FBTNodeIndex ParentNodeIndex(InstanceIdx, GetParentNode() != nullptr ? GetParentNode()->GetExecutionIndex() : 0);
		const UBTNode* MyNode = GetMyNode();
		const FBTNodeIndex MyNodeIndex(InstanceIdx, MyNode != nullptr ? MyNode->GetExecutionIndex() : 0);
		bCanForceSuccess = (SearchData.SearchRootNode == ParentNodeIndex || SearchData.SearchRootNode == MyNodeIndex);
	}

	if (bCanForceSuccess)
	{
		checkf(NodeResult != EBTNodeResult::Aborted, TEXT("Should never change a result set to 'Aborted'"));
		NodeResult = EBTNodeResult::Succeeded;
		BT_SEARCHLOG(SearchData, Log, TEXT("Forcing Success: %s"), *UBehaviorTreeTypes::DescribeNodeHelper(this));
	}
}

#if WITH_EDITOR

FName UBTDecorator_ForceSuccess::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Decorator.ForceSuccess.Icon");
}

#endif	// WITH_EDITOR

