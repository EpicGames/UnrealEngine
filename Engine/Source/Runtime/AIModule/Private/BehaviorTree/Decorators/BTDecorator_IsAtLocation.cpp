// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Decorators/BTDecorator_IsAtLocation.h"
#include "GameFramework/Actor.h"
#include "Navigation/PathFollowingComponent.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Object.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Vector.h"
#include "AIController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTDecorator_IsAtLocation)

UBTDecorator_IsAtLocation::UBTDecorator_IsAtLocation(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Is At Location";
	AcceptableRadius = 50.0f;
	bUseParametrizedRadius = false;
	bUseNavAgentGoalLocation = true;
	bPathFindingBasedTest = true;
	GeometricDistanceType = FAIDistanceType::Distance3D;

	// accept only actors and vectors
	BlackboardKey.AddObjectFilter(this, GET_MEMBER_NAME_CHECKED(UBTDecorator_IsAtLocation, BlackboardKey), AActor::StaticClass());
	BlackboardKey.AddVectorFilter(this, GET_MEMBER_NAME_CHECKED(UBTDecorator_IsAtLocation, BlackboardKey));

	// can't abort, it's not observing anything
	bAllowAbortLowerPri = false;
	bAllowAbortNone = false;
	bAllowAbortChildNodes = false;
	FlowAbortMode = EBTFlowAbortMode::None;
}

FVector::FReal UBTDecorator_IsAtLocation::GetGeometricDistanceSquared(const FVector& A, const FVector& B) const
{
	FVector::FReal Result = TNumericLimits<FVector::FReal>::Max();

	switch (GeometricDistanceType)
	{
	case FAIDistanceType::Distance3D:
		Result = FVector::DistSquared(A, B);
		break;
	case FAIDistanceType::Distance2D:
		Result = FVector::DistSquaredXY(A, B);
		break;
	case FAIDistanceType::DistanceZ:
		Result = FMath::Square(A.Z - B.Z);
		break;
	default:
		checkNoEntry();
		break;
	}
	return Result;
}

bool UBTDecorator_IsAtLocation::CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	bool bHasReached = false;

	AAIController* AIOwner = OwnerComp.GetAIOwner();
	UPathFollowingComponent* PathFollowingComponent = AIOwner ? AIOwner->GetPathFollowingComponent() : nullptr;
	if (PathFollowingComponent)
	{
		const UBlackboardComponent* MyBlackboard = OwnerComp.GetBlackboardComponent();
		
		float Radius = AcceptableRadius;
		if (bUseParametrizedRadius && AIOwner && AIOwner->GetPawn() && ParametrizedAcceptableRadius.IsDynamic())
		{
			ParametrizedAcceptableRadius.BindData(AIOwner->GetPawn(), INDEX_NONE);
			Radius = ParametrizedAcceptableRadius.GetValue();
		}

		if (BlackboardKey.SelectedKeyType == UBlackboardKeyType_Object::StaticClass())
		{
			UObject* KeyValue = MyBlackboard->GetValue<UBlackboardKeyType_Object>(BlackboardKey.GetSelectedKeyID());
			AActor* TargetActor = Cast<AActor>(KeyValue);
			if (TargetActor)
			{
				bHasReached = bPathFindingBasedTest.GetValue(OwnerComp) 
					? PathFollowingComponent->HasReached(*TargetActor, EPathFollowingReachMode::OverlapAgentAndGoal, Radius, bUseNavAgentGoalLocation.GetValue(OwnerComp))
					: (AIOwner->GetPawn()
						? (GetGeometricDistanceSquared(AIOwner->GetPawn()->GetActorLocation(), TargetActor->GetActorLocation()) < FMath::Square(Radius))
						: false);
			}
		}
		else if (BlackboardKey.SelectedKeyType == UBlackboardKeyType_Vector::StaticClass())
		{
			const FVector TargetLocation = MyBlackboard->GetValue<UBlackboardKeyType_Vector>(BlackboardKey.GetSelectedKeyID());
			if (FAISystem::IsValidLocation(TargetLocation))
			{
				bHasReached = bPathFindingBasedTest.GetValue(OwnerComp)
					? PathFollowingComponent->HasReached(TargetLocation, EPathFollowingReachMode::OverlapAgent, Radius)
					: (AIOwner->GetPawn()
						? (GetGeometricDistanceSquared(AIOwner->GetPawn()->GetActorLocation(), TargetLocation) < FMath::Square(Radius))
						: false);
			}
		}
	}

	return bHasReached;
}

FString UBTDecorator_IsAtLocation::GetStaticDescription() const
{
	FString KeyDesc("invalid");
	if (BlackboardKey.SelectedKeyType == UBlackboardKeyType_Object::StaticClass() ||
		BlackboardKey.SelectedKeyType == UBlackboardKeyType_Vector::StaticClass())
	{
		KeyDesc = BlackboardKey.SelectedKeyName.ToString();
	}

	return FString::Printf(TEXT("%s: %s"), *Super::GetStaticDescription(), *KeyDesc);
}

#if WITH_EDITOR

FName UBTDecorator_IsAtLocation::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Decorator.ReachedMoveGoal.Icon");
}

#endif	// WITH_EDITOR

