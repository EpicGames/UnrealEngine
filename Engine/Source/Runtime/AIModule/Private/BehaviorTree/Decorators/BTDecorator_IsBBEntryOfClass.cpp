// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Decorators/BTDecorator_IsBBEntryOfClass.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Object.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTDecorator_IsBBEntryOfClass)

UBTDecorator_IsBBEntryOfClass::UBTDecorator_IsBBEntryOfClass(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer)
{
	NodeName = TEXT("Is BlackBoard value of given Class");
	
	BlackboardKey.AddObjectFilter(this, GET_MEMBER_NAME_CHECKED(UBTDecorator_IsBBEntryOfClass, BlackboardKey), UObject::StaticClass());
}

// @todo refactor CalculateRawConditionValue to take const UBlackboardComponent& instead
bool UBTDecorator_IsBBEntryOfClass::CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	const UBlackboardComponent* MyBlackboard = OwnerComp.GetBlackboardComponent();

	if (MyBlackboard)
	{
		UObject* KeyValue = MyBlackboard->GetValue<UBlackboardKeyType_Object>(BlackboardKey.GetSelectedKeyID());

		return KeyValue != nullptr && KeyValue->GetClass()->IsChildOf(TestClass.GetValue(*MyBlackboard));
	}

	return false;
}

EBlackboardNotificationResult UBTDecorator_IsBBEntryOfClass::OnBlackboardKeyValueChange(const UBlackboardComponent& Blackboard, FBlackboard::FKey ChangedKeyID)
{
	check(Cast<UBehaviorTreeComponent>(Blackboard.GetBrainComponent()));
	check(BlackboardKey.GetSelectedKeyID() == ChangedKeyID);

	// using nullptr for memory since we're not using any
	if (CalculateRawConditionValue(*CastChecked<UBehaviorTreeComponent>(Blackboard.GetBrainComponent()), nullptr) != IsInversed())
	{
		static_cast<UBehaviorTreeComponent*>(Blackboard.GetBrainComponent())->RequestExecution(this);
	}

	return EBlackboardNotificationResult::ContinueObserving;
}

void UBTDecorator_IsBBEntryOfClass::DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const
{
	Values.Add(GetStaticDescription());
}

FString UBTDecorator_IsBBEntryOfClass::GetStaticDescription() const
{
	UBlackboardData* BlackboardAsset = GetBlackboardAsset();
	const FBlackboardEntry* EntryInfo = BlackboardAsset ? BlackboardAsset->GetKey(BlackboardKey.GetSelectedKeyID()) : NULL;

	FString BlackboardDesc = "invalid";
	if (EntryInfo != nullptr
		// safety feature to not crash when changing couple of properties on a bunch
		// while "post edit property" triggers for every each of them
		&& EntryInfo->KeyType->GetClass() == BlackboardKey.SelectedKeyType)
	{
		const FString KeyName = EntryInfo->EntryName.ToString();

		BlackboardDesc = FString::Printf(TEXT("Check if %s is of class %s"), *KeyName, *TestClass.ToString());
	}

	return BlackboardDesc;
}

