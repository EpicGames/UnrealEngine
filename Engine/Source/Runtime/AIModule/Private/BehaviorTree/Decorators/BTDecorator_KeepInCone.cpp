// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Decorators/BTDecorator_KeepInCone.h"
#include "GameFramework/Actor.h"
#include "BehaviorTree/BlackboardComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTDecorator_KeepInCone)

UBTDecorator_KeepInCone::UBTDecorator_KeepInCone(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Keep in Cone";

	// accept only actors and vectors
	ConeOrigin.AddObjectFilter(this, GET_MEMBER_NAME_CHECKED(UBTDecorator_KeepInCone, ConeOrigin), AActor::StaticClass());
	ConeOrigin.AddVectorFilter(this, GET_MEMBER_NAME_CHECKED(UBTDecorator_KeepInCone, ConeOrigin));
	Observed.AddObjectFilter(this, GET_MEMBER_NAME_CHECKED(UBTDecorator_KeepInCone, Observed), AActor::StaticClass());
	Observed.AddVectorFilter(this, GET_MEMBER_NAME_CHECKED(UBTDecorator_KeepInCone, Observed));

	INIT_DECORATOR_NODE_NOTIFY_FLAGS();

	// KeepInCone always abort current branch
	bAllowAbortLowerPri = false;
	bAllowAbortNone = false;
	FlowAbortMode = EBTFlowAbortMode::Self;
	
	ConeOrigin.SelectedKeyName = FBlackboard::KeySelf;
	ConeHalfAngle = 45.0f;
}

float UBTDecorator_KeepInCone::GetConeHalfAngleDot(const UBehaviorTreeComponent& OwnerComp) const
{
	return  FMath::Cos(FMath::DegreesToRadians(ConeHalfAngle.GetValue(OwnerComp)));
}

void UBTDecorator_KeepInCone::InitializeFromAsset(UBehaviorTree& Asset)
{
	Super::InitializeFromAsset(Asset);

	if (bUseSelfAsOrigin)
	{
		ConeOrigin.SelectedKeyName = FBlackboard::KeySelf;
		bUseSelfAsOrigin = false;
	}

	if (bUseSelfAsObserved)
	{
		Observed.SelectedKeyName = FBlackboard::KeySelf;
		bUseSelfAsObserved = false;
	}

	if (const UBlackboardData* BBAsset = GetBlackboardAsset())
	{
		ConeOrigin.ResolveSelectedKey(*BBAsset);
		Observed.ResolveSelectedKey(*BBAsset);
	}
	else
	{
		ConeOrigin.InvalidateResolvedKey();
		Observed.InvalidateResolvedKey();
	}
}

bool UBTDecorator_KeepInCone::CalculateCurrentDirection(const UBehaviorTreeComponent& OwnerComp, FVector& Direction) const
{
	const UBlackboardComponent* BlackboardComp = OwnerComp.GetBlackboardComponent();
	if (BlackboardComp == nullptr)
	{
		return false;
	}

	FVector PointA = FVector::ZeroVector;
	FVector PointB = FVector::ZeroVector;
	const bool bHasPointA = BlackboardComp->GetLocationFromEntry(ConeOrigin.GetSelectedKeyID(), PointA);
	const bool bHasPointB = BlackboardComp->GetLocationFromEntry(Observed.GetSelectedKeyID(), PointB);

	if (bHasPointA && bHasPointB)
	{
		Direction = (PointB - PointA).GetSafeNormal();
		return true;
	}

	return false;
}

void UBTDecorator_KeepInCone::OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	TNodeInstanceMemory* DecoratorMemory = CastInstanceNodeMemory<TNodeInstanceMemory>(NodeMemory);
	FVector InitialDir(1.0f, 0, 0);

	CalculateCurrentDirection(OwnerComp, InitialDir);
	DecoratorMemory->InitialDirection = InitialDir;
}

void UBTDecorator_KeepInCone::TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds)
{
	const TNodeInstanceMemory* DecoratorMemory = CastInstanceNodeMemory<TNodeInstanceMemory>(NodeMemory);
	FVector CurrentDir(1.0f, 0, 0);
	
	if (CalculateCurrentDirection(OwnerComp, CurrentDir))
	{
		const FVector::FReal Angle = DecoratorMemory->InitialDirection.CosineAngle2D(CurrentDir);
		const float ConeHalfAngleDot = GetConeHalfAngleDot(OwnerComp);
		if (Angle < ConeHalfAngleDot || (IsInversed() && Angle > ConeHalfAngleDot))
		{
			OwnerComp.RequestExecution(this);
		}
	}
}

FString UBTDecorator_KeepInCone::GetStaticDescription() const
{
	return FString::Printf(TEXT("%s: %s in +- %s degree cone of initial direction [%s-%s]"),
		*Super::GetStaticDescription(),
		*Observed.SelectedKeyName.ToString(),
		*ConeHalfAngle.ToString(),
		*ConeOrigin.SelectedKeyName.ToString(),
		*Observed.SelectedKeyName.ToString());
}

void UBTDecorator_KeepInCone::DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const
{
	const TNodeInstanceMemory* DecoratorMemory = CastInstanceNodeMemory<TNodeInstanceMemory>(NodeMemory);
	FVector CurrentDir(1.0f, 0, 0);
	
	if (CalculateCurrentDirection(OwnerComp, CurrentDir))
	{
		const FVector::FReal CurrentAngleDot = DecoratorMemory->InitialDirection.CosineAngle2D(CurrentDir);
		const FVector::FReal CurrentAngleRad = FMath::Acos(CurrentAngleDot);

		Values.Add(FString::Printf(TEXT("Angle: %.0f (%s cone)"),
			FMath::RadiansToDegrees(CurrentAngleRad),
			CurrentAngleDot < GetConeHalfAngleDot(OwnerComp) ? TEXT("outside") : TEXT("inside")
			));

	}
}

uint16 UBTDecorator_KeepInCone::GetInstanceMemorySize() const
{
	return sizeof(TNodeInstanceMemory);
}

void UBTDecorator_KeepInCone::InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const
{
	InitializeNodeMemory<TNodeInstanceMemory>(NodeMemory, InitType);
}

void UBTDecorator_KeepInCone::CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const
{
	CleanupNodeMemory<TNodeInstanceMemory>(NodeMemory, CleanupType);
}

#if WITH_EDITOR

FName UBTDecorator_KeepInCone::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Decorator.KeepInCone.Icon");
}

FString UBTDecorator_KeepInCone::GetErrorMessage() const
{
	if (GetBlackboardAsset() == nullptr)
	{
		return UE::BehaviorTree::Messages::BlackboardNotSet.ToString();
	}
	return Super::GetErrorMessage();
}

#endif	// WITH_EDITOR

