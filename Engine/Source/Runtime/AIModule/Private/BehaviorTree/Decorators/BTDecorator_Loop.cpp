// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Decorators/BTDecorator_Loop.h"
#include "Engine/World.h"
#include "BehaviorTree/BTCompositeNode.h"
#include "BehaviorTree/Composites/BTComposite_SimpleParallel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTDecorator_Loop)

UBTDecorator_Loop::UBTDecorator_Loop(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Loop";
	NumLoops = 3;
	InfiniteLoopTimeoutTime = -1.f;
	INIT_DECORATOR_NODE_NOTIFY_FLAGS();
	
	bAllowAbortNone = false;
	bAllowAbortLowerPri = false;
	bAllowAbortChildNodes = false;
}

void UBTDecorator_Loop::OnNodeActivation(FBehaviorTreeSearchData& SearchData)
{
	FBTLoopDecoratorMemory* DecoratorMemory = GetNodeMemory<FBTLoopDecoratorMemory>(SearchData);
	FBTCompositeMemory* ParentMemory = GetParentNode()->GetNodeMemory<FBTCompositeMemory>(SearchData);
	const bool bIsSpecialNode = GetParentNode()->IsA(UBTComposite_SimpleParallel::StaticClass());

	if ((bIsSpecialNode && ParentMemory->CurrentChild == BTSpecialChild::NotInitialized) ||
		(!bIsSpecialNode && ParentMemory->CurrentChild != ChildIndex))
	{
		// initialize counter if it's first activation
		DecoratorMemory->RemainingExecutions = IntCastChecked<uint8>(NumLoops.GetValue(SearchData.OwnerComp));
		DecoratorMemory->TimeStarted = GetWorld()->GetTimeSeconds();
	}

	bool bShouldLoop = false;
	if (bInfiniteLoop)
	{
		// protect from truly infinite loop within single search
		if (SearchData.SearchId != DecoratorMemory->SearchId)
		{
			const float Timeout = InfiniteLoopTimeoutTime.GetValue(SearchData.OwnerComp);
			if ((Timeout < 0.f) || ((DecoratorMemory->TimeStarted + Timeout) > GetWorld()->GetTimeSeconds()))
			{
				bShouldLoop = true;
			}
		}

		DecoratorMemory->SearchId = SearchData.SearchId;
	}
	else
	{
		if (DecoratorMemory->RemainingExecutions > 0)
		{
			DecoratorMemory->RemainingExecutions--;
		}
		bShouldLoop = DecoratorMemory->RemainingExecutions > 0;
	}


	// set child selection overrides
	if (bShouldLoop)
	{
		GetParentNode()->SetChildOverride(SearchData, ChildIndex);
	}
}

FString UBTDecorator_Loop::GetStaticDescription() const
{
	// basic info: infinite / num loops
	if (bInfiniteLoop)
	{
		if (InfiniteLoopTimeoutTime.GetKey().IsNone() && InfiniteLoopTimeoutTime.GetValue(static_cast<const UBehaviorTreeComponent*>(nullptr)) <= 0.f)
		{
			return FString::Printf(TEXT("%s: infinite"), *Super::GetStaticDescription());
		}
		else
		{
			return FString::Printf(TEXT("%s: loop for %s seconds"), *Super::GetStaticDescription(), *InfiniteLoopTimeoutTime.ToString());
		}
	}
	else
	{
		return FString::Printf(TEXT("%s: %s loops"), *Super::GetStaticDescription(), *NumLoops.ToString());
	}
}

void UBTDecorator_Loop::DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const
{
	Super::DescribeRuntimeValues(OwnerComp, NodeMemory, Verbosity, Values);

	const float Timeout = InfiniteLoopTimeoutTime.GetValue(OwnerComp);
	if (!bInfiniteLoop)
	{
		FBTLoopDecoratorMemory* DecoratorMemory = (FBTLoopDecoratorMemory*)NodeMemory;
		Values.Add(FString::Printf(TEXT("loops remaining: %d"), DecoratorMemory->RemainingExecutions));
	}
	else if (Timeout > 0.f)
	{
		FBTLoopDecoratorMemory* DecoratorMemory = (FBTLoopDecoratorMemory*)NodeMemory;

		const double TimeRemaining = FMath::Max(Timeout - (GetWorld()->GetTimeSeconds() - DecoratorMemory->TimeStarted), 0.f);
		Values.Add(FString::Printf(TEXT("time remaining: %s"), *FString::SanitizeFloat(TimeRemaining)));
	}
}

uint16 UBTDecorator_Loop::GetInstanceMemorySize() const
{
	return sizeof(FBTLoopDecoratorMemory);
}

void UBTDecorator_Loop::InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const
{
	InitializeNodeMemory<FBTLoopDecoratorMemory>(NodeMemory, InitType);
}

void UBTDecorator_Loop::CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const
{
	CleanupNodeMemory<FBTLoopDecoratorMemory>(NodeMemory, CleanupType);
}

#if WITH_EDITOR

FName UBTDecorator_Loop::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Decorator.Loop.Icon");
}

#endif	// WITH_EDITOR

