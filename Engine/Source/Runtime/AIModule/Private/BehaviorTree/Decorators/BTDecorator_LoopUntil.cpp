// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Decorators/BTDecorator_LoopUntil.h"
#include "GameFramework/Actor.h"
#include "VisualLogger/VisualLogger.h"
#include "BehaviorTree/BTCompositeNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTDecorator_LoopUntil)

UBTDecorator_LoopUntil::UBTDecorator_LoopUntil(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	SetNodeName();
	INIT_DECORATOR_NODE_NOTIFY_FLAGS();

	bAllowAbortNone = false;
	bAllowAbortLowerPri = false;
	bAllowAbortChildNodes = false;
}

void UBTDecorator_LoopUntil::OnNodeDeactivation(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type NodeResult)
{
	FBTLoopUntilDecoratorMemory* DecoratorMemory = GetNodeMemory<FBTLoopUntilDecoratorMemory>(SearchData);

	// protect from infinite loop within single search
	if (DecoratorMemory->SearchId != SearchData.SearchId)
	{
		DecoratorMemory->SearchId = SearchData.SearchId;

		// We only check aborted here since we can never be in progressed when deactivating
		if (NodeResult != EBTNodeResult::Aborted)
		{
			const EBTNodeResult::Type ResolvedRequiredResult = static_cast<EBTNodeResult::Type>(RequiredResult.GetValue(SearchData.OwnerComp));
			const bool bShouldLoop = NodeResult != ResolvedRequiredResult;
			UE_VLOG(SearchData.OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT("Loop until %s: Node Result is %s -> %s"),
				ResolvedRequiredResult == EBTNodeResult::Succeeded ? TEXT("Success") : TEXT("Failure"),
				NodeResult == EBTNodeResult::Succeeded ? TEXT("Succeeded") : TEXT("Failed"),
				bShouldLoop ? TEXT("Run Again!") : TEXT("Break"));

			if (bShouldLoop)
			{
				GetParentNode()->SetChildOverride(SearchData, GetChildIndex());
			}
		}
	}
}

uint16 UBTDecorator_LoopUntil::GetInstanceMemorySize() const
{
	return sizeof(FBTLoopUntilDecoratorMemory);
}

void UBTDecorator_LoopUntil::InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const
{
	InitializeNodeMemory<FBTLoopUntilDecoratorMemory>(NodeMemory, InitType);
}

void UBTDecorator_LoopUntil::CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const
{
	CleanupNodeMemory<FBTLoopUntilDecoratorMemory>(NodeMemory, CleanupType);
}

#if WITH_EDITOR

FString UBTDecorator_LoopUntil::GetErrorMessage() const
{
	EBTNodeResult::Type DefaultResult = static_cast<EBTNodeResult::Type>(RequiredResult.GetValue(static_cast<UBlackboardComponent*>(nullptr)));
	if (DefaultResult == EBTNodeResult::Aborted)
	{
		return TEXT("Can't use 'Aborted' as Required Result since the node won't be allowed to loop in that case");
	}
	else if (DefaultResult == EBTNodeResult::InProgress)
	{
		return TEXT("Can't use 'InProgress' as Required Result since the node can never be in that state when evaluating the condition to loop");
	}

	return Super::GetErrorMessage();
}

FName UBTDecorator_LoopUntil::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Decorator.Loop.Icon");
}

void UBTDecorator_LoopUntil::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == TEXT("DefaultValue"))
	{
		SetNodeName();
	}
	else if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == TEXT("Key"))
	{
		SetNodeName();
	}
}

#endif	// WITH_EDITOR

void UBTDecorator_LoopUntil::SetNodeName()
{
	EBTNodeResult::Type DefaultResult = static_cast<EBTNodeResult::Type>(RequiredResult.GetValue(static_cast<UBlackboardComponent*>(nullptr)));
	if (DefaultResult == EBTNodeResult::Succeeded)
	{
		NodeName = TEXT("Loop Until Success");
	}
	else if (DefaultResult == EBTNodeResult::Failed)
	{
		NodeName = TEXT("Loop Until Failure");
	}
	else
	{
		NodeName = TEXT("Loop Until");
	}
}
