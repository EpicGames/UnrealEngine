// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Decorators/BTDecorator_ReachedMoveGoal.h"
#include "AIController.h"
#include "Navigation/PathFollowingComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTDecorator_ReachedMoveGoal)

UBTDecorator_ReachedMoveGoal::UBTDecorator_ReachedMoveGoal(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Reached move goal";

	// can't abort, it's not observing anything
	bAllowAbortLowerPri = false;
	bAllowAbortNone = false;
	bAllowAbortChildNodes = false;
	FlowAbortMode = EBTFlowAbortMode::None;
}

bool UBTDecorator_ReachedMoveGoal::CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const 
{
	AAIController* AIOwner = OwnerComp.GetAIOwner();
	const bool bReachedGoal = AIOwner && AIOwner->GetPathFollowingComponent() && AIOwner->GetPathFollowingComponent()->DidMoveReachGoal();
	return bReachedGoal;
}

#if WITH_EDITOR

FName UBTDecorator_ReachedMoveGoal::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Decorator.ReachedMoveGoal.Icon");
}

#endif	// WITH_EDITOR

