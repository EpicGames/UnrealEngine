// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Decorators/BTDecorator_SetTagCooldown.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTDecorator_SetTagCooldown)

UBTDecorator_SetTagCooldown::UBTDecorator_SetTagCooldown(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Set Tag Cooldown";
	INIT_DECORATOR_NODE_NOTIFY_FLAGS();
	CooldownDuration = 5.0f;
	bAddToExistingDuration = false;

	bAllowAbortNone = false;
	bAllowAbortLowerPri = false;
	bAllowAbortChildNodes = false;
	FlowAbortMode = EBTFlowAbortMode::None;
}

void UBTDecorator_SetTagCooldown::OnNodeDeactivation(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type NodeResult)
{
	SearchData.OwnerComp.AddCooldownTagDuration(CooldownTag, CooldownDuration.GetValue(SearchData.OwnerComp), bAddToExistingDuration.GetValue(SearchData.OwnerComp));
}

FString UBTDecorator_SetTagCooldown::GetStaticDescription() const
{
	// basic info: result after time
	return FString::Printf(TEXT("%s: set to %s s after execution"), *CooldownTag.ToString(), *CooldownDuration.ToString());
}

#if WITH_EDITOR

FName UBTDecorator_SetTagCooldown::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Decorator.Cooldown.Icon");
}

#endif	// WITH_EDITOR6

