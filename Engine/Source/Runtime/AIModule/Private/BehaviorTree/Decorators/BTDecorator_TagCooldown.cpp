// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Decorators/BTDecorator_TagCooldown.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTDecorator_TagCooldown)

UBTDecorator_TagCooldown::UBTDecorator_TagCooldown(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Tag Cooldown";
	INIT_DECORATOR_NODE_NOTIFY_FLAGS();
	CooldownDuration = 5.0f;
	bAddToExistingDuration = false;
	bActivatesCooldown = true;
	
	// aborting child nodes doesn't makes sense, cooldown starts after leaving this branch
	bAllowAbortChildNodes = false;
}

void UBTDecorator_TagCooldown::PostLoad()
{
	Super::PostLoad();
	bNotifyTick = (FlowAbortMode != EBTFlowAbortMode::None);
}

bool UBTDecorator_TagCooldown::HasCooldownFinished(const UBehaviorTreeComponent& OwnerComp) const
{
	const double TagCooldownEndTime = OwnerComp.GetTagCooldownEndTime(CooldownTag);

	if (TagCooldownEndTime == 0.)
	{
		// special case, we don't have an end time yet for this cooldown tag
		return true;
	}

	return (OwnerComp.GetWorld()->GetTimeSeconds() >= TagCooldownEndTime);
}

bool UBTDecorator_TagCooldown::CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	return HasCooldownFinished(OwnerComp);
}

void UBTDecorator_TagCooldown::InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const
{
	FBTTagCooldownDecoratorMemory* DecoratorMemory = InitializeNodeMemory<FBTTagCooldownDecoratorMemory>(NodeMemory, InitType);
	DecoratorMemory->bRequestedRestart = false;
}

void UBTDecorator_TagCooldown::CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const
{
	CleanupNodeMemory<FBTTagCooldownDecoratorMemory>(NodeMemory, CleanupType);
}

void UBTDecorator_TagCooldown::OnNodeDeactivation(FBehaviorTreeSearchData& SearchData, EBTNodeResult::Type NodeResult)
{
	FBTTagCooldownDecoratorMemory* DecoratorMemory = GetNodeMemory<FBTTagCooldownDecoratorMemory>(SearchData);
	DecoratorMemory->bRequestedRestart = false;

	if (bActivatesCooldown.GetValue(SearchData.OwnerComp))
	{
		SearchData.OwnerComp.AddCooldownTagDuration(CooldownTag, CooldownDuration.GetValue(SearchData.OwnerComp), bAddToExistingDuration.GetValue(SearchData.OwnerComp));
	}
}

void UBTDecorator_TagCooldown::TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds)
{
	FBTTagCooldownDecoratorMemory* DecoratorMemory = CastInstanceNodeMemory<FBTTagCooldownDecoratorMemory>(NodeMemory);
	if (!DecoratorMemory->bRequestedRestart)
	{
		if (HasCooldownFinished(OwnerComp))
		{
			DecoratorMemory->bRequestedRestart = true;
			OwnerComp.RequestExecution(this);
		}
	}
}

FString UBTDecorator_TagCooldown::GetStaticDescription() const
{
	// basic info: result after time
	return FString::Printf(TEXT("%s %s: lock for %s s after execution and return %s"), *Super::GetStaticDescription(), *CooldownTag.ToString(), *CooldownDuration.ToString(), *UBehaviorTreeTypes::DescribeNodeResult(EBTNodeResult::Failed));
}

void UBTDecorator_TagCooldown::DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const
{
	Super::DescribeRuntimeValues(OwnerComp, NodeMemory, Verbosity, Values);
	
	const double TagCooldownEndTime = OwnerComp.GetTagCooldownEndTime(CooldownTag);

	// if the tag cooldown end time is 0.f then it hasn't been set yet.
	if (TagCooldownEndTime > 0.)
	{
		const double TimePassed = (OwnerComp.GetWorld()->GetTimeSeconds() - TagCooldownEndTime);

		if (TimePassed < CooldownDuration.GetValue(OwnerComp))
		{
			Values.Add(FString::Printf(TEXT("%s in %ss"),
				(FlowAbortMode == EBTFlowAbortMode::None) ? TEXT("unlock") : TEXT("restart"),
				*FString::SanitizeFloat(CooldownDuration.GetValue(OwnerComp) - TimePassed)));
		}
	}
}

uint16 UBTDecorator_TagCooldown::GetInstanceMemorySize() const
{
	return sizeof(FBTTagCooldownDecoratorMemory);
}

#if WITH_EDITOR

FName UBTDecorator_TagCooldown::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Decorator.Cooldown.Icon");
}

#endif	// WITH_EDITOR

