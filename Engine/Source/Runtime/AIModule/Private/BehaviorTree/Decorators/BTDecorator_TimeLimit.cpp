// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Decorators/BTDecorator_TimeLimit.h"
#include "BehaviorTree/BehaviorTree.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTDecorator_TimeLimit)

struct FBTimeLimitMemory
{
	bool bElapsed = false;
};

UBTDecorator_TimeLimit::UBTDecorator_TimeLimit(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NodeName = "TimeLimit";
	TimeLimit = 5.0f;
	INIT_DECORATOR_NODE_NOTIFY_FLAGS();
	bTickIntervals = true;

	// time limit always abort current branch
	bAllowAbortLowerPri = false;
	bAllowAbortNone = false;
	FlowAbortMode = EBTFlowAbortMode::Self;
}

uint16 UBTDecorator_TimeLimit::GetInstanceMemorySize() const
{
	return sizeof(FBTimeLimitMemory);
}

void UBTDecorator_TimeLimit::InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, const EBTMemoryInit::Type InitType) const
{
	InitializeNodeMemory<FBTimeLimitMemory>(NodeMemory, InitType);
}

void UBTDecorator_TimeLimit::CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, const EBTMemoryClear::Type CleanupType) const
{
	CleanupNodeMemory<FBTimeLimitMemory>(NodeMemory, CleanupType);
}

void UBTDecorator_TimeLimit::OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	Super::OnBecomeRelevant(OwnerComp, NodeMemory);

	SetNextTickTime(NodeMemory, TimeLimit.GetValue(OwnerComp));
}

void UBTDecorator_TimeLimit::OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	Super::OnCeaseRelevant(OwnerComp, NodeMemory);

	reinterpret_cast<FBTimeLimitMemory*>(NodeMemory)->bElapsed = false;
}

bool UBTDecorator_TimeLimit::CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	return !reinterpret_cast<FBTimeLimitMemory*>(NodeMemory)->bElapsed;
}

void UBTDecorator_TimeLimit::TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, const float DeltaSeconds)
{
	FBTimeLimitMemory* TimeLimitMemory = CastInstanceNodeMemory<FBTimeLimitMemory>(NodeMemory);
	const float Limit = TimeLimit.GetValue(OwnerComp);
	if (DeltaSeconds < Limit && !FMath::IsNearlyEqual(DeltaSeconds, Limit, UE_KINDA_SMALL_NUMBER))
	{
		const FBTAuxiliaryMemory* AuxMemory = GetSpecialNodeMemory<FBTAuxiliaryMemory>(NodeMemory);
		ensureMsgf(false,
			TEXT("Using SetNextTickTime in OnBecomeRelevant should guarantee that we are only getting ticked when the time limit is finished. DT=%f, TimeLimit=%f Node:%d, BT: %s, NextTickRemainingTime=%f, AccumulatedDeltaTime=%f, bElapsed=%s"),
			DeltaSeconds,
			Limit,
			static_cast<int32>(GetExecutionIndex()),
			*GetNameSafe(GetTreeAsset()),
			AuxMemory->NextTickRemainingTime,
			AuxMemory->AccumulatedDeltaTime,
			TimeLimitMemory->bElapsed ? TEXT("True") : TEXT("False"));
	}
	
	// Mark this decorator instance as Elapsed for calls to CalculateRawConditionValue
	TimeLimitMemory->bElapsed = true;

	// Set our next tick time to large value so we don't get ticked again in case the decorator
	// is still active after requesting execution (e.g. latent abort)
	SetNextTickTime(NodeMemory, FLT_MAX);

	OwnerComp.RequestExecution(this);
}

FString UBTDecorator_TimeLimit::GetStaticDescription() const
{
	// basic info: result after time
	return FString::Printf(TEXT("%s: %s after %s s"), *Super::GetStaticDescription(),
		*UBehaviorTreeTypes::DescribeNodeResult(EBTNodeResult::Failed), *TimeLimit.ToString());
}

void UBTDecorator_TimeLimit::DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const
{
	Super::DescribeRuntimeValues(OwnerComp, NodeMemory, Verbosity, Values);

	const FBTAuxiliaryMemory* DecoratorMemory = GetSpecialNodeMemory<FBTAuxiliaryMemory>(NodeMemory);
	if (OwnerComp.GetWorld())
	{
		const float TimeLeft = DecoratorMemory->NextTickRemainingTime > 0.f ? DecoratorMemory->NextTickRemainingTime : 0.f;
		Values.Add(FString::Printf(TEXT("%s in %ss"),
			*UBehaviorTreeTypes::DescribeNodeResult(EBTNodeResult::Failed),
			*FString::SanitizeFloat(TimeLeft)));
	}
}

#if WITH_EDITOR

FName UBTDecorator_TimeLimit::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Decorator.TimeLimit.Icon");
}

#endif // WITH_EDITOR
