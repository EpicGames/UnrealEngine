// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Services/BTService_BlackboardBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTService_BlackboardBase)

UBTService_BlackboardBase::UBTService_BlackboardBase(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "BlackboardBase";

	// empty KeySelector = allow everything
}

void UBTService_BlackboardBase::InitializeFromAsset(UBehaviorTree& Asset)
{
	Super::InitializeFromAsset(Asset);

	if (const UBlackboardData* BBAsset = GetBlackboardAsset())
	{
		BlackboardKey.ResolveSelectedKey(*BBAsset);
	}
	else
	{
		BlackboardKey.InvalidateResolvedKey();
	}
}

#if WITH_EDITOR
FString UBTService_BlackboardBase::GetErrorMessage() const
{
	if (GetBlackboardAsset() == nullptr)
	{
		return UE::BehaviorTree::Messages::BlackboardNotSet.ToString();
	}
	return Super::GetErrorMessage();
}
#endif // WITH_EDITOR
