// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Services/BTService_BlueprintBase.h"
#include "AIController.h"
#include "BehaviorTree/BTFunctionLibrary.h"
#include "BlueprintNodeHelpers.h"
#include "BehaviorTree/BehaviorTree.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTService_BlueprintBase)

UBTService_BlueprintBase::UBTService_BlueprintBase(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	UClass* StopAtClass = UBTService_BlueprintBase::StaticClass();
	ReceiveTickImplementations = FBTNodeBPImplementationHelper::CheckEventImplementationVersion(TEXT("ReceiveTick"), TEXT("ReceiveTickAI"), *this, *StopAtClass);
	ReceiveActivationImplementations = FBTNodeBPImplementationHelper::CheckEventImplementationVersion(TEXT("ReceiveActivation"), TEXT("ReceiveActivationAI"), *this, *StopAtClass);
	ReceiveDeactivationImplementations = FBTNodeBPImplementationHelper::CheckEventImplementationVersion(TEXT("ReceiveDeactivation"), TEXT("ReceiveDeactivationAI"), *this, *StopAtClass);
	ReceiveSearchStartImplementations = FBTNodeBPImplementationHelper::CheckEventImplementationVersion(TEXT("ReceiveSearchStart"), TEXT("ReceiveSearchStartAI"), *this, *StopAtClass);

	bNotifyBecomeRelevant = ReceiveActivationImplementations != 0;
	bNotifyCeaseRelevant = ReceiveDeactivationImplementations != 0;
	bNotifyOnSearch = ReceiveSearchStartImplementations != 0;
	bNotifyTick = ReceiveTickImplementations != 0;
	bShowPropertyDetails = true;

	// all blueprint based nodes must create instances
	bCreateNodeInstance = true;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		BlueprintNodeHelpers::CollectPropertyData(this, StopAtClass, PropertyData);
	}
}

void UBTService_BlueprintBase::InitializeFromAsset(UBehaviorTree& Asset)
{
	Super::InitializeFromAsset(Asset);
	if (Asset.BlackboardAsset)
	{
		BlueprintNodeHelpers::ResolveBlackboardSelectors(*this, *StaticClass(), *Asset.BlackboardAsset);
	}
}

void UBTService_BlueprintBase::SetOwner(AActor* InActorOwner)
{
	ActorOwner = InActorOwner;
	AIOwner = Cast<AAIController>(InActorOwner);
}

void UBTService_BlueprintBase::OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	Super::OnBecomeRelevant(OwnerComp, NodeMemory);

	// check flag, it could be used because user wants tick
	if (AIOwner != nullptr && (ReceiveActivationImplementations & FBTNodeBPImplementationHelper::AISpecific))
	{
		ReceiveActivationAI(AIOwner, AIOwner->GetPawn());
	}
	else if (ReceiveActivationImplementations & FBTNodeBPImplementationHelper::Generic)
	{
		ReceiveActivation(ActorOwner);
	}
}

void UBTService_BlueprintBase::OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	Super::OnCeaseRelevant(OwnerComp, NodeMemory);

	if (!OwnerComp.HasAnyFlags(RF_BeginDestroyed) && OwnerComp.GetOwner())
	{
		// force dropping all pending latent actions associated with this blueprint
		// we can't have those resuming activity when node is/was aborted
		BlueprintNodeHelpers::AbortLatentActions(OwnerComp, *this);

		if (AIOwner != nullptr && (ReceiveDeactivationImplementations & FBTNodeBPImplementationHelper::AISpecific))
		{
			ReceiveDeactivationAI(AIOwner, AIOwner->GetPawn());
		}
		else if (ReceiveDeactivationImplementations & FBTNodeBPImplementationHelper::Generic)
		{
			ReceiveDeactivation(ActorOwner);
		}
	}
	else
	{
		UE_LOGF(LogBehaviorTree, Warning,
			"OnCeaseRelevant called on Blueprint service %ls with invalid owner.  OwnerComponent: %ls, OwnerComponent Owner: %ls.  %ls"
			, *GetNameSafe(this), *OwnerComp.GetName(), *GetNameSafe(OwnerComp.GetOwner())
			, OwnerComp.HasAnyFlags(RF_BeginDestroyed) ? TEXT("OwnerComponent has BeginDestroyed flag") : TEXT("")
		);
	}
}

void UBTService_BlueprintBase::OnSearchStart(FBehaviorTreeSearchData& SearchData)
{
	Super::OnSearchStart(SearchData);

	if (ReceiveSearchStartImplementations != 0)
	{
		if (AIOwner != nullptr && (ReceiveSearchStartImplementations & FBTNodeBPImplementationHelper::AISpecific))
		{
			ReceiveSearchStartAI(AIOwner, AIOwner->GetPawn());
		}
		else if (ReceiveSearchStartImplementations & FBTNodeBPImplementationHelper::Generic)
		{
			ReceiveSearchStart(ActorOwner);
		}
	}
}

void UBTService_BlueprintBase::TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds)
{
	Super::TickNode(OwnerComp, NodeMemory, DeltaSeconds);

	if (AIOwner != nullptr && (ReceiveTickImplementations & FBTNodeBPImplementationHelper::AISpecific))
	{
		ReceiveTickAI(AIOwner, AIOwner->GetPawn(), DeltaSeconds);
	}
	else if (ReceiveTickImplementations & FBTNodeBPImplementationHelper::Generic)
	{
		ReceiveTick(ActorOwner, DeltaSeconds);
	}
}

bool UBTService_BlueprintBase::IsServiceActive() const
{
	UBehaviorTreeComponent* OwnerComp = Cast<UBehaviorTreeComponent>(GetOuter());
	const bool bIsActive = OwnerComp->IsAuxNodeActive(this);
	return bIsActive;
}

FString UBTService_BlueprintBase::GetStaticServiceDescription() const
{
	FString ReturnDesc
#if WITH_EDITORONLY_DATA
		= CustomDescription
#endif // WITH_EDITORONLY_DATA
		;

	UBTService_BlueprintBase* CDO = (UBTService_BlueprintBase*)(GetClass()->GetDefaultObject());
	if (CDO)
	{
		if (bShowEventDetails)
		{
			ReturnDesc += FString::Printf(TEXT("%s, %s, %s, %s\n"),
				ReceiveTickImplementations != 0 ? *GetStaticTickIntervalDescription() : TEXT("No tick"),
				ReceiveActivationImplementations != 0 ? TEXT("Activation") : TEXT("No Activation"),
				ReceiveDeactivationImplementations != 0 ? TEXT("Deactivation") : TEXT("No Deactivation"),
				ReceiveSearchStartImplementations != 0 ? TEXT("Search Start") : TEXT("No Search Start"));
		}
		else
		{
			// Only show the description text ("tick every 0.40s..0.60s") if we actually have a Tick implementation
			if (ReceiveTickImplementations != 0)
			{
				ReturnDesc += Super::GetStaticServiceDescription();
			}
			ReturnDesc += TEXT('\n');
		}
						
		if (bShowPropertyDetails)
		{
			UClass* StopAtClass = UBTService_BlueprintBase::StaticClass();
			FString PropertyDesc = BlueprintNodeHelpers::CollectPropertyDescription(this, StopAtClass, CDO->PropertyData);
			if (PropertyDesc.Len())
			{
				ReturnDesc += TEXT("\n");
				ReturnDesc += PropertyDesc;
			}
		}
	}

	return ReturnDesc;
}

void UBTService_BlueprintBase::DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const
{
	UBTService_BlueprintBase* CDO = (UBTService_BlueprintBase*)(GetClass()->GetDefaultObject());
	if (CDO && CDO->PropertyData.Num())
	{
		UClass* StopAtClass = UBTService_BlueprintBase::StaticClass();
		BlueprintNodeHelpers::DescribeRuntimeValues(this, CDO->PropertyData, Values);
	}
}

void UBTService_BlueprintBase::OnInstanceDestroyed(UBehaviorTreeComponent& OwnerComp)
{
	// force dropping all pending latent actions associated with this blueprint
	BlueprintNodeHelpers::AbortLatentActions(OwnerComp, *this);
}

#if WITH_EDITOR

bool UBTService_BlueprintBase::UsesBlueprint() const
{
	return true;
}

#endif // WITH_EDITOR

