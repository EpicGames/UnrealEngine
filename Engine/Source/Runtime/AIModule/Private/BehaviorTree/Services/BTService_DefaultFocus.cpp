// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Services/BTService_DefaultFocus.h"
#include "GameFramework/Actor.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Object.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Vector.h"
#include "AIController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTService_DefaultFocus)

UBTService_DefaultFocus::UBTService_DefaultFocus(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer)
{
	NodeName = "Set default focus";

	bTickIntervals = false;
	INIT_SERVICE_NODE_NOTIFY_FLAGS();

	FocusPriority = EAIFocusPriority::Default;

	// accept only actors and vectors
	BlackboardKey.AddObjectFilter(this, GET_MEMBER_NAME_CHECKED(UBTService_DefaultFocus, BlackboardKey), AActor::StaticClass());
	BlackboardKey.AddVectorFilter(this, GET_MEMBER_NAME_CHECKED(UBTService_DefaultFocus, BlackboardKey));
}

void UBTService_DefaultFocus::OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	Super::OnBecomeRelevant(OwnerComp, NodeMemory);

	FBTFocusMemory* MyMemory = CastInstanceNodeMemory<FBTFocusMemory>(NodeMemory);
	check(MyMemory);
	MyMemory->Reset();

	AAIController* OwnerController = OwnerComp.GetAIOwner();
	UBlackboardComponent* MyBlackboard = OwnerComp.GetBlackboardComponent();
	
	if (OwnerController != nullptr && MyBlackboard != nullptr)
	{
		if (BlackboardKey.SelectedKeyType == UBlackboardKeyType_Object::StaticClass())
		{
			UObject* KeyValue = MyBlackboard->GetValue<UBlackboardKeyType_Object>(BlackboardKey.GetSelectedKeyID());
			AActor* TargetActor = Cast<AActor>(KeyValue);
			if (TargetActor)
			{
				OwnerController->SetFocus(TargetActor, FocusPriority);
				MyMemory->FocusActorSet = TargetActor;
				MyMemory->bActorSet = true;
			}
		}
		else
		{
			const FVector FocusLocation = MyBlackboard->GetValue<UBlackboardKeyType_Vector>(BlackboardKey.GetSelectedKeyID());
			OwnerController->SetFocalPoint(FocusLocation, FocusPriority);
			MyMemory->FocusLocationSet = FocusLocation;
		}

		const FBlackboard::FKey KeyID = BlackboardKey.GetSelectedKeyID();
		MyBlackboard->RegisterObserver(KeyID, this, FOnBlackboardChangeNotification::CreateUObject(this, &UBTService_DefaultFocus::OnBlackboardKeyValueChange));
	}
}

void UBTService_DefaultFocus::OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	Super::OnCeaseRelevant(OwnerComp, NodeMemory);

	FBTFocusMemory* MyMemory = CastInstanceNodeMemory<FBTFocusMemory>(NodeMemory);
	check(MyMemory);
	AAIController* OwnerController = OwnerComp.GetAIOwner();
	if (OwnerController != nullptr)
	{
		bool bClearFocus = false;
		if (MyMemory->bActorSet)
		{
			bClearFocus = (MyMemory->FocusActorSet == OwnerController->GetFocusActorForPriority(FocusPriority));
		}
		else
		{
			bClearFocus = (MyMemory->FocusLocationSet == OwnerController->GetFocalPointForPriority(FocusPriority));
		}

		if (bClearFocus)
		{
			OwnerController->ClearFocus(FocusPriority);
		}
	}

	UBlackboardComponent* BlackboardComp = OwnerComp.GetBlackboardComponent();
	if (BlackboardComp)
	{
		BlackboardComp->UnregisterObserversFrom(this);
	}
}

FString UBTService_DefaultFocus::GetStaticDescription() const
{	
	FString KeyDesc("invalid");
	if (BlackboardKey.SelectedKeyType == UBlackboardKeyType_Object::StaticClass() ||
		BlackboardKey.SelectedKeyType == UBlackboardKeyType_Vector::StaticClass())
	{
		KeyDesc = BlackboardKey.SelectedKeyName.ToString();
	}

	return FString::Printf(TEXT("Set default focus to %s"), *KeyDesc);
}

EBlackboardNotificationResult UBTService_DefaultFocus::OnBlackboardKeyValueChange(const UBlackboardComponent& Blackboard, FBlackboard::FKey ChangedKeyID)
{
	UBehaviorTreeComponent* OwnerComp = Cast<UBehaviorTreeComponent>(Blackboard.GetBrainComponent());
	AAIController* OwnerController = OwnerComp ? OwnerComp->GetAIOwner() : nullptr;
	if (OwnerController == nullptr)
	{
		return EBlackboardNotificationResult::ContinueObserving;
	}

	const int32 NodeInstanceIdx = OwnerComp->FindInstanceContainingNode(this);
	FBTFocusMemory* MyMemory = CastInstanceNodeMemory<FBTFocusMemory>(OwnerComp->GetNodeMemory(this, NodeInstanceIdx));
	MyMemory->Reset();
	OwnerController->ClearFocus(FocusPriority);

	if (BlackboardKey.SelectedKeyType == UBlackboardKeyType_Object::StaticClass())
	{
		UObject* KeyValue = Blackboard.GetValue<UBlackboardKeyType_Object>(ChangedKeyID);
		AActor* TargetActor = Cast<AActor>(KeyValue);
		if (TargetActor)
		{
			OwnerController->SetFocus(TargetActor, FocusPriority);
			MyMemory->FocusActorSet = TargetActor;
			MyMemory->bActorSet = true;
		}
	}
	else
	{
		const FVector FocusLocation = Blackboard.GetValue<UBlackboardKeyType_Vector>(ChangedKeyID);
		OwnerController->SetFocalPoint(FocusLocation, FocusPriority);
		MyMemory->FocusLocationSet = FocusLocation;
	}

	return EBlackboardNotificationResult::ContinueObserving;
}

void UBTService_DefaultFocus::InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const
{
	InitializeNodeMemory<FBTFocusMemory>(NodeMemory, InitType);
}

void UBTService_DefaultFocus::CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const
{
	CleanupNodeMemory<FBTFocusMemory>(NodeMemory, CleanupType);
}

#if WITH_EDITOR

FName UBTService_DefaultFocus::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Service.DefaultFocus.Icon");
}

#endif	// WITH_EDITOR

