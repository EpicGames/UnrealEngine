// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Services/BTService_RunEQS.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/Controller.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType.h"
#include "VisualLogger/VisualLogger.h"
#include "EnvironmentQuery/EnvQuery.h"
#include "EnvironmentQuery/EnvQueryManager.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "AISystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTService_RunEQS)


UBTService_RunEQS::UBTService_RunEQS(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NodeName = "Run EQS query on a regular basis";

	bNotifyBecomeRelevant = false;
	bNotifyCeaseRelevant = true;

#if WITH_EDITORONLY_DATA
	// Do not expose the option to tick on search start since the request is async and it
	// requires the node to be relevant to properly manage the request and its delegate.
	bCanTickOnSearchStartBeExposed = false;
	bCallTickOnSearchStart = false;
#endif // WITH_EDITORONLY_DATA
	
	// accept only actors and vectors
	BlackboardKey.AddObjectFilter(this, GET_MEMBER_NAME_CHECKED(UBTService_RunEQS, BlackboardKey), AActor::StaticClass());
	BlackboardKey.AddVectorFilter(this, GET_MEMBER_NAME_CHECKED(UBTService_RunEQS, BlackboardKey));

	QueryFinishedDelegate = FQueryFinishedSignature::CreateUObject(this, &UBTService_RunEQS::OnQueryFinished);

	bUpdateBBOnFail = GET_AI_CONFIG_VAR(bClearBBEntryOnBTEQSFail);
}

void UBTService_RunEQS::InitializeFromAsset(UBehaviorTree& Asset)
{
	Super::InitializeFromAsset(Asset);

	EQSRequest.InitForOwnerAndBlackboard(*this, GetBlackboardAsset());
}

void UBTService_RunEQS::TickNode(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds)
{
	AActor* QueryOwner = OwnerComp.GetOwner();
	AController* ControllerOwner = Cast<AController>(QueryOwner);
	if (ControllerOwner)
	{
		QueryOwner = ControllerOwner->GetPawn();
	}

	if (QueryOwner && EQSRequest.IsValid())
	{
		const UBlackboardComponent* BlackboardComponent = OwnerComp.GetBlackboardComponent();
		FBTEQSServiceMemory* MyMemory = CastInstanceNodeMemory<FBTEQSServiceMemory>(NodeMemory);

		// Trigger new query only if the previous one has already finished
		if (MyMemory->RequestID == INDEX_NONE)
		{
			MyMemory->RequestID = EQSRequest.Execute(*QueryOwner, BlackboardComponent, QueryFinishedDelegate);
		}
	}
	
	Super::TickNode(OwnerComp, NodeMemory, DeltaSeconds);
}

FString UBTService_RunEQS::GetStaticDescription() const
{
	return EQSRequest.bUseBBKeyForQueryTemplate ? FString::Printf(TEXT("%s: EQS query indicated by %s blackboard key\nResult Blackboard key: %s"), *Super::GetStaticDescription(), *EQSRequest.EQSQueryBlackboardKey.SelectedKeyName.ToString(), *BlackboardKey.SelectedKeyName.ToString())
		: FString::Printf(TEXT("%s: %s\nResult Blackboard key: %s"), *Super::GetStaticDescription(), *GetNameSafe(EQSRequest.QueryTemplate), *BlackboardKey.SelectedKeyName.ToString());
}

void UBTService_RunEQS::OnQueryFinished(TSharedPtr<FEnvQueryResult> Result)
{
	if (Result->IsAborted())
	{
		return;
	}

	AActor* MyOwner = Cast<AActor>(Result->Owner.Get());
	if (APawn* PawnOwner = Cast<APawn>(MyOwner))
	{
		MyOwner = PawnOwner->GetController();
	}

	UBehaviorTreeComponent* BTComp = MyOwner ? MyOwner->FindComponentByClass<UBehaviorTreeComponent>() : NULL;
	if (!BTComp)
	{
		UE_LOGF(LogBehaviorTree, Warning, "Unable to find behavior tree to notify about finished query from %ls!", *GetNameSafe(MyOwner));
		return;
	}

	FBTEQSServiceMemory* MyMemory = CastInstanceNodeMemory<FBTEQSServiceMemory>(BTComp->GetNodeMemory(this, BTComp->FindInstanceContainingNode(this)));
	if (!ensureMsgf(MyMemory && MyMemory->RequestID != INDEX_NONE, TEXT("%hs called while the BT node is not or no longer active."), __FUNCTION__))
	{
		return;
	}

	bool bSuccess = Result->IsSuccessful() && (Result->Items.Num() >= 1);
	if (bSuccess)
	{
		UBlackboardComponent* MyBlackboard = BTComp->GetBlackboardComponent();
		UEnvQueryItemType* ItemTypeCDO = Result->ItemType->GetDefaultObject<UEnvQueryItemType>();

		bSuccess = ItemTypeCDO->StoreInBlackboard(BlackboardKey, MyBlackboard, Result->RawData.GetData() + Result->Items[0].DataOffset);
		if (!bSuccess)
		{
			UE_VLOG(MyOwner, LogBehaviorTree, Warning, TEXT("Failed to store query result! item:%s key:%s"),
				*UEnvQueryTypes::GetShortTypeName(Result->ItemType).ToString(),
				*UBehaviorTreeTypes::GetShortTypeName(BlackboardKey.SelectedKeyType));
		}
	}
	else if (bUpdateBBOnFail)
	{
		UBlackboardComponent* MyBlackboard = BTComp->GetBlackboardComponent();
		check(MyBlackboard);
		MyBlackboard->ClearValue(BlackboardKey.GetSelectedKeyID());
	}

	MyMemory->RequestID = INDEX_NONE;
}

void UBTService_RunEQS::OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	FBTEQSServiceMemory* MyMemory = CastInstanceNodeMemory<FBTEQSServiceMemory>(NodeMemory);
	check(MyMemory);

	if (MyMemory->RequestID != INDEX_NONE)
	{
		UWorld* World = OwnerComp.GetWorld();
		if (World)
		{
			// make EQS abort that query
			UEnvQueryManager* EQSManager = UEnvQueryManager::GetCurrent(World);
			check(EQSManager);
			EQSManager->AbortQuery(MyMemory->RequestID);
		}

		MyMemory->RequestID = INDEX_NONE;
	}

	Super::OnCeaseRelevant(OwnerComp, NodeMemory);
}

void UBTService_RunEQS::InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const
{
	FBTEQSServiceMemory* MyMemory = InitializeNodeMemory<FBTEQSServiceMemory>(NodeMemory, InitType);
	check(MyMemory);
	MyMemory->RequestID = INDEX_NONE;
}

void UBTService_RunEQS::CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const
{
#if WITH_EDITOR
	const FBTEQSServiceMemory* MyMemory = CastInstanceNodeMemory<FBTEQSServiceMemory>(NodeMemory);
	check(MyMemory);
	ensure(MyMemory->RequestID == INDEX_NONE);
#endif // WITH_EDITOR
	CleanupNodeMemory<FBTEQSServiceMemory>(NodeMemory, CleanupType);
}

#if WITH_EDITOR
void UBTService_RunEQS::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	if (PropertyChangedEvent.MemberProperty &&
		PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UBTService_RunEQS, EQSRequest))
	{
		EQSRequest.PostEditChangeProperty(*this, PropertyChangedEvent);
	}
}

#endif // WITH_EDITOR

