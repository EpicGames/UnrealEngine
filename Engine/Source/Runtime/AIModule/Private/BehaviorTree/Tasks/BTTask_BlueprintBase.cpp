// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Tasks/BTTask_BlueprintBase.h"
#include "AIController.h"
#include "BehaviorTree/BTFunctionLibrary.h"
#include "BlueprintNodeHelpers.h"
#include "BehaviorTree/BehaviorTree.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTTask_BlueprintBase)

UBTTask_BlueprintBase::UBTTask_BlueprintBase(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	const UClass* StopAtClass = UBTTask_BlueprintBase::StaticClass();
	ReceiveTickImplementations = FBTNodeBPImplementationHelper::CheckEventImplementationVersion(TEXT("ReceiveTick"), TEXT("ReceiveTickAI"), *this, *StopAtClass);
	ReceiveExecuteImplementations = FBTNodeBPImplementationHelper::CheckEventImplementationVersion(TEXT("ReceiveExecute"), TEXT("ReceiveExecuteAI"), *this, *StopAtClass);
	ReceiveAbortImplementations = FBTNodeBPImplementationHelper::CheckEventImplementationVersion(TEXT("ReceiveAbort"), TEXT("ReceiveAbortAI"), *this, *StopAtClass);

	INIT_TASK_NODE_NOTIFY_FLAGS();
	bNotifyTick = ReceiveTickImplementations != FBTNodeBPImplementationHelper::NoImplementation;
	bShowPropertyDetails = true;
	bIsAborting = false;

	// all blueprint based nodes must create instances
	bCreateNodeInstance = true;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		BlueprintNodeHelpers::CollectPropertyData(this, StopAtClass, PropertyData);
	}
}

void UBTTask_BlueprintBase::InitializeFromAsset(UBehaviorTree& Asset)
{
	Super::InitializeFromAsset(Asset);
	if (Asset.BlackboardAsset)
	{
		BlueprintNodeHelpers::ResolveBlackboardSelectors(*this, *StaticClass(), *Asset.BlackboardAsset);
	}
}

void UBTTask_BlueprintBase::SetOwner(AActor* InActorOwner) 
{ 
	ActorOwner = InActorOwner;
	AIOwner = Cast<AAIController>(InActorOwner);
}

EBTNodeResult::Type UBTTask_BlueprintBase::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	// fail when task doesn't react to execution (start or tick)
	CurrentCallResult = (ReceiveExecuteImplementations != 0 || ReceiveTickImplementations != 0) ? EBTNodeResult::InProgress : EBTNodeResult::Failed;
	bIsAborting = false;

	if (ReceiveExecuteImplementations != FBTNodeBPImplementationHelper::NoImplementation)
	{
		bStoreFinishResult = true;

		if (AIOwner != nullptr && (ReceiveExecuteImplementations & FBTNodeBPImplementationHelper::AISpecific))
		{
			ReceiveExecuteAI(AIOwner, AIOwner->GetPawn());
		}
		else if (ReceiveExecuteImplementations & FBTNodeBPImplementationHelper::Generic)
		{
			ReceiveExecute(ActorOwner);
		}

		bStoreFinishResult = false;
	}

	return CurrentCallResult;
}

EBTNodeResult::Type UBTTask_BlueprintBase::AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	// force dropping all pending latent actions associated with this blueprint
	// we can't have those resuming activity when node is/was aborted
	BlueprintNodeHelpers::AbortLatentActions(OwnerComp, *this);

	CurrentCallResult = ReceiveAbortImplementations != 0 ? EBTNodeResult::InProgress : EBTNodeResult::Aborted;
	bIsAborting = true;

	if (ReceiveAbortImplementations != FBTNodeBPImplementationHelper::NoImplementation)
	{
		bStoreFinishResult = true;

		if (AIOwner != nullptr && (ReceiveAbortImplementations & FBTNodeBPImplementationHelper::AISpecific))
		{
			ReceiveAbortAI(AIOwner, AIOwner->GetPawn());
		}
		else if (ReceiveAbortImplementations & FBTNodeBPImplementationHelper::Generic)
		{
			ReceiveAbort(ActorOwner);
		}

		bStoreFinishResult = false;
	}

	return CurrentCallResult;
}

void UBTTask_BlueprintBase::TickTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds)
{
	if (TickInterval.Tick(DeltaSeconds))
	{
		DeltaSeconds = TickInterval.GetElapsedTimeWithFallback(DeltaSeconds);

		if (AIOwner != nullptr && (ReceiveTickImplementations & FBTNodeBPImplementationHelper::AISpecific))
		{
			ReceiveTickAI(AIOwner, AIOwner->GetPawn(), DeltaSeconds);
		}
		else if (ReceiveTickImplementations & FBTNodeBPImplementationHelper::Generic)
		{
			ReceiveTick(ActorOwner, DeltaSeconds);
		}

		TickInterval.Reset();
	}
}

void UBTTask_BlueprintBase::OnTaskFinished(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTNodeResult::Type TaskResult)
{
	Super::OnTaskFinished(OwnerComp, NodeMemory, TaskResult);

	TickInterval.Set(0); // so that we tick as soon as enabled back

	if (TaskResult != EBTNodeResult::InProgress)
	{
		BlueprintNodeHelpers::AbortLatentActions(OwnerComp, *this);
	}
}

void UBTTask_BlueprintBase::FinishExecute(bool bSuccess)
{
	UBehaviorTreeComponent* OwnerComp = Cast<UBehaviorTreeComponent>(GetOuter());
	const EBTNodeResult::Type NodeResult(bSuccess ? EBTNodeResult::Succeeded : EBTNodeResult::Failed);

	if (bStoreFinishResult)
	{
		CurrentCallResult = NodeResult;
	}
	else if (OwnerComp && !bIsAborting)
	{
		FinishLatentTask(*OwnerComp, NodeResult);
	}
}

void UBTTask_BlueprintBase::FinishAbort()
{
	UBehaviorTreeComponent* OwnerComp = Cast<UBehaviorTreeComponent>(GetOuter());

	if (bStoreFinishResult)
	{
		constexpr EBTNodeResult::Type NodeResult(EBTNodeResult::Aborted);
		CurrentCallResult = NodeResult;
	}
	else if (OwnerComp && bIsAborting)
	{
		FinishLatentAbort(*OwnerComp);
	}
}

bool UBTTask_BlueprintBase::IsTaskExecuting() const
{
	const UBehaviorTreeComponent* OwnerComp = Cast<UBehaviorTreeComponent>(GetOuter());
	const EBTTaskStatus::Type TaskStatus = OwnerComp->GetTaskStatus(this);

	return (TaskStatus == EBTTaskStatus::Active);
}

bool UBTTask_BlueprintBase::IsTaskAborting() const
{
	// use already cached data
	return bIsAborting;
}

void UBTTask_BlueprintBase::SetFinishOnMessage(FName MessageName)
{
	UBehaviorTreeComponent* OwnerComp = Cast<UBehaviorTreeComponent>(GetOuter());
	if (OwnerComp)
	{
		OwnerComp->RegisterMessageObserver(this, MessageName);
	}
}

void UBTTask_BlueprintBase::SetFinishOnMessageWithId(FName MessageName, int32 RequestID)
{
	UBehaviorTreeComponent* OwnerComp = Cast<UBehaviorTreeComponent>(GetOuter());
	if (OwnerComp)
	{
		OwnerComp->RegisterMessageObserver(this, MessageName, RequestID);
	}
}

FString UBTTask_BlueprintBase::GetStaticDescription() const
{
	FString ReturnDesc =
#if WITH_EDITORONLY_DATA
		CustomDescription.Len() ? CustomDescription :
#endif // WITH_EDITORONLY_DATA
		Super::GetStaticDescription();

	UBTTask_BlueprintBase* CDO = static_cast<UBTTask_BlueprintBase*>(GetClass()->GetDefaultObject());
	if (bShowPropertyDetails && CDO)
	{
		const UClass* StopAtClass = UBTTask_BlueprintBase::StaticClass();
		FString PropertyDesc = BlueprintNodeHelpers::CollectPropertyDescription(this, StopAtClass, CDO->PropertyData);
		if (PropertyDesc.Len())
		{
			ReturnDesc += TEXT(":\n\n");
			ReturnDesc += PropertyDesc;
		}
	}

	return ReturnDesc;
}

void UBTTask_BlueprintBase::DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const
{
	UBTTask_BlueprintBase* CDO = static_cast<UBTTask_BlueprintBase*>(GetClass()->GetDefaultObject());
	if (CDO && CDO->PropertyData.Num())
	{
		BlueprintNodeHelpers::DescribeRuntimeValues(this, CDO->PropertyData, Values);
	}
}

void UBTTask_BlueprintBase::OnInstanceDestroyed(UBehaviorTreeComponent& OwnerComp)
{
	// force dropping all pending latent actions associated with this blueprint
	BlueprintNodeHelpers::AbortLatentActions(OwnerComp, *this);
}

#if WITH_EDITOR

bool UBTTask_BlueprintBase::UsesBlueprint() const
{
	return true;
}

#endif // WITH_EDITOR

