// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Tasks/BTTask_FinishWithResult.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTTask_FinishWithResult)

UBTTask_FinishWithResult::UBTTask_FinishWithResult(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NodeName = "FinishWithResult";
	Result = EBTNodeResult::Succeeded;
}

EBTNodeResult::Type UBTTask_FinishWithResult::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	return Result.GetValue<EBTNodeResult::Type>(OwnerComp);
}

FString UBTTask_FinishWithResult::GetStaticDescription() const
{
	return FString::Printf(TEXT("%s %s"), *Super::GetStaticDescription()
		, *Result.ToString());
}
