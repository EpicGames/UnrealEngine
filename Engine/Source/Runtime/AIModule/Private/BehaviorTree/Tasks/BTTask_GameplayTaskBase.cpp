// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Tasks/BTTask_GameplayTaskBase.h"
#include "GameFramework/Actor.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTTask_GameplayTaskBase)

UBTTask_GameplayTaskBase::UBTTask_GameplayTaskBase(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "GameplayTask Base";
	INIT_TASK_NODE_NOTIFY_FLAGS();
	bWaitForGameplayTask = true;
}

EBTNodeResult::Type UBTTask_GameplayTaskBase::StartGameplayTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, UAITask& Task)
{
	FBTGameplayTaskMemory* MyMemory = CastInstanceNodeMemory<FBTGameplayTaskMemory>(NodeMemory);
	MyMemory->bObserverCanFinishTask = false;
	MyMemory->Task = &Task;

#if ENABLE_VISUAL_LOG
	const UObject* TaskOwnerOb = Cast<const UObject>(Task.GetTaskOwner());
	UE_VLOG(OwnerComp.GetOwner(), LogBehaviorTree, Log, TEXT("%s is ready to execute gameplay task: %s, finish %s"),
		*UBehaviorTreeTypes::DescribeNodeHelper(this), *Task.GetName(),
		!bWaitForGameplayTask.GetValue(OwnerComp) ? TEXT("instantly") :
		((TaskOwnerOb == this) ? TEXT("with gameplay task") : *FString::Printf(TEXT("UNKNOWN (gameplay task owner: %s)"), *GetNameSafe(TaskOwnerOb))) );
#endif

	Task.ReadyForActivation();
	MyMemory->bObserverCanFinishTask = bWaitForGameplayTask.GetValue(OwnerComp);

	return (Task.GetState() != EGameplayTaskState::Finished) ? EBTNodeResult::InProgress : DetermineGameplayTaskResult(Task);
}

EBTNodeResult::Type UBTTask_GameplayTaskBase::AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	FBTGameplayTaskMemory* MyMemory = CastInstanceNodeMemory<FBTGameplayTaskMemory>(NodeMemory);
	MyMemory->bObserverCanFinishTask = false;

	UAITask* TaskOb = MyMemory->Task.Get();
	if (TaskOb && !TaskOb->IsFinished())
	{
		if (bWaitForGameplayTask.GetValue(OwnerComp))
		{
			TaskOb->ExternalCancel();
		}
		else
		{
			// Inform the task that we're done and no longer want to receive OnGameplayTaskDeactivated callback
			TaskOb->MarkOwnerFinished();
		}
	}

	return EBTNodeResult::Aborted;
}

void UBTTask_GameplayTaskBase::OnTaskFinished(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTNodeResult::Type TaskResult)
{
	FBTGameplayTaskMemory* MyMemory = CastInstanceNodeMemory<FBTGameplayTaskMemory>(NodeMemory);
	MyMemory->Task.Reset();

	Super::OnTaskFinished(OwnerComp, NodeMemory, TaskResult);
}

EBTNodeResult::Type UBTTask_GameplayTaskBase::DetermineGameplayTaskResult(UAITask& Task) const
{
	return EBTNodeResult::Succeeded;
}

void UBTTask_GameplayTaskBase::OnGameplayTaskDeactivated(UGameplayTask& Task)
{
	UAITask* AITask = Cast<UAITask>(&Task);
	if (AITask && AITask->GetAIController() && AITask->GetState() != EGameplayTaskState::Paused)
	{
		UBehaviorTreeComponent* BehaviorComp = GetBTComponentForTask(Task);
		if (BehaviorComp)
		{
			uint8* RawMemory = BehaviorComp->GetNodeMemory(this, BehaviorComp->FindInstanceContainingNode(this));
			const FBTGameplayTaskMemory* MyMemory = CastInstanceNodeMemory<FBTGameplayTaskMemory>(RawMemory);
			if (ensureMsgf(MyMemory, TEXT("OnGameplayTaskDeactivated called after behavior tree instance was shutdown."
				" Make sure that AITask was cancelled ('ExternalCancel'), ended ('TaskOwnerEnded') or informed that owner is done ('MarkOwnerFinished')")))
			{
				if (MyMemory->bObserverCanFinishTask && (AITask == MyMemory->Task))
				{
					const EBTNodeResult::Type FinishResult = DetermineGameplayTaskResult(*AITask);
					FinishLatentTask(*BehaviorComp, FinishResult);
				}
			}
		}
	}
}

uint16 UBTTask_GameplayTaskBase::GetInstanceMemorySize() const
{
	return sizeof(FBTGameplayTaskMemory);
}

void UBTTask_GameplayTaskBase::InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const
{
	InitializeNodeMemory<FBTGameplayTaskMemory>(NodeMemory, InitType);
}

void UBTTask_GameplayTaskBase::CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const
{
	CleanupNodeMemory<FBTGameplayTaskMemory>(NodeMemory, CleanupType);
}