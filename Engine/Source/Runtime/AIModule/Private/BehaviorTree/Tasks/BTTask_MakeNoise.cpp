// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Tasks/BTTask_MakeNoise.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/Controller.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTTask_MakeNoise)

UBTTask_MakeNoise::UBTTask_MakeNoise(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer)
	, Loudnes(1.0f)
{
	NodeName = "MakeNoise";
}

EBTNodeResult::Type UBTTask_MakeNoise::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	const AController* MyController = Cast<AController>(OwnerComp.GetOwner());
	APawn* MyPawn = MyController ? MyController->GetPawn().Get() : NULL;

	if (MyPawn)
	{
		MyPawn->MakeNoise(Loudnes.GetValue(OwnerComp), MyPawn);
		return EBTNodeResult::Succeeded;
	}

	return EBTNodeResult::Failed;
}
	
#if WITH_EDITOR

FName UBTTask_MakeNoise::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Task.MakeNoise.Icon");
}

#endif	// WITH_EDITOR

