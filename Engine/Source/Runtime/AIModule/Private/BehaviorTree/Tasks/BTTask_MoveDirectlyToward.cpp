// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Tasks/BTTask_MoveDirectlyToward.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTTask_MoveDirectlyToward)

UBTTask_MoveDirectlyToward::UBTTask_MoveDirectlyToward(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "MoveDirectlyToward";
	bUsePathfinding = false;
}

#if WITH_EDITOR

FName UBTTask_MoveDirectlyToward::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Task.MoveDirectlyToward.Icon");
}

#endif	// WITH_EDITOR

