// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Tasks/BTTask_MoveTo.h"
#include "GameFramework/Actor.h"
#include "AISystem.h"
#include "Navigation/PathFollowingComponent.h"
#include "NavFilters/NavigationQueryFilter.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Object.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Vector.h"
#include "VisualLogger/VisualLogger.h"
#include "AIController.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "Tasks/AITask_MoveTo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTTask_MoveTo)

UBTTask_MoveTo::UBTTask_MoveTo(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Move To";
	INIT_TASK_NODE_NOTIFY_FLAGS();

	const float AcceptanceRadius = GET_AI_CONFIG_VAR(AcceptanceRadius);
	AcceptableRadius = AcceptanceRadius;
	ObservedBlackboardValueTolerance = AcceptanceRadius * 0.95f;
	bReachTestIncludesGoalRadius = bReachTestIncludesAgentRadius = GET_AI_CONFIG_VAR(bFinishMoveOnGoalOverlap);
	bAllowStrafe = GET_AI_CONFIG_VAR(bAllowStrafing);
	bAllowPartialPath = GET_AI_CONFIG_VAR(bAcceptPartialPaths);
	bTrackMovingGoal = true;
	bRequireNavigableEndLocation = true;
	bProjectGoalLocation = true;
	bUsePathfinding = true;
	bStartFromPreviousPath = false;

	// accept only actors and vectors
	BlackboardKey.AddObjectFilter(this, GET_MEMBER_NAME_CHECKED(UBTTask_MoveTo, BlackboardKey), AActor::StaticClass());
	BlackboardKey.AddVectorFilter(this, GET_MEMBER_NAME_CHECKED(UBTTask_MoveTo, BlackboardKey));
}

EBTNodeResult::Type UBTTask_MoveTo::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	EBTNodeResult::Type NodeResult = EBTNodeResult::InProgress;

	FBTMoveToTaskMemory* MyMemory = CastInstanceNodeMemory<FBTMoveToTaskMemory>(NodeMemory);
	MyMemory->PreviousGoalLocation = FAISystem::InvalidLocation;
	MyMemory->MoveRequestID = FAIRequestID::InvalidRequest;

	AAIController* MyController = OwnerComp.GetAIOwner();
	if (MyController == nullptr)
	{
		UE_VLOG(OwnerComp.GetOwner(), LogBehaviorTree, Error, TEXT("UBTTask_MoveTo::ExecuteTask failed since AIController is missing."));
		NodeResult = EBTNodeResult::Failed;
	}
	else
	{
		NodeResult = PerformMoveTask(OwnerComp, NodeMemory);
	}

	if (NodeResult == EBTNodeResult::InProgress && bObserveBlackboardValue)
	{
		UBlackboardComponent* BlackboardComp = OwnerComp.GetBlackboardComponent();
		if (ensure(BlackboardComp))
		{
			if (MyMemory->BBObserverDelegateHandle.IsValid())
			{
				UE_VLOG(MyController, LogBehaviorTree, Warning, TEXT("UBTTask_MoveTo::ExecuteTask \'%s\' Old BBObserverDelegateHandle is still valid! Removing old Observer."), *GetNodeName());
				BlackboardComp->UnregisterObserver(BlackboardKey.GetSelectedKeyID(), MyMemory->BBObserverDelegateHandle);
			}
			MyMemory->BBObserverDelegateHandle = BlackboardComp->RegisterObserver(BlackboardKey.GetSelectedKeyID(), this, FOnBlackboardChangeNotification::CreateUObject(this, &UBTTask_MoveTo::OnBlackboardValueChange));
		}
	}	
	
	return NodeResult;
}

EBTNodeResult::Type UBTTask_MoveTo::PerformMoveTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	const UBlackboardComponent* MyBlackboard = OwnerComp.GetBlackboardComponent();
	FBTMoveToTaskMemory* MyMemory = CastInstanceNodeMemory<FBTMoveToTaskMemory>(NodeMemory);
	AAIController* MyController = OwnerComp.GetAIOwner();

	EBTNodeResult::Type NodeResult = EBTNodeResult::Failed;
	if (MyController && MyBlackboard)
	{
		TSubclassOf<UNavigationQueryFilter> OverrideFiler = FilterClass.GetValue(*MyBlackboard);
		FAIMoveRequest MoveReq;
		MoveReq.SetNavigationFilter(*OverrideFiler ? OverrideFiler : MyController->GetDefaultNavigationFilterClass());
		MoveReq.SetAllowPartialPath(bAllowPartialPath.GetValue(*MyBlackboard));
		MoveReq.SetAcceptanceRadius(AcceptableRadius.GetValue(*MyBlackboard));
		MoveReq.SetCanStrafe(bAllowStrafe.GetValue(*MyBlackboard));
		MoveReq.SetReachTestIncludesAgentRadius(bReachTestIncludesAgentRadius.GetValue(*MyBlackboard));
		MoveReq.SetReachTestIncludesGoalRadius(bReachTestIncludesGoalRadius.GetValue(*MyBlackboard));
		MoveReq.SetRequireNavigableEndLocation(bRequireNavigableEndLocation.GetValue(*MyBlackboard));
		MoveReq.SetProjectGoalLocation(bProjectGoalLocation.GetValue(*MyBlackboard));
		MoveReq.SetUsePathfinding(bUsePathfinding);
		MoveReq.SetStartFromPreviousPath(bStartFromPreviousPath.GetValue(*MyBlackboard));

		if (BlackboardKey.SelectedKeyType == UBlackboardKeyType_Object::StaticClass())
		{
			UObject* KeyValue = MyBlackboard->GetValue<UBlackboardKeyType_Object>(BlackboardKey.GetSelectedKeyID());
			AActor* TargetActor = Cast<AActor>(KeyValue);
			if (TargetActor)
			{
				if (bTrackMovingGoal.GetValue(*MyBlackboard))
				{
					MoveReq.SetGoalActor(TargetActor);
				}
				else
				{
					MoveReq.SetGoalLocation(TargetActor->GetActorLocation());
				}
			}
			else
			{
				UE_VLOG(MyController, LogBehaviorTree, Warning, TEXT("UBTTask_MoveTo::ExecuteTask tried to go to actor while BB %s entry was empty"), *BlackboardKey.SelectedKeyName.ToString());
			}
		}
		else if (BlackboardKey.SelectedKeyType == UBlackboardKeyType_Vector::StaticClass())
		{
			const FVector TargetLocation = MyBlackboard->GetValue<UBlackboardKeyType_Vector>(BlackboardKey.GetSelectedKeyID());
			MoveReq.SetGoalLocation(TargetLocation);

			MyMemory->PreviousGoalLocation = TargetLocation;
		}

		if (MoveReq.IsValid())
		{
			UAITask_MoveTo* MoveTask = MyMemory->Task.Get();
			const bool bReuseExistingTask = (MoveTask != nullptr);

			MoveTask = PrepareMoveTask(OwnerComp, MoveTask, MoveReq);
			if (MoveTask)
			{
				MyMemory->bObserverCanFinishTask = false;

				if (bReuseExistingTask)
				{
					if (MoveTask->IsActive())
					{
						UE_VLOG(MyController, LogBehaviorTree, Verbose, TEXT("\'%s\' reusing AITask %s"), *GetNodeName(), *MoveTask->GetName());
						MoveTask->ConditionalPerformMove();
					}
					else
					{
						UE_VLOG(MyController, LogBehaviorTree, Verbose, TEXT("\'%s\' reusing AITask %s, but task is not active - handing over move performing to task mechanics"), *GetNodeName(), *MoveTask->GetName());
					}
				}
				else
				{
					MyMemory->Task = MoveTask;
					UE_VLOG(MyController, LogBehaviorTree, Verbose, TEXT("\'%s\' task implementing move with task %s"), *GetNodeName(), *MoveTask->GetName());
					MoveTask->ReadyForActivation();
				}

				MyMemory->bObserverCanFinishTask = true;
				NodeResult = (MoveTask->GetState() != EGameplayTaskState::Finished) ? EBTNodeResult::InProgress :
					MoveTask->WasMoveSuccessful() ? EBTNodeResult::Succeeded :
					EBTNodeResult::Failed;
			}
		}
	}

	return NodeResult;
}

UAITask_MoveTo* UBTTask_MoveTo::PrepareMoveTask(UBehaviorTreeComponent& OwnerComp, UAITask_MoveTo* ExistingTask, FAIMoveRequest& MoveRequest)
{
	UAITask_MoveTo* MoveTask = ExistingTask ? ExistingTask : NewBTAITask<UAITask_MoveTo>(OwnerComp);
	if (MoveTask)
	{
		MoveTask->SetUp(MoveTask->GetAIController(), MoveRequest);
	}

	return MoveTask;
}

EBlackboardNotificationResult UBTTask_MoveTo::OnBlackboardValueChange(const UBlackboardComponent& Blackboard, FBlackboard::FKey ChangedKeyID)
{
	UBehaviorTreeComponent* BehaviorComp = Cast<UBehaviorTreeComponent>(Blackboard.GetBrainComponent());
	if (BehaviorComp == nullptr)
	{
		return EBlackboardNotificationResult::RemoveObserver;
	}

	const AAIController* MyController = BehaviorComp->GetAIOwner();
	uint8* RawMemory = BehaviorComp->GetNodeMemory(this, BehaviorComp->FindInstanceContainingNode(this));
	FBTMoveToTaskMemory* MyMemory = CastInstanceNodeMemory<FBTMoveToTaskMemory>(RawMemory);

	const EBTTaskStatus::Type TaskStatus = BehaviorComp->GetTaskStatus(this);
	if (TaskStatus != EBTTaskStatus::Active)
	{
		UE_VLOG(MyController, LogBehaviorTree, Error, TEXT("BT MoveTo \'%s\' task observing BB entry while no longer being active!"), *GetNodeName());

		// resetting BBObserverDelegateHandle without unregistering observer since 
		// returning EBlackboardNotificationResult::RemoveObserver here will take care of that for us
		MyMemory->BBObserverDelegateHandle.Reset(); //-V595

		return EBlackboardNotificationResult::RemoveObserver;
	}
	
	// this means the move has already started. 
	if (MyMemory != nullptr && BehaviorComp->GetAIOwner() != nullptr)
	{
		check(BehaviorComp->GetAIOwner()->GetPathFollowingComponent());

		bool bUpdateMove = true;
		// check if new goal is almost identical to previous one
		if (BlackboardKey.SelectedKeyType == UBlackboardKeyType_Vector::StaticClass())
		{
			const FVector TargetLocation = Blackboard.GetValue<UBlackboardKeyType_Vector>(BlackboardKey.GetSelectedKeyID());

			bUpdateMove = (FVector::DistSquared(TargetLocation, MyMemory->PreviousGoalLocation) > FMath::Square(ObservedBlackboardValueTolerance.GetValue(Blackboard)));
		}

		if (bUpdateMove)
		{
			// don't abort move if using AI tasks - it will mess things up
			if (MyMemory->MoveRequestID.IsValid())
			{
				UE_VLOG(MyController, LogBehaviorTree, Log, TEXT("Blackboard value for goal has changed, aborting current move request"));
				StopWaitingForMessages(*BehaviorComp);
				BehaviorComp->GetAIOwner()->GetPathFollowingComponent()->AbortMove(*this, FPathFollowingResultFlags::NewRequest, MyMemory->MoveRequestID, EPathFollowingVelocityMode::Keep);
			}

			const EBTNodeResult::Type NodeResult = PerformMoveTask(*BehaviorComp, RawMemory);
			if (NodeResult != EBTNodeResult::InProgress)
			{
				FinishLatentTask(*BehaviorComp, NodeResult);
			}
		}
	}

	return EBlackboardNotificationResult::ContinueObserving;
}

EBTNodeResult::Type UBTTask_MoveTo::AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	FBTMoveToTaskMemory* MyMemory = CastInstanceNodeMemory<FBTMoveToTaskMemory>(NodeMemory);
	if (MyMemory->MoveRequestID.IsValid())
	{
		AAIController* MyController = OwnerComp.GetAIOwner();
		if (MyController && MyController->GetPathFollowingComponent())
		{
			MyController->GetPathFollowingComponent()->AbortMove(*this, FPathFollowingResultFlags::OwnerFinished, MyMemory->MoveRequestID);
		}
	}
	else
	{
		MyMemory->bObserverCanFinishTask = false;
		UAITask_MoveTo* MoveTask = MyMemory->Task.Get();
		if (MoveTask)
		{
			MoveTask->ExternalCancel();
		}
	}

	return Super::AbortTask(OwnerComp, NodeMemory);
}

void UBTTask_MoveTo::OnTaskFinished(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTNodeResult::Type TaskResult)
{
	FBTMoveToTaskMemory* MyMemory = CastInstanceNodeMemory<FBTMoveToTaskMemory>(NodeMemory);
	MyMemory->Task.Reset();

	if (bObserveBlackboardValue)
	{
		UBlackboardComponent* BlackboardComp = OwnerComp.GetBlackboardComponent();
		if (ensure(BlackboardComp) && MyMemory->BBObserverDelegateHandle.IsValid())
		{
			BlackboardComp->UnregisterObserver(BlackboardKey.GetSelectedKeyID(), MyMemory->BBObserverDelegateHandle);
		}

		MyMemory->BBObserverDelegateHandle.Reset();
	}

	Super::OnTaskFinished(OwnerComp, NodeMemory, TaskResult);
}

void UBTTask_MoveTo::OnMessage(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, FName Message, int32 SenderID, bool bSuccess)
{
	// AIMessage_RepathFailed means task has failed
	bSuccess &= (Message != UBrainComponent::AIMessage_RepathFailed);
	Super::OnMessage(OwnerComp, NodeMemory, Message, SenderID, bSuccess);
}

void UBTTask_MoveTo::OnGameplayTaskDeactivated(UGameplayTask& Task)
{
	// AI move task finished
	UAITask_MoveTo* MoveTask = Cast<UAITask_MoveTo>(&Task);
	if (MoveTask && MoveTask->GetAIController() && MoveTask->GetState() != EGameplayTaskState::Paused)
	{
		UBehaviorTreeComponent* BehaviorComp = GetBTComponentForTask(Task);
		if (BehaviorComp)
		{
			uint8* RawMemory = BehaviorComp->GetNodeMemory(this, BehaviorComp->FindInstanceContainingNode(this));
			const FBTMoveToTaskMemory* MyMemory = CastInstanceNodeMemory<FBTMoveToTaskMemory>(RawMemory);

			if (MyMemory && MyMemory->bObserverCanFinishTask && (MoveTask == MyMemory->Task))
			{
				const bool bSuccess = MoveTask->WasMoveSuccessful();
				FinishLatentTask(*BehaviorComp, bSuccess ? EBTNodeResult::Succeeded : EBTNodeResult::Failed);
			}
		}
	}
}

FString UBTTask_MoveTo::GetStaticDescription() const
{
	FString KeyDesc("invalid");
	if (BlackboardKey.SelectedKeyType == UBlackboardKeyType_Object::StaticClass() ||
		BlackboardKey.SelectedKeyType == UBlackboardKeyType_Vector::StaticClass())
	{
		KeyDesc = BlackboardKey.SelectedKeyName.ToString();
	}

	return FString::Printf(TEXT("%s: %s"), *Super::GetStaticDescription(), *KeyDesc);
}

void UBTTask_MoveTo::DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const
{
	Super::DescribeRuntimeValues(OwnerComp, NodeMemory, Verbosity, Values);

	const UBlackboardComponent* BlackboardComp = OwnerComp.GetBlackboardComponent();

	if (BlackboardComp)
	{
		const FString KeyValue = BlackboardComp->DescribeKeyValue(BlackboardKey.GetSelectedKeyID(), EBlackboardDescription::OnlyValue);

		const FBTMoveToTaskMemory* MyMemory = CastInstanceNodeMemory<FBTMoveToTaskMemory>(NodeMemory);
		const bool bIsUsingTask = MyMemory->Task.IsValid();
		
		const FString ModeDesc = bIsUsingTask ? TEXT("(task)") : TEXT("");

		Values.Add(FString::Printf(TEXT("move target: %s%s"), *KeyValue, *ModeDesc));
	}
}

uint16 UBTTask_MoveTo::GetInstanceMemorySize() const
{
	return sizeof(FBTMoveToTaskMemory);
}

void UBTTask_MoveTo::InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const
{
	InitializeNodeMemory<FBTMoveToTaskMemory>(NodeMemory, InitType);
}

void UBTTask_MoveTo::CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const
{
	CleanupNodeMemory<FBTMoveToTaskMemory>(NodeMemory, CleanupType);
}


#if WITH_EDITOR

FName UBTTask_MoveTo::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Task.MoveTo.Icon");
}
#endif	// WITH_EDITOR

