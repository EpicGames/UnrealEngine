// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Tasks/BTTask_PlayAnimation.h"
#include "Engine/World.h"
#include "VisualLogger/VisualLogger.h"
#include "AIController.h"
#include "GameFramework/Character.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTTask_PlayAnimation)

//----------------------------------------------------------------------//
// UBTTask_PlayAnimation
//----------------------------------------------------------------------//
UBTTask_PlayAnimation::UBTTask_PlayAnimation(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer)
{
	NodeName = "Play Animation";
	// instantiating to be able to use Timers
	bCreateNodeInstance = true;

	bLooping = false;
	bNonBlocking = false;

	TimerDelegate = FTimerDelegate::CreateUObject(this, &UBTTask_PlayAnimation::OnAnimationTimerDone);
	PreviousAnimationMode = EAnimationMode::AnimationBlueprint;
}

EBTNodeResult::Type UBTTask_PlayAnimation::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	AAIController* const MyController = OwnerComp.GetAIOwner();
	EBTNodeResult::Type Result = EBTNodeResult::Failed;

	// reset timer handle
	TimerHandle.Invalidate();
	MyOwnerComp = &OwnerComp;

	UAnimationAsset* AnimToPlay = AnimationToPlay.GetValue<UAnimationAsset>(OwnerComp);
	if (AnimToPlay && MyController && MyController->GetPawn())
	{
		USkeletalMeshComponent* SkelMesh = nullptr;
		ACharacter* const MyCharacter = Cast<ACharacter>(MyController->GetPawn());
		if (MyCharacter)
		{
			SkelMesh = MyCharacter->GetMesh();
		}
		else
		{
			SkelMesh = MyController->GetPawn()->FindComponentByClass<USkeletalMeshComponent>();
		}

		if (SkelMesh != nullptr)
		{
			PreviousAnimationMode = SkelMesh->GetAnimationMode();
			CachedSkelMesh = SkelMesh;

			SkelMesh->PlayAnimation(AnimToPlay, bLooping.GetValue(OwnerComp));
			const float FinishDelay = AnimToPlay->GetPlayLength();

			if (bNonBlocking.GetValue(OwnerComp) == false && FinishDelay > 0)
			{
				if (bLooping.GetValue(OwnerComp) == false)
				{
					MyController->GetWorld()->GetTimerManager().SetTimer(TimerHandle, TimerDelegate, FinishDelay, /*bLoop=*/false);
				}
				Result = EBTNodeResult::InProgress;
			}
			else
			{
				UE_CVLOG(bNonBlocking.GetValue(OwnerComp) == false, MyController, LogBehaviorTree, Log, TEXT("%s> Instant success due to having a valid AnimationToPlay and Character with SkelMesh, but 0-length animation"), *GetNodeName());
				// we're done here, report success so that BT can pick next task
				Result = EBTNodeResult::Succeeded;
			}
		}
	}

	return Result;
}

EBTNodeResult::Type UBTTask_PlayAnimation::AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	AAIController* const MyController = OwnerComp.GetAIOwner();

	if (AnimationToPlay.GetValue<UAnimationAsset>(OwnerComp) && MyController && TimerHandle.IsValid())
	{
		MyController->GetWorld()->GetTimerManager().ClearTimer(TimerHandle);
	}

	TimerHandle.Invalidate();

	CleanUp(OwnerComp);

	return EBTNodeResult::Aborted;
}

FString UBTTask_PlayAnimation::GetStaticDescription() const
{
	return FString::Printf(TEXT("%s: Anim:'%s' Looping:%s Blocking:%s"), *Super::GetStaticDescription(), *AnimationToPlay.ToString()
		, *bLooping.ToString() 
		, *bNonBlocking.ToString());
}

void UBTTask_PlayAnimation::OnAnimationTimerDone()
{
	if (MyOwnerComp)
	{
		CleanUp(*MyOwnerComp);
		FinishLatentTask(*MyOwnerComp, EBTNodeResult::Succeeded);
	}
}

void UBTTask_PlayAnimation::CleanUp(UBehaviorTreeComponent& OwnerComp)
{
	if (CachedSkelMesh != nullptr && PreviousAnimationMode == EAnimationMode::AnimationBlueprint)
	{
		CachedSkelMesh->SetAnimationMode(EAnimationMode::AnimationBlueprint);
	}
}

#if WITH_EDITOR

FName UBTTask_PlayAnimation::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Task.PlaySound.Icon");
}

#endif	// WITH_EDITOR

