// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Tasks/BTTask_PlaySound.h"
#include "Kismet/GameplayStatics.h"
#include "AIController.h"
#include "Sound/SoundCue.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTTask_PlaySound)

UBTTask_PlaySound::UBTTask_PlaySound(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "PlaySound";
}

EBTNodeResult::Type UBTTask_PlaySound::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	const AAIController* MyController = OwnerComp.GetAIOwner();

	UAudioComponent* AC = NULL;
	USoundCue* Sound = SoundToPlay.GetValue<USoundCue>(OwnerComp);
	if (Sound && MyController)
	{
		if (const APawn* MyPawn = MyController->GetPawn())
		{
			AC = UGameplayStatics::SpawnSoundAttached(Sound, MyPawn->GetRootComponent());
		}
	}
	return AC ? EBTNodeResult::Succeeded : EBTNodeResult::Failed;
}

FString UBTTask_PlaySound::GetStaticDescription() const
{
	return FString::Printf(TEXT("%s: '%s'"), *Super::GetStaticDescription(), *SoundToPlay.ToString());
}

#if WITH_EDITOR

FName UBTTask_PlaySound::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Task.PlaySound.Icon");
}

#endif	// WITH_EDITOR

