// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Tasks/BTTask_RotateToFaceBBEntry.h"
#include "GameFramework/Actor.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Object.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Vector.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Rotator.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "AIController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTTask_RotateToFaceBBEntry)

UBTTask_RotateToFaceBBEntry::UBTTask_RotateToFaceBBEntry(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Precision(10.f)
{
	NodeName = "Rotate to face BB entry";
	INIT_TASK_NODE_NOTIFY_FLAGS();
	
	// accept only actors and vectors
	BlackboardKey.AddObjectFilter(this, GET_MEMBER_NAME_CHECKED(UBTTask_RotateToFaceBBEntry, BlackboardKey), AActor::StaticClass());
	BlackboardKey.AddVectorFilter(this, GET_MEMBER_NAME_CHECKED(UBTTask_RotateToFaceBBEntry, BlackboardKey));
	BlackboardKey.AddRotatorFilter(this, GET_MEMBER_NAME_CHECKED(UBTTask_RotateToFaceBBEntry, BlackboardKey));
}

namespace
{
	FORCEINLINE_DEBUGGABLE FVector::FReal CalculateAngleDifferenceDot(const FVector& VectorA, const FVector& VectorB)
	{
		return (VectorA.IsNearlyZero() || VectorB.IsNearlyZero())
            ? 1.f
            : VectorA.CosineAngle2D(VectorB);
	}
}

EBTNodeResult::Type UBTTask_RotateToFaceBBEntry::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	AAIController* AIController = OwnerComp.GetAIOwner();

	if (AIController == NULL || AIController->GetPawn() == NULL)
	{
		return EBTNodeResult::Failed;
	}

	FBTFocusMemory* MyMemory = (FBTFocusMemory*)NodeMemory;
	check(MyMemory);
	MyMemory->Reset();
	const float PrecisionDot = GetPrecisionDot(OwnerComp);
	EBTNodeResult::Type Result = EBTNodeResult::Failed;

	APawn* Pawn = AIController->GetPawn();
	const FVector PawnLocation = Pawn->GetActorLocation();
	const UBlackboardComponent* MyBlackboard = OwnerComp.GetBlackboardComponent();

	if (BlackboardKey.SelectedKeyType == UBlackboardKeyType_Object::StaticClass())
	{
		UObject* KeyValue = MyBlackboard->GetValue<UBlackboardKeyType_Object>(BlackboardKey.GetSelectedKeyID());
		AActor* ActorValue = Cast<AActor>(KeyValue);

		if (ActorValue != NULL)
		{
			const FVector::FReal AngleDifference = CalculateAngleDifferenceDot(Pawn->GetActorForwardVector()
				, (ActorValue->GetActorLocation() - PawnLocation));
			
			if (AngleDifference >= PrecisionDot)
			{
				Result = EBTNodeResult::Succeeded;
			}
			else
			{
				AIController->SetFocus(ActorValue, EAIFocusPriority::Gameplay);
				MyMemory->FocusActorSet = ActorValue;
				MyMemory->bActorSet = true;
				Result = EBTNodeResult::InProgress;
			}
		}
	}
	else if (BlackboardKey.SelectedKeyType == UBlackboardKeyType_Vector::StaticClass())
	{
		const FVector KeyValue = MyBlackboard->GetValue<UBlackboardKeyType_Vector>(BlackboardKey.GetSelectedKeyID());
		
		if (FAISystem::IsValidLocation(KeyValue))
		{
			const FVector::FReal AngleDifference = CalculateAngleDifferenceDot(Pawn->GetActorForwardVector()
				, (KeyValue - PawnLocation));

			if (AngleDifference >= PrecisionDot)
			{
				Result = EBTNodeResult::Succeeded;
			}
			else
			{
				AIController->SetFocalPoint(KeyValue, EAIFocusPriority::Gameplay);
				MyMemory->FocusLocationSet = KeyValue;
				Result = EBTNodeResult::InProgress;
			}
		}
	}
	else if (BlackboardKey.SelectedKeyType == UBlackboardKeyType_Rotator::StaticClass())
	{
		const FRotator KeyValue = MyBlackboard->GetValue<UBlackboardKeyType_Rotator>(BlackboardKey.GetSelectedKeyID());

		if (FAISystem::IsValidRotation(KeyValue))
		{
			const FVector DirectionVector = KeyValue.Vector();
			const FVector::FReal AngleDifference = CalculateAngleDifferenceDot(Pawn->GetActorForwardVector(), DirectionVector);

			if (AngleDifference >= PrecisionDot)
			{
				Result = EBTNodeResult::Succeeded;
			}
			else
			{
				const FVector FocalPoint = PawnLocation + DirectionVector * 10000.0f;
				// set focal somewhere far in the indicated direction
				AIController->SetFocalPoint(FocalPoint, EAIFocusPriority::Gameplay);
				MyMemory->FocusLocationSet = FocalPoint;
				Result = EBTNodeResult::InProgress;
			}
		}
	}

	return Result;
}

void UBTTask_RotateToFaceBBEntry::TickTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds)
{
	AAIController* AIController = OwnerComp.GetAIOwner();

	if (AIController == NULL || AIController->GetPawn() == NULL)
	{
		FinishLatentTask(OwnerComp, EBTNodeResult::Failed);
	}
	else
	{
		const FVector PawnDirection = AIController->GetPawn()->GetActorForwardVector();				
		const FVector FocalPoint = AIController->GetFocalPointForPriority(EAIFocusPriority::Gameplay);

		if (FocalPoint != FAISystem::InvalidLocation)
		{
			if (CalculateAngleDifferenceDot(PawnDirection, FocalPoint - AIController->GetPawn()->GetActorLocation()) >= GetPrecisionDot(OwnerComp))
			{
				CleanUp(*AIController, NodeMemory);
				FinishLatentTask(OwnerComp, EBTNodeResult::Succeeded);
			}
		}
		else
		{
			CleanUp(*AIController, NodeMemory);
			FinishLatentTask(OwnerComp, EBTNodeResult::Failed);
		}
	}
}

float UBTTask_RotateToFaceBBEntry::GetPrecisionDot(const UBehaviorTreeComponent& OwnerComp) const
{
	return FMath::Cos(FMath::DegreesToRadians(Precision.GetValue(OwnerComp)));;
}

void UBTTask_RotateToFaceBBEntry::CleanUp(AAIController& AIController, uint8* NodeMemory)
{
	FBTFocusMemory* MyMemory = (FBTFocusMemory*)NodeMemory;
	check(MyMemory);
	
	bool bClearFocus = false;
	if (MyMemory->bActorSet)
	{
		bClearFocus = (MyMemory->FocusActorSet == AIController.GetFocusActorForPriority(EAIFocusPriority::Gameplay));
	}
	else
	{
		bClearFocus = (MyMemory->FocusLocationSet == AIController.GetFocalPointForPriority(EAIFocusPriority::Gameplay));
	}
	
	if (bClearFocus)
	{
		AIController.ClearFocus(EAIFocusPriority::Gameplay);
	}
}

EBTNodeResult::Type UBTTask_RotateToFaceBBEntry::AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	AAIController* AIController = OwnerComp.GetAIOwner();

	if (AIController != NULL)
	{
		CleanUp(*AIController, NodeMemory);
	}

	return EBTNodeResult::Aborted;
}

void UBTTask_RotateToFaceBBEntry::DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const
{
	FString KeyDesc = BlackboardKey.SelectedKeyName.ToString();
	Values.Add(FString::Printf(TEXT("%s: %s"), *Super::GetStaticDescription(), *KeyDesc));

	AAIController* AIController = OwnerComp.GetAIOwner();

	if (AIController != NULL && AIController->GetPawn() != NULL)
	{
		const FVector PawnDirection = AIController->GetPawn()->GetActorForwardVector();
		const FVector FocalPoint = AIController->GetFocalPointForPriority(EAIFocusPriority::Gameplay);

		if (FocalPoint != FAISystem::InvalidLocation)
		{
			const FVector::FReal CurrentAngleRadians = CalculateAngleDifferenceDot(PawnDirection, (FocalPoint - AIController->GetPawn()->GetActorLocation()));
			Values.Add(FString::Printf(TEXT("Current angle: %.2f"), FMath::RadiansToDegrees(FMath::Acos(CurrentAngleRadians))));
		}
		else
		{
			Values.Add(TEXT("FocalPoint is an Invalid Location"));
		}
	}
	else
	{
		Values.Add(TEXT("Controller or Pawn is NULL"));
	}
}

FString UBTTask_RotateToFaceBBEntry::GetStaticDescription() const
{
	FString KeyDesc = BlackboardKey.SelectedKeyName.ToString();
	return FString::Printf(TEXT("%s: %s"), *Super::GetStaticDescription(), *KeyDesc);
}

void UBTTask_RotateToFaceBBEntry::InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const
{
	InitializeNodeMemory<FBTFocusMemory>(NodeMemory, InitType);
}

void UBTTask_RotateToFaceBBEntry::CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const
{
	CleanupNodeMemory<FBTFocusMemory>(NodeMemory, CleanupType);
}
