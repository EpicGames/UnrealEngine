// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Tasks/BTTask_RunBehavior.h"
#include "BehaviorTree/BlackboardData.h"
#include "VisualLogger/VisualLogger.h"
#include "AIController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTTask_RunBehavior)

UBTTask_RunBehavior::UBTTask_RunBehavior(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Run Behavior";
}

EBTNodeResult::Type UBTTask_RunBehavior::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	UE_CVLOG(BehaviorAsset == nullptr, OwnerComp.GetAIOwner(), LogBehaviorTree, Error, TEXT("\'%s\' is missing BehaviorAsset!"), *GetNodeName());

	const bool bPushed = BehaviorAsset != nullptr && OwnerComp.PushInstance(*BehaviorAsset);
	if (bPushed && OwnerComp.InstanceStack.Num() > 0)
	{
		FBehaviorTreeInstance& MyInstance = OwnerComp.InstanceStack[OwnerComp.InstanceStack.Num() - 1];
		MyInstance.DeactivationNotify.BindUObject(this, &UBTTask_RunBehavior::OnSubtreeDeactivated);
		// unbinding is not required, MyInstance will be destroyed after firing that delegate (usually by UBehaviorTreeComponent::ProcessPendingExecution) 

		return EBTNodeResult::InProgress;
	}

	return EBTNodeResult::Failed;
}

void UBTTask_RunBehavior::OnSubtreeDeactivated(UBehaviorTreeComponent& OwnerComp, EBTNodeResult::Type NodeResult)
{
	const int32 MyInstanceIdx = OwnerComp.FindInstanceContainingNode(this);
	uint8* NodeMemory = OwnerComp.GetNodeMemory(this, MyInstanceIdx);

	UE_VLOG(OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT("OnSubtreeDeactivated: %s (result: %s)"),
		*UBehaviorTreeTypes::DescribeNodeHelper(this), *UBehaviorTreeTypes::DescribeNodeResult(NodeResult));

	OnTaskFinished(OwnerComp, NodeMemory, NodeResult);
}

FString UBTTask_RunBehavior::GetStaticDescription() const
{
	bool bIsBBCompatible = false;
	if (const UBlackboardData* BlackboardData = GetBlackboardAsset())
	{
		if (const UBlackboardData* OtherBlackboardData = BehaviorAsset ? BehaviorAsset->GetBlackboardAsset() : nullptr )
		{
			bIsBBCompatible = BlackboardData == OtherBlackboardData || BlackboardData->IsChildOf(*OtherBlackboardData);
		}
	}
		
	return FString::Printf(TEXT("%s: %s%s"), *Super::GetStaticDescription(), *GetNameSafe(BehaviorAsset), bIsBBCompatible ? TEXT("") : TEXT(" (Blackboard not compatible)"));
}

#if WITH_EDITOR

FName UBTTask_RunBehavior::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Task.RunBehavior.Icon");
}

const UObject* UBTTask_RunBehavior::GetAssociatedAsset(TOptional<FBehaviorTreeNodeDebugContext> DebugContext) const
{
	return BehaviorAsset;
}

#endif	// WITH_EDITOR

