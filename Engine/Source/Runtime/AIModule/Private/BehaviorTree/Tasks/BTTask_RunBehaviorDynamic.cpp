// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Tasks/BTTask_RunBehaviorDynamic.h"
#include "GameFramework/Actor.h"
#include "VisualLogger/VisualLogger.h"
#include "BehaviorTree/BehaviorTree.h"
#include "Internationalization/Regex.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTTask_RunBehaviorDynamic)

UBTTask_RunBehaviorDynamic::UBTTask_RunBehaviorDynamic(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Run Behavior Dynamic";
	bCreateNodeInstance = true;
}

EBTNodeResult::Type UBTTask_RunBehaviorDynamic::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	UE_CLOGF(BehaviorAsset == nullptr, LogBehaviorTree, Warning, "BTTask_RunBehaviorDynamic node (\"%ls\") executed with no BehaviorAsset"
		, *GetNodeName());
	UE_CLOGF(BehaviorAsset != nullptr && BehaviorAsset->RootDecorators.Num() > 0, LogBehaviorTree
		, Error, "BTTask_RunBehaviorDynamic node (\"%ls\") executed with a BehaviorAsset (\"%ls\") containing root level decorators. These decorators will be ignored by design."
		, *GetNodeName(), *BehaviorAsset->GetName());

	const bool bPushed = BehaviorAsset != nullptr && OwnerComp.PushInstance(*BehaviorAsset);
	if (bPushed && OwnerComp.InstanceStack.Num() > 0)
	{
		FBehaviorTreeInstance& MyInstance = OwnerComp.InstanceStack[OwnerComp.InstanceStack.Num() - 1];
		MyInstance.DeactivationNotify.BindUObject(this, &UBTTask_RunBehaviorDynamic::OnSubtreeDeactivated);
		// unbinding is not required, MyInstance will be destroyed after firing that delegate (usually by UBehaviorTreeComponent::ProcessPendingExecution) 

		return EBTNodeResult::InProgress;
	}

	return EBTNodeResult::Failed;
}

void UBTTask_RunBehaviorDynamic::OnInstanceCreated(UBehaviorTreeComponent& OwnerComp)
{
	Super::OnInstanceCreated(OwnerComp);
	BehaviorAsset = DefaultBehaviorAsset;
}

void UBTTask_RunBehaviorDynamic::OnSubtreeDeactivated(UBehaviorTreeComponent& OwnerComp, EBTNodeResult::Type NodeResult)
{
	const int32 MyInstanceIdx = OwnerComp.FindInstanceContainingNode(this);
	uint8* NodeMemory = OwnerComp.GetNodeMemory(this, MyInstanceIdx);

	UE_VLOG(OwnerComp.GetOwner(), LogBehaviorTree, Verbose, TEXT("OnSubtreeDeactivated: %s (result: %s)"),
		*UBehaviorTreeTypes::DescribeNodeHelper(this), *UBehaviorTreeTypes::DescribeNodeResult(NodeResult));

	OnTaskFinished(OwnerComp, NodeMemory, NodeResult);
}

FString UBTTask_RunBehaviorDynamic::GetStaticDescription() const
{
	return FString::Printf(TEXT("%s: %s"), *Super::GetStaticDescription(), *InjectionTag.ToString());
}

void UBTTask_RunBehaviorDynamic::DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const
{
	Super::DescribeRuntimeValues(OwnerComp, NodeMemory, Verbosity, Values);
	Values.Add(FString::Printf(TEXT("subtree: %s"), *GetNameSafe(BehaviorAsset)));
	if (BehaviorAsset)
	{
		Values.Add(FString::Printf(TEXT("subtree path: %s"), *BehaviorAsset->GetPathName()));
	}
}

bool UBTTask_RunBehaviorDynamic::SetBehaviorAsset(UBehaviorTree* NewBehaviorAsset)
{
	if (BehaviorAsset != NewBehaviorAsset)
	{
		BehaviorAsset = NewBehaviorAsset;
		return true;
	}
	return false;
}

#if WITH_EDITOR
FName UBTTask_RunBehaviorDynamic::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Task.RunBehavior.Icon");
}

const UObject* UBTTask_RunBehaviorDynamic::GetAssociatedAsset(TOptional<FBehaviorTreeNodeDebugContext> DebugContext) const
{
	if (DebugContext.IsSet())
	{
		static const FRegexPattern RegexPatern(FString("[\\n\\r].*subtree path:\\s*([^\\n\\r]*)"));
		FRegexMatcher Matcher(RegexPatern, DebugContext->RuntimeDescription);
		if (Matcher.FindNext())
		{
			// Capture group 0 is the whole subtree line, capture group 1 is the path.
			const FString SubtreePath = Matcher.GetCaptureGroup(1);
			return FindObject<UBehaviorTree>(nullptr, *SubtreePath);
		}
	}
	return GetDefaultBehaviorAsset();
}

UBehaviorTree* UBTTask_RunBehaviorDynamic::GetBehaviorAssetFromRuntimeValue(const FString& RuntimeValue) const
{
	static const FRegexPattern RegexPatern(FString("[\\n\\r].*subtree path:\\s*([^\\n\\r]*)"));
	FRegexMatcher Matcher(RegexPatern, RuntimeValue);
	if (Matcher.FindNext())
	{
		// Capture group 0 is the whole subtree line, capture group 1 is the path.
		const FString SubtreePath = Matcher.GetCaptureGroup(1);
		return FindObject<UBehaviorTree>(nullptr, *SubtreePath);
	}
	return nullptr;
}

#endif	// WITH_EDITOR

