// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Tasks/BTTask_RunEQSQuery.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/Controller.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType.h"
#include "VisualLogger/VisualLogger.h"
#include "EnvironmentQuery/EnvQuery.h"
#include "EnvironmentQuery/EnvQueryManager.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "AISystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTTask_RunEQSQuery)


UBTTask_RunEQSQuery::UBTTask_RunEQSQuery(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer)
{
	NodeName = "Run EQS Query";

	// filter with keys that have matching env query values, only for instances
	// CDO won't be able to access types from game dlls
	if (GIsEditor && !HasAnyFlags(RF_ClassDefaultObject))
	{
		CollectKeyFilters();
	}

	QueryFinishedDelegate = FQueryFinishedSignature::CreateUObject(this, &UBTTask_RunEQSQuery::OnQueryFinished);

	bUpdateBBOnFail = GET_AI_CONFIG_VAR(bClearBBEntryOnBTEQSFail);
}

void UBTTask_RunEQSQuery::InitializeFromAsset(UBehaviorTree& Asset)
{
	Super::InitializeFromAsset(Asset);
	
	EQSRequest.InitForOwnerAndBlackboard(*this, GetBlackboardAsset());
}

EBTNodeResult::Type UBTTask_RunEQSQuery::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	AActor* QueryOwner = OwnerComp.GetOwner();
	AController* ControllerOwner = Cast<AController>(QueryOwner);
	if (ControllerOwner)
	{
		QueryOwner = ControllerOwner->GetPawn();
	}

	if (QueryOwner && EQSRequest.IsValid())
	{
		const UBlackboardComponent* BlackboardComponent = OwnerComp.GetBlackboardComponent();
		FBTEnvQueryTaskMemory* MyMemory = CastInstanceNodeMemory<FBTEnvQueryTaskMemory>(NodeMemory);

		MyMemory->RequestID = EQSRequest.Execute(*QueryOwner, BlackboardComponent, QueryFinishedDelegate);
		
		const bool bValid = (MyMemory->RequestID >= 0);
		if (bValid)
		{
			WaitForMessage(OwnerComp, UBrainComponent::AIMessage_QueryFinished, MyMemory->RequestID);
			return EBTNodeResult::InProgress;
		}
	}

	return EBTNodeResult::Failed;
}

EBTNodeResult::Type UBTTask_RunEQSQuery::AbortTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	UWorld* MyWorld = OwnerComp.GetWorld();
	UEnvQueryManager* QueryManager = UEnvQueryManager::GetCurrent(MyWorld);
	
	if (QueryManager)
	{
		FBTEnvQueryTaskMemory* MyMemory = (FBTEnvQueryTaskMemory*)NodeMemory;
		QueryManager->AbortQuery(MyMemory->RequestID);
	}

	return EBTNodeResult::Aborted;
}

FString UBTTask_RunEQSQuery::GetStaticDescription() const
{
	return EQSRequest.bUseBBKeyForQueryTemplate ? FString::Printf(TEXT("%s: EQS query indicated by %s blackboard key\nResult Blackboard key: %s"), *Super::GetStaticDescription(), *EQSRequest.EQSQueryBlackboardKey.SelectedKeyName.ToString(), *BlackboardKey.SelectedKeyName.ToString())
		: FString::Printf(TEXT("%s: %s\nResult Blackboard key: %s"), *Super::GetStaticDescription(), *GetNameSafe(EQSRequest.QueryTemplate), *BlackboardKey.SelectedKeyName.ToString());
}

void UBTTask_RunEQSQuery::DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const
{
	Super::DescribeRuntimeValues(OwnerComp, NodeMemory, Verbosity, Values);

	if (Verbosity == EBTDescriptionVerbosity::Detailed)
	{
		FBTEnvQueryTaskMemory* MyMemory = (FBTEnvQueryTaskMemory*)NodeMemory;
		Values.Add(FString::Printf(TEXT("request: %d"), MyMemory->RequestID));
	}
}

uint16 UBTTask_RunEQSQuery::GetInstanceMemorySize() const
{
	return sizeof(FBTEnvQueryTaskMemory);
}

void UBTTask_RunEQSQuery::InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const
{
	InitializeNodeMemory<FBTEnvQueryTaskMemory>(NodeMemory, InitType);
}

void UBTTask_RunEQSQuery::CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const
{
	CleanupNodeMemory<FBTEnvQueryTaskMemory>(NodeMemory, CleanupType);
}

void UBTTask_RunEQSQuery::OnQueryFinished(TSharedPtr<FEnvQueryResult> Result)
{
	if (Result->IsAborted())
	{
		return;
	}

	AActor* MyOwner = Cast<AActor>(Result->Owner.Get());
	if (APawn* PawnOwner = Cast<APawn>(MyOwner))
	{
		MyOwner = PawnOwner->GetController();
	}

	UBehaviorTreeComponent* MyComp = MyOwner ? MyOwner->FindComponentByClass<UBehaviorTreeComponent>() : NULL;
	if (!MyComp)
	{
		UE_LOGF(LogBehaviorTree, Warning, "Unable to find behavior tree to notify about finished query from %ls!", *GetNameSafe(MyOwner));
		return;
	}

	bool bSuccess = Result->IsSuccessful() && (Result->Items.Num() >= 1);
	if (bSuccess)
	{
		UBlackboardComponent* MyBlackboard = MyComp->GetBlackboardComponent();
		UEnvQueryItemType* ItemTypeCDO = Result->ItemType->GetDefaultObject<UEnvQueryItemType>();

		bSuccess = ItemTypeCDO->StoreInBlackboard(BlackboardKey, MyBlackboard, Result->RawData.GetData() + Result->Items[0].DataOffset);		
		if (!bSuccess)
		{
			UE_VLOG(MyOwner, LogBehaviorTree, Warning, TEXT("Failed to store query result! item:%s key:%s"),
				*UEnvQueryTypes::GetShortTypeName(Result->ItemType).ToString(),
				*UBehaviorTreeTypes::GetShortTypeName(BlackboardKey.SelectedKeyType));
		}
	}
	else if (bUpdateBBOnFail)
	{
		UBlackboardComponent* MyBlackboard = MyComp->GetBlackboardComponent();
		check(MyBlackboard);
		MyBlackboard->ClearValue(BlackboardKey.GetSelectedKeyID());
	}

	FAIMessage::Send(MyComp, FAIMessage(UBrainComponent::AIMessage_QueryFinished, this, Result->QueryID, bSuccess));
}

void UBTTask_RunEQSQuery::CollectKeyFilters()
{
// 	for (int32 TypeIndex = 0; TypeIndex < UEnvQueryManager::RegisteredItemTypes.Num(); TypeIndex++)
// 	{
// 		UEnvQueryItemType* ItemTypeCDO = (UEnvQueryItemType*)UEnvQueryManager::RegisteredItemTypes[TypeIndex]->GetDefaultObject();
// 		ItemTypeCDO->AddBlackboardFilters(BlackboardKey, this);
// 	}
}

#if WITH_EDITOR
void UBTTask_RunEQSQuery::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.MemberProperty &&
		PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UBTTask_RunEQSQuery, EQSRequest))
	{
		EQSRequest.PostEditChangeProperty(*this, PropertyChangedEvent);
	}
}

FName UBTTask_RunEQSQuery::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Task.RunEQSQuery.Icon");
}

#endif

