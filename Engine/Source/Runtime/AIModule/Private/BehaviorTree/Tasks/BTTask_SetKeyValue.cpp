// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Tasks/BTTask_SetKeyValue.h"

#include "BehaviorTree/BehaviorTreeComponent.h"
#include "BehaviorTree/Blackboard/BlackboardKeyAllTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTTask_SetKeyValue)

namespace FBlackboard
{
	template <typename KeyType, typename ValueType>
	EBTNodeResult::Type SetBlackboardKeyValue(UBehaviorTreeComponent& OwnerComp, const FBlackboardKeySelector& BlackboardKey, const ValueType& Value)
	{
		UBlackboardComponent* Blackboard = OwnerComp.GetBlackboardComponent();
		if (!Blackboard)
		{
			return EBTNodeResult::Failed;
		}

		if (BlackboardKey.GetSelectedKeyID() == FBlackboard::InvalidKey)
		{
			return EBTNodeResult::Failed;
		}

		Blackboard->SetValue<KeyType>(BlackboardKey.GetSelectedKeyID(), Value.GetValue(*Blackboard));
		return EBTNodeResult::Succeeded;
	}
} // namespace FBlackboard

UBTTask_SetKeyValueBool::UBTTask_SetKeyValueBool(const FObjectInitializer& ObjectInitializer /*= FObjectInitializer::Get()*/)
	: Super(ObjectInitializer)
{
	NodeName = "Set Bool Key";
	BlackboardKey.AllowNoneAsValue(false);
	BlackboardKey.AddBoolFilter(this, GET_MEMBER_NAME_CHECKED(UBTTask_SetKeyValueBool, BlackboardKey));
	INIT_TASK_NODE_NOTIFY_FLAGS();
}

UBTTask_SetKeyValueClass::UBTTask_SetKeyValueClass(const FObjectInitializer& ObjectInitializer /*= FObjectInitializer::Get()*/)
{
	NodeName = "Set Class Key";
	BlackboardKey.AllowNoneAsValue(false);
	BlackboardKey.AddClassFilter(this, GET_MEMBER_NAME_CHECKED(UBTTask_SetKeyValueClass, BlackboardKey), BaseClass);
	INIT_TASK_NODE_NOTIFY_FLAGS();
}

UBTTask_SetKeyValueEnum::UBTTask_SetKeyValueEnum(const FObjectInitializer& ObjectInitializer /*= FObjectInitializer::Get()*/)
{
	NodeName = "Set Enum Key";
	// Since there are no good default value for enum don't add a filter and allow none as a value so the key isn't bound to a random key.
	INIT_TASK_NODE_NOTIFY_FLAGS();
}

UBTTask_SetKeyValueInt32::UBTTask_SetKeyValueInt32(const FObjectInitializer& ObjectInitializer /*= FObjectInitializer::Get()*/)
{
	NodeName = "Set Int Key";
	BlackboardKey.AllowNoneAsValue(false);
	BlackboardKey.AddIntFilter(this, GET_MEMBER_NAME_CHECKED(UBTTask_SetKeyValueInt32, BlackboardKey));
	INIT_TASK_NODE_NOTIFY_FLAGS();
}

UBTTask_SetKeyValueFloat::UBTTask_SetKeyValueFloat(const FObjectInitializer& ObjectInitializer /*= FObjectInitializer::Get()*/)
{
	NodeName = "Set Float Key";
	BlackboardKey.AllowNoneAsValue(false);
	BlackboardKey.AddFloatFilter(this, GET_MEMBER_NAME_CHECKED(UBTTask_SetKeyValueFloat, BlackboardKey));
	INIT_TASK_NODE_NOTIFY_FLAGS();
}

UBTTask_SetKeyValueName::UBTTask_SetKeyValueName(const FObjectInitializer& ObjectInitializer /*= FObjectInitializer::Get()*/)
{
	NodeName = "Set Name Key";
	BlackboardKey.AllowNoneAsValue(false);
	BlackboardKey.AddNameFilter(this, GET_MEMBER_NAME_CHECKED(UBTTask_SetKeyValueName, BlackboardKey));
	INIT_TASK_NODE_NOTIFY_FLAGS();
}

UBTTask_SetKeyValueString::UBTTask_SetKeyValueString(const FObjectInitializer& ObjectInitializer /*= FObjectInitializer::Get()*/)
{
	NodeName = "Set String Key";
	BlackboardKey.AllowNoneAsValue(false);
	BlackboardKey.AddStringFilter(this, GET_MEMBER_NAME_CHECKED(UBTTask_SetKeyValueString, BlackboardKey));
	INIT_TASK_NODE_NOTIFY_FLAGS();
}

UBTTask_SetKeyValueObject::UBTTask_SetKeyValueObject(const FObjectInitializer& ObjectInitializer /*= FObjectInitializer::Get()*/)
{
	NodeName = "Set Object Key";
	BlackboardKey.AllowNoneAsValue(false);
	BlackboardKey.AddObjectFilter(this, GET_MEMBER_NAME_CHECKED(UBTTask_SetKeyValueObject, BlackboardKey), BaseClass);
	INIT_TASK_NODE_NOTIFY_FLAGS();
}

UBTTask_SetKeyValueRotator::UBTTask_SetKeyValueRotator(const FObjectInitializer& ObjectInitializer /*= FObjectInitializer::Get()*/)
{
	NodeName = "Set Rotator Key";
	BlackboardKey.AllowNoneAsValue(false);
	BlackboardKey.AddRotatorFilter(this, GET_MEMBER_NAME_CHECKED(UBTTask_SetKeyValueRotator, BlackboardKey));
	INIT_TASK_NODE_NOTIFY_FLAGS();
}

UBTTask_SetKeyValueStruct::UBTTask_SetKeyValueStruct(const FObjectInitializer& ObjectInitializer /*= FObjectInitializer::Get()*/)
{
	NodeName = "Set Struct Key";
	//Since there are no good default value for structtype don't add a filter and allow none as a value so the key isn't bound to a random key.
	INIT_TASK_NODE_NOTIFY_FLAGS();
}

UBTTask_SetKeyValueVector::UBTTask_SetKeyValueVector(const FObjectInitializer& ObjectInitializer /*= FObjectInitializer::Get()*/)
{
	NodeName = "Set Vector Key";
	BlackboardKey.AllowNoneAsValue(false);
	BlackboardKey.AddVectorFilter(this, GET_MEMBER_NAME_CHECKED(UBTTask_SetKeyValueVector, BlackboardKey));
	INIT_TASK_NODE_NOTIFY_FLAGS();
}

EBTNodeResult::Type UBTTask_SetKeyValueBool::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	return FBlackboard::SetBlackboardKeyValue<UBlackboardKeyType_Bool>(OwnerComp, BlackboardKey, Value);
}

EBTNodeResult::Type UBTTask_SetKeyValueClass::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	return FBlackboard::SetBlackboardKeyValue<UBlackboardKeyType_Class>(OwnerComp, BlackboardKey, Value);
}

EBTNodeResult::Type UBTTask_SetKeyValueEnum::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	return FBlackboard::SetBlackboardKeyValue<UBlackboardKeyType_Enum>(OwnerComp, BlackboardKey, Value);
}

EBTNodeResult::Type UBTTask_SetKeyValueInt32::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	return FBlackboard::SetBlackboardKeyValue<UBlackboardKeyType_Int>(OwnerComp, BlackboardKey, Value);
}

EBTNodeResult::Type UBTTask_SetKeyValueFloat::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	return FBlackboard::SetBlackboardKeyValue<UBlackboardKeyType_Float>(OwnerComp, BlackboardKey, Value);
}

EBTNodeResult::Type UBTTask_SetKeyValueName::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	return FBlackboard::SetBlackboardKeyValue<UBlackboardKeyType_Name>(OwnerComp, BlackboardKey, Value);
}

EBTNodeResult::Type UBTTask_SetKeyValueString::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	return FBlackboard::SetBlackboardKeyValue<UBlackboardKeyType_String>(OwnerComp, BlackboardKey, Value);
}

EBTNodeResult::Type UBTTask_SetKeyValueObject::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	return FBlackboard::SetBlackboardKeyValue<UBlackboardKeyType_Object>(OwnerComp, BlackboardKey, Value);
}

EBTNodeResult::Type UBTTask_SetKeyValueRotator::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	return FBlackboard::SetBlackboardKeyValue<UBlackboardKeyType_Rotator>(OwnerComp, BlackboardKey, Value);
}

EBTNodeResult::Type UBTTask_SetKeyValueStruct::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	return FBlackboard::SetBlackboardKeyValue<UBlackboardKeyType_Struct>(OwnerComp, BlackboardKey, Value);
}

EBTNodeResult::Type UBTTask_SetKeyValueVector::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	return FBlackboard::SetBlackboardKeyValue<UBlackboardKeyType_Vector>(OwnerComp, BlackboardKey, Value);
}

FString UBTTask_SetKeyValueBool::GetStaticDescription() const
{
	return FString::Format(TEXT("Setting {0} to {1}"), { BlackboardKey.SelectedKeyName.ToString(), Value.ToString() });
}

FString UBTTask_SetKeyValueClass::GetStaticDescription() const
{
	return FString::Format(TEXT("Setting {0} to {1}"), { BlackboardKey.SelectedKeyName.ToString(), Value.ToString() });
}

FString UBTTask_SetKeyValueEnum::GetStaticDescription() const
{
	return FString::Format(TEXT("Setting {0} to {1}"), { BlackboardKey.SelectedKeyName.ToString(), Value.ToString() });
}

FString UBTTask_SetKeyValueInt32::GetStaticDescription() const
{
	return FString::Format(TEXT("Setting {0} to {1}"), { BlackboardKey.SelectedKeyName.ToString(), Value.ToString() });
}

FString UBTTask_SetKeyValueFloat::GetStaticDescription() const
{
	return FString::Format(TEXT("Setting {0} to {1}"), { BlackboardKey.SelectedKeyName.ToString(), Value.ToString() });
}

FString UBTTask_SetKeyValueName::GetStaticDescription() const
{
	return FString::Format(TEXT("Setting {0} to {1}"), { BlackboardKey.SelectedKeyName.ToString(), Value.ToString() });
}

FString UBTTask_SetKeyValueString::GetStaticDescription() const
{
	return FString::Format(TEXT("Setting {0} to {1}"), { BlackboardKey.SelectedKeyName.ToString(), Value.ToString() });
}

FString UBTTask_SetKeyValueObject::GetStaticDescription() const
{
	return FString::Format(TEXT("Setting {0} to {1}"), { BlackboardKey.SelectedKeyName.ToString(), Value.ToString() });
}

FString UBTTask_SetKeyValueRotator::GetStaticDescription() const
{
	return FString::Format(TEXT("Setting {0} to {1}"), { BlackboardKey.SelectedKeyName.ToString(), Value.ToString() });
}

FString UBTTask_SetKeyValueStruct::GetStaticDescription() const
{
	return FString::Format(TEXT("Setting {0} to {1}"), { BlackboardKey.SelectedKeyName.ToString(), Value.ToString() });
}

FString UBTTask_SetKeyValueVector::GetStaticDescription() const
{
	return FString::Format(TEXT("Setting {0} to {1}"), { BlackboardKey.SelectedKeyName.ToString(), Value.ToString() });
}

void UBTTask_SetKeyValueClass::PostLoad()
{
	Super::PostLoad();

	if (BaseClass != UObject::StaticClass())
	{
		BlackboardKey.AllowedTypes.Empty();
		BlackboardKey.AddClassFilter(this, GET_MEMBER_NAME_CHECKED(UBTTask_SetKeyValueClass, BlackboardKey), BaseClass);
		Value.SetBaseClass(BaseClass);
	}
}

void UBTTask_SetKeyValueEnum::PostLoad()
{
	Super::PostLoad();

	if (EnumType)
	{
		BlackboardKey.AllowedTypes.Empty();
		BlackboardKey.AddEnumFilter(this, GET_MEMBER_NAME_CHECKED(UBTTask_SetKeyValueEnum, BlackboardKey), EnumType);
		BlackboardKey.AllowNoneAsValue(false);
		Value.SetEnumType(EnumType);
	}
}

void UBTTask_SetKeyValueObject::PostLoad()
{
	Super::PostLoad();

	if (BaseClass && BaseClass != UObject::StaticClass())
	{
		BlackboardKey.AllowedTypes.Empty();
		BlackboardKey.AddObjectFilter(this, GET_MEMBER_NAME_CHECKED(UBTTask_SetKeyValueObject, BlackboardKey), BaseClass);
		Value.SetBaseClass(BaseClass);
	}
}

void UBTTask_SetKeyValueStruct::PostLoad()
{
	Super::PostLoad();

	if (StructType)
	{
		BlackboardKey.AllowedTypes.Empty();
		BlackboardKey.AddStructFilter(this, GET_MEMBER_NAME_CHECKED(UBTTask_SetKeyValueStruct, BlackboardKey), StructType);
		BlackboardKey.AllowNoneAsValue(false);
		Value.SetStructType(StructType);
	}
}

#if WITH_EDITOR
void UBTTask_SetKeyValueClass::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UBTTask_SetKeyValueClass, BaseClass))
	{
		BlackboardKey.AllowedTypes.Empty();
		BlackboardKey.AddClassFilter(this, GET_MEMBER_NAME_CHECKED(UBTTask_SetKeyValueClass, BlackboardKey), BaseClass);
		BlackboardKey.SelectedKeyName = NAME_None;
		BlackboardKey.InvalidateResolvedKey();
		if (const UBlackboardData* Blackboard = GetBlackboardAsset())
		{
			BlackboardKey.ResolveSelectedKey(*Blackboard);
		}
		Value.SetBaseClass(BaseClass);
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UBTTask_SetKeyValueEnum::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UBTTask_SetKeyValueEnum, EnumType))
	{
		BlackboardKey.AllowedTypes.Empty();
		BlackboardKey.AddEnumFilter(this, GET_MEMBER_NAME_CHECKED(UBTTask_SetKeyValueEnum, BlackboardKey), EnumType);
		BlackboardKey.SelectedKeyName = NAME_None;
		BlackboardKey.InvalidateResolvedKey();
		BlackboardKey.AllowNoneAsValue(EnumType != nullptr);
		if (const UBlackboardData* Blackboard = GetBlackboardAsset())
		{
			BlackboardKey.ResolveSelectedKey(*Blackboard);
		}
		Value.SetEnumType(EnumType);


	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UBTTask_SetKeyValueObject::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UBTTask_SetKeyValueObject, BaseClass))
	{
		BlackboardKey.AllowedTypes.Empty();
		BlackboardKey.AddObjectFilter(this, GET_MEMBER_NAME_CHECKED(UBTTask_SetKeyValueObject, BlackboardKey), BaseClass);
		BlackboardKey.SelectedKeyName = NAME_None;
		BlackboardKey.InvalidateResolvedKey();
		if (const UBlackboardData* Blackboard = GetBlackboardAsset())
		{
			BlackboardKey.ResolveSelectedKey(*Blackboard);
		}
		Value.SetBaseClass(BaseClass);
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UBTTask_SetKeyValueStruct::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UBTTask_SetKeyValueStruct, StructType))
	{
		BlackboardKey.AllowedTypes.Empty();
		BlackboardKey.AddStructFilter(this, GET_MEMBER_NAME_CHECKED(UBTTask_SetKeyValueStruct, BlackboardKey), StructType);
		BlackboardKey.SelectedKeyName = NAME_None;
		BlackboardKey.InvalidateResolvedKey();
		BlackboardKey.AllowNoneAsValue(StructType != nullptr);
		if (const UBlackboardData* Blackboard = GetBlackboardAsset())
		{
			BlackboardKey.ResolveSelectedKey(*Blackboard);
		}
		Value.SetStructType(StructType);
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR
