// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Tasks/BTTask_SetTagCooldown.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTTask_SetTagCooldown)

UBTTask_SetTagCooldown::UBTTask_SetTagCooldown(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Set Tag Cooldown";
	CooldownDuration = 5.0f;	
	bAddToExistingDuration = false;
}

EBTNodeResult::Type UBTTask_SetTagCooldown::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	OwnerComp.AddCooldownTagDuration(CooldownTag, CooldownDuration.GetValue(OwnerComp), bAddToExistingDuration.GetValue(OwnerComp));
	
	return EBTNodeResult::Succeeded;
}

FString UBTTask_SetTagCooldown::GetStaticDescription() const
{
	return FString::Printf(TEXT("%s %s: %s s"), *Super::GetStaticDescription(), *CooldownTag.ToString(), *CooldownDuration.ToString());
}

#if WITH_EDITOR

FName UBTTask_SetTagCooldown::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Decorator.Cooldown.Icon");
}

#endif	// WITH_EDITOR

