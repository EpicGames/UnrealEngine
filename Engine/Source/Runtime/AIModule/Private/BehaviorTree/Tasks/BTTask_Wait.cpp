// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Tasks/BTTask_Wait.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTTask_Wait)

UBTTask_Wait::UBTTask_Wait(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Wait";
	WaitTime = 5.0f;
	bTickIntervals = true;
	INIT_TASK_NODE_NOTIFY_FLAGS();
}

EBTNodeResult::Type UBTTask_Wait::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	const float WaitSecond = WaitTime.GetValue(OwnerComp);
	const float DeviationSecond = RandomDeviation.GetValue(OwnerComp); 
	const float RemainingWaitTime = FMath::FRandRange(FMath::Max(0.0f, WaitSecond - DeviationSecond), (WaitSecond + DeviationSecond));
	SetNextTickTime(NodeMemory, RemainingWaitTime);
	
	return EBTNodeResult::InProgress;
}

void UBTTask_Wait::TickTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, float DeltaSeconds)
{
	// Using the SetNextTickTime in ExecuteTask we are certain we are only getting ticked when the wait is finished
	ensure(GetSpecialNodeMemory<FBTTaskMemory>(NodeMemory)->NextTickRemainingTime <= 0.f);

	// continue execution from this node
	FinishLatentTask(OwnerComp, EBTNodeResult::Succeeded);
}

FString UBTTask_Wait::GetStaticDescription() const
{
	if (!RandomDeviation.IsBoundOrNonZero())
	{
		return FString::Printf(TEXT("%s: %s s"), *Super::GetStaticDescription(), *WaitTime.ToString());
	}
	else
	{
		return FString::Printf(TEXT("%s: %s s +- %s s"), *Super::GetStaticDescription(), *WaitTime.ToString(), *RandomDeviation.ToString());
	}
}

void UBTTask_Wait::DescribeRuntimeValues(const UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTDescriptionVerbosity::Type Verbosity, TArray<FString>& Values) const
{
	Super::DescribeRuntimeValues(OwnerComp, NodeMemory, Verbosity, Values);

	FBTTaskMemory* TaskMemory = GetSpecialNodeMemory<FBTTaskMemory>(NodeMemory);
	if (TaskMemory->NextTickRemainingTime)
	{
		Values.Add(FString::Printf(TEXT("remaining: %ss"), *FString::SanitizeFloat(TaskMemory->NextTickRemainingTime)));
	}
}

#if WITH_EDITOR

FName UBTTask_Wait::GetNodeIconName() const
{
	return FName("BTEditor.Graph.BTNode.Task.Wait.Icon");
}

#endif	// WITH_EDITOR

