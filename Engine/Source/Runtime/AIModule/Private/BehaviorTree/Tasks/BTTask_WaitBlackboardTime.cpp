// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/Tasks/BTTask_WaitBlackboardTime.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Float.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTTask_WaitBlackboardTime)

UBTTask_WaitBlackboardTime::UBTTask_WaitBlackboardTime(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Wait Blackboard Time";

	BlackboardKey.AddFloatFilter(this, GET_MEMBER_NAME_CHECKED(UBTTask_WaitBlackboardTime, BlackboardKey));
}

void UBTTask_WaitBlackboardTime::InitializeFromAsset(UBehaviorTree& Asset)
{
	Super::InitializeFromAsset(Asset);

	if (const UBlackboardData* BBAsset = GetBlackboardAsset())
	{
		BlackboardKey.ResolveSelectedKey(*BBAsset);
		WaitTime.SetKey(BlackboardKey.SelectedKeyName);
	}
	else
	{
		BlackboardKey.InvalidateResolvedKey();
		WaitTime = {};
	}
}

#if WITH_EDITOR
FString UBTTask_WaitBlackboardTime::GetErrorMessage() const
{
	if (GetBlackboardAsset() == nullptr)
	{
		return UE::BehaviorTree::Messages::BlackboardNotSet.ToString();
	}
	return Super::GetErrorMessage();
}
#endif // WITH_EDITOR