// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/ValueOrBBKey.h"

#include "BehaviorTree/BehaviorTree.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Bool.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Class.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Enum.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Float.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Int.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Name.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Object.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Rotator.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_String.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Struct.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Vector.h"
#include "BehaviorTree/BlackboardData.h"
#include "BehaviorTree/BTNode.h" // For LogBehaviorTree

#include UE_INLINE_GENERATED_CPP_BY_NAME(ValueOrBBKey)

namespace FBlackboard
{
	FConstStructView TryGetBlackboardKeyStruct(const UBlackboardComponent& Blackboard, const FName& Name, FBlackboard::FKey& InOutCachedKey, const UScriptStruct* TargetStruct)
	{
		if (InOutCachedKey == FBlackboard::InvalidKey)
		{
			InOutCachedKey = Blackboard.GetKeyID(Name);
		}

		if (const UBlackboardData* BlackBoarData = Blackboard.GetBlackboardAsset())
		{
			if (const FBlackboardEntry* BBEntry = BlackBoarData->GetKey(InOutCachedKey))
			{
				if (UBlackboardKeyType_Struct* StructKeyType = Cast<UBlackboardKeyType_Struct>(BBEntry->KeyType))
				{
					if (StructKeyType->DefaultValue.GetScriptStruct() == TargetStruct)
					{
						return Blackboard.GetValue<UBlackboardKeyType_Struct>(InOutCachedKey);
					}
				}
			}
		}

		return FConstStructView{};
	}

	FConstStructView GetStructValue(const UBlackboardComponent& Blackboard, const FName& Name, FBlackboard::FKey& InOutCachedKey, const FConstStructView& DefaultValue)
	{
		if (!Name.IsNone())
		{
			FConstStructView KeyValue = FBlackboard::TryGetBlackboardKeyStruct(Blackboard, Name, InOutCachedKey, DefaultValue.GetScriptStruct());
			if (KeyValue.IsValid())
			{
				return KeyValue;
			}
		}
		return DefaultValue;
	}

	FConstStructView GetStructValue(const UBehaviorTreeComponent& BehaviorComp, const FName& Name, FBlackboard::FKey& InOutCachedKey, const FConstStructView& DefaultValue)
	{
		if (const UBlackboardComponent* Blackboard = BehaviorComp.GetBlackboardComponent())
		{
			return GetStructValue(*Blackboard, Name, InOutCachedKey, DefaultValue);
		}
		return DefaultValue;
	}
} // namespace FBlackboard

FBlackboard::FKey FValueOrBlackboardKeyBase::GetKeyId(const UBehaviorTreeComponent& OwnerComp) const
{
	if (KeyId == FBlackboard::InvalidKey)
	{
		if (const UBlackboardComponent* BlackboardComp = OwnerComp.GetBlackboardComponent())
		{
			KeyId = BlackboardComp->GetKeyID(Key);
		}
	}
	return KeyId;
}

FString FValueOrBlackboardKeyBase::ToStringKeyName() const
{
	return FString::Format(TEXT("Key: {0}"), { *Key.ToString() });
}

FString FValueOrBBKey_Class::ToString() const
{
	if (!Key.IsNone())
	{
		return ToStringKeyName();
	}
	else
	{
		return GetNameSafe(DefaultValue);
	}
}

FString FValueOrBBKey_Enum::ToString() const
{
	if (!Key.IsNone())
	{
		return ToStringKeyName();
	}
	else
	{
		return EnumType ? EnumType->GetNameStringByValue(DefaultValue) : TEXT("Invalid Enum");
	}
}

FString FValueOrBBKey_Float::ToString() const
{
	if (!Key.IsNone())
	{
		return ToStringKeyName();
	}
	else
	{
		return FString::Printf(TEXT("%.2f"), DefaultValue);
	}
}

bool FValueOrBBKey_Float::IsBoundOrNonZero() const
{
	return !Key.IsNone() || !FMath::IsNearlyEqual(DefaultValue, 0.f);
}

FString FValueOrBBKey_Name::ToString() const
{
	if (!Key.IsNone())
	{
		return ToStringKeyName();
	}
	else
	{
		return DefaultValue.ToString();
	}
}

FString FValueOrBBKey_Object::ToString() const
{
	if (!Key.IsNone())
	{
		return ToStringKeyName();
	}
	else
	{
		return GetNameSafe(DefaultValue);
	}
}

FString FValueOrBBKey_Rotator::ToString() const
{
	if (!Key.IsNone())
	{
		return ToStringKeyName();
	}
	else
	{
		return DefaultValue.ToString();
	}
}

FString FValueOrBBKey_Struct::ToString() const
{
	if (!Key.IsNone())
	{
		return ToStringKeyName();
	}
	else
	{
		return GetNameSafe(DefaultValue.GetScriptStruct());
	}
}

FString FValueOrBBKey_Vector::ToString() const
{
	if (!Key.IsNone())
	{
		return ToStringKeyName();
	}
	else
	{
		return DefaultValue.ToString();
	}
}

bool FValueOrBBKey_Bool::GetValue(const UBlackboardComponent& Blackboard) const
{
	return FBlackboard::GetValue<UBlackboardKeyType_Bool>(Blackboard, Key, KeyId, DefaultValue);
}

bool FValueOrBBKey_Bool::GetValue(const UBlackboardComponent* Blackboard) const
{
	return Blackboard ? GetValue(*Blackboard) : DefaultValue;
}

bool FValueOrBBKey_Bool::GetValue(const UBehaviorTreeComponent& BehaviorComp) const
{
	return FBlackboard::GetValue<UBlackboardKeyType_Bool>(BehaviorComp, Key, KeyId, DefaultValue);
}

bool FValueOrBBKey_Bool::GetValue(const UBehaviorTreeComponent* BehaviorComp) const
{
	return BehaviorComp ? GetValue(*BehaviorComp) : DefaultValue;
}

UClass* FValueOrBBKey_Class::GetValue(const UBlackboardComponent& Blackboard) const
{
	return FBlackboard::GetValue<UBlackboardKeyType_Class>(Blackboard, Key, KeyId, DefaultValue.Get());
}

UClass* FValueOrBBKey_Class::GetValue(const UBlackboardComponent* Blackboard) const
{
	return Blackboard ? GetValue(*Blackboard) : DefaultValue.Get();
}

UClass* FValueOrBBKey_Class::GetValue(const UBehaviorTreeComponent& BehaviorComp) const
{
	return FBlackboard::GetValue<UBlackboardKeyType_Class>(BehaviorComp, Key, KeyId, DefaultValue.Get());
}

UClass* FValueOrBBKey_Class::GetValue(const UBehaviorTreeComponent* BehaviorComp) const
{
	return BehaviorComp ? GetValue(*BehaviorComp) : DefaultValue.Get();
}

uint8 FValueOrBBKey_Enum::GetValue(const UBlackboardComponent& Blackboard) const
{
	return FBlackboard::GetValue<UBlackboardKeyType_Enum>(Blackboard, Key, KeyId, DefaultValue);
}

uint8 FValueOrBBKey_Enum::GetValue(const UBlackboardComponent* Blackboard) const
{
	return Blackboard ? GetValue(*Blackboard) : DefaultValue;
}

uint8 FValueOrBBKey_Enum::GetValue(const UBehaviorTreeComponent& BehaviorComp) const
{
	return FBlackboard::GetValue<UBlackboardKeyType_Enum>(BehaviorComp, Key, KeyId, DefaultValue);
}

uint8 FValueOrBBKey_Enum::GetValue(const UBehaviorTreeComponent* BehaviorComp) const
{
	return BehaviorComp ? GetValue(*BehaviorComp) : DefaultValue;
}

float FValueOrBBKey_Float::GetValue(const UBlackboardComponent& Blackboard) const
{
	return FBlackboard::GetValue<UBlackboardKeyType_Float>(Blackboard, Key, KeyId, DefaultValue);
}

float FValueOrBBKey_Float::GetValue(const UBlackboardComponent* Blackboard) const
{
	return Blackboard ? GetValue(*Blackboard) : DefaultValue;
}

float FValueOrBBKey_Float::GetValue(const UBehaviorTreeComponent& BehaviorComp) const
{
	return FBlackboard::GetValue<UBlackboardKeyType_Float>(BehaviorComp, Key, KeyId, DefaultValue);
}

float FValueOrBBKey_Float::GetValue(const UBehaviorTreeComponent* BehaviorComp) const
{
	return BehaviorComp ? GetValue(*BehaviorComp) : DefaultValue;
}

int32 FValueOrBBKey_Int32::GetValue(const UBlackboardComponent& Blackboard) const
{
	return FBlackboard::GetValue<UBlackboardKeyType_Int>(Blackboard, Key, KeyId, DefaultValue);
}

int32 FValueOrBBKey_Int32::GetValue(const UBlackboardComponent* Blackboard) const
{
	return Blackboard ? GetValue(*Blackboard) : DefaultValue;
}

int32 FValueOrBBKey_Int32::GetValue(const UBehaviorTreeComponent& BehaviorComp) const
{
	return FBlackboard::GetValue<UBlackboardKeyType_Int>(BehaviorComp, Key, KeyId, DefaultValue);
}

int32 FValueOrBBKey_Int32::GetValue(const UBehaviorTreeComponent* BehaviorComp) const
{
	return BehaviorComp ? GetValue(*BehaviorComp) : DefaultValue;
}

FName FValueOrBBKey_Name::GetValue(const UBlackboardComponent& Blackboard) const
{
	return FBlackboard::GetValue<UBlackboardKeyType_Name>(Blackboard, Key, KeyId, DefaultValue);
}

FName FValueOrBBKey_Name::GetValue(const UBlackboardComponent* Blackboard) const
{
	return Blackboard ? GetValue(*Blackboard) : DefaultValue;
}

FName FValueOrBBKey_Name::GetValue(const UBehaviorTreeComponent& BehaviorComp) const
{
	return FBlackboard::GetValue<UBlackboardKeyType_Name>(BehaviorComp, Key, KeyId, DefaultValue);
}

FName FValueOrBBKey_Name::GetValue(const UBehaviorTreeComponent* BehaviorComp) const
{
	return BehaviorComp ? GetValue(*BehaviorComp) : DefaultValue;
}

FString FValueOrBBKey_String::GetValue(const UBlackboardComponent& Blackboard) const
{
	return FBlackboard::GetValue<UBlackboardKeyType_String>(Blackboard, Key, KeyId, DefaultValue);
}

FString FValueOrBBKey_String::GetValue(const UBlackboardComponent* Blackboard) const
{
	return Blackboard ? GetValue(*Blackboard) : DefaultValue;
}

FString FValueOrBBKey_String::GetValue(const UBehaviorTreeComponent& BehaviorComp) const
{
	return FBlackboard::GetValue<UBlackboardKeyType_String>(BehaviorComp, Key, KeyId, DefaultValue);
}

FString FValueOrBBKey_String::GetValue(const UBehaviorTreeComponent* BehaviorComp) const
{
	return BehaviorComp ? GetValue(*BehaviorComp) : DefaultValue;
}

UObject* FValueOrBBKey_Object::GetValue(const UBlackboardComponent& Blackboard) const
{
	return FBlackboard::GetValue<UBlackboardKeyType_Object>(Blackboard, Key, KeyId, DefaultValue.Get());
}

UObject* FValueOrBBKey_Object::GetValue(const UBlackboardComponent* Blackboard) const
{
	return Blackboard ? GetValue(*Blackboard) : DefaultValue.Get();
}

UObject* FValueOrBBKey_Object::GetValue(const UBehaviorTreeComponent& BehaviorComp) const
{
	return FBlackboard::GetValue<UBlackboardKeyType_Object>(BehaviorComp, Key, KeyId, DefaultValue.Get());
}

UObject* FValueOrBBKey_Object::GetValue(const UBehaviorTreeComponent* BehaviorComp) const
{
	return BehaviorComp ? GetValue(*BehaviorComp) : DefaultValue.Get();
}

FRotator FValueOrBBKey_Rotator::GetValue(const UBlackboardComponent& Blackboard) const
{
	return FBlackboard::GetValue<UBlackboardKeyType_Rotator>(Blackboard, Key, KeyId, DefaultValue);
}

FRotator FValueOrBBKey_Rotator::GetValue(const UBlackboardComponent* Blackboard) const
{
	return Blackboard ? GetValue(*Blackboard) : DefaultValue;
}

FRotator FValueOrBBKey_Rotator::GetValue(const UBehaviorTreeComponent& BehaviorComp) const
{
	return FBlackboard::GetValue<UBlackboardKeyType_Rotator>(BehaviorComp, Key, KeyId, DefaultValue);
}

FRotator FValueOrBBKey_Rotator::GetValue(const UBehaviorTreeComponent* BehaviorComp) const
{
	return BehaviorComp ? GetValue(*BehaviorComp) : DefaultValue;
}

FConstStructView FValueOrBBKey_Struct::GetValue(const UBlackboardComponent& Blackboard) const
{
	return FBlackboard::GetStructValue(Blackboard, Key, KeyId, FConstStructView(DefaultValue));
}

FConstStructView FValueOrBBKey_Struct::GetValue(const UBlackboardComponent* Blackboard) const
{
	return Blackboard ? GetValue(*Blackboard) : DefaultValue;
}

FConstStructView FValueOrBBKey_Struct::GetValue(const UBehaviorTreeComponent& BehaviorComp) const
{
	return FBlackboard::GetStructValue(BehaviorComp, Key, KeyId, FConstStructView(DefaultValue));
}

FConstStructView FValueOrBBKey_Struct::GetValue(const UBehaviorTreeComponent* BehaviorComp) const
{
	return BehaviorComp ? GetValue(*BehaviorComp) : DefaultValue;
}

FVector FValueOrBBKey_Vector::GetValue(const UBlackboardComponent& Blackboard) const
{
	return FBlackboard::GetValue<UBlackboardKeyType_Vector>(Blackboard, Key, KeyId, DefaultValue);
}

FVector FValueOrBBKey_Vector::GetValue(const UBlackboardComponent* Blackboard) const
{
	return Blackboard ? GetValue(*Blackboard) : DefaultValue;
}

FVector FValueOrBBKey_Vector::GetValue(const UBehaviorTreeComponent& BehaviorComp) const
{
	return FBlackboard::GetValue<UBlackboardKeyType_Vector>(BehaviorComp, Key, KeyId, DefaultValue);
}

FVector FValueOrBBKey_Vector::GetValue(const UBehaviorTreeComponent* BehaviorComp) const
{
	return BehaviorComp ? GetValue(*BehaviorComp) : DefaultValue;
}

bool FValueOrBBKey_Bool::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_BoolProperty)
	{
		DefaultValue = Tag.BoolVal != 0;
		return true;
	}
	return false;
}

bool FValueOrBBKey_Class::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_ObjectProperty) // Class and object share the same tag
	{
		Slot << DefaultValue;
		return true;
	}
	return false;
}

bool FValueOrBBKey_Enum::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if ((Tag.Type == NAME_EnumProperty || Tag.Type == NAME_ByteProperty) && Tag.GetType().GetParameterCount() > 0)
	{
		if (const UEnum* Enum = FindFirstObject<UEnum>(*Tag.GetType().GetParameterName(0).ToString(), EFindFirstObjectOptions::NativeFirst))
		{
			FName EnumValue;
			Slot << EnumValue;
			DefaultValue = static_cast<uint8>(Enum->GetValueByName(EnumValue));
			return true;
		}
	}
	return false;
}

bool FValueOrBBKey_Float::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_FloatProperty)
	{
		Slot << DefaultValue;
		return true;
	}
	return false;
}

bool FValueOrBBKey_Int32::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_Int32Property || Tag.Type == NAME_IntProperty)
	{
		Slot << DefaultValue;
		return true;
	}
	return false;
}

bool FValueOrBBKey_Name::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_NameProperty)
	{
		Slot << DefaultValue;
		return true;
	}
	return false;
}

bool FValueOrBBKey_String::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_StrProperty)
	{
		Slot << DefaultValue;
		return true;
	}
	return false;
}

bool FValueOrBBKey_Object::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_ObjectProperty)
	{
		Slot << DefaultValue;
		return true;
	}
	return false;
}

bool FValueOrBBKey_Rotator::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_RotatorProperty)
	{
		Slot << DefaultValue;
		return true;
	}
	return false;
}

bool FValueOrBBKey_Struct::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_StructProperty && Tag.GetType().GetParameterCount() > 0)
	{
		if (UScriptStruct* Struct = FindFirstObject<UScriptStruct>(*Tag.GetType().GetParameterName(0).ToString(), EFindFirstObjectOptions::NativeFirst))
		{
			if (Struct == DefaultValue.GetScriptStruct() || DefaultValue.GetScriptStruct() == nullptr)
			{
				TUniquePtr<uint8[]> SerializedStruct = MakeUnique<uint8[]>(Struct->GetStructureSize());
				Struct->InitializeStruct(SerializedStruct.Get());
				Struct->SerializeItem(Slot, SerializedStruct.Get(), nullptr);
				DefaultValue.InitializeAs(Struct, SerializedStruct.Get());
				return true;
			}
		}
	}
	return false;
}

bool FValueOrBBKey_Vector::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_VectorProperty)
	{
		Slot << DefaultValue;
		return true;
	}
	return false;
}

void FValueOrBBKey_Class::SetBaseClass(UClass* NewBaseClass)
{
	if (NewBaseClass != BaseClass)
	{
		if (!BaseClass || !BaseClass->IsChildOf(NewBaseClass))
		{
			Key = NAME_None;
			DefaultValue = nullptr;
		}
		BaseClass = NewBaseClass;
	}
}

void FValueOrBBKey_Enum::SetEnumType(UEnum* NewEnumType)
{
	if (EnumType != NewEnumType)
	{
		Key = NAME_None;
		EnumType = NewEnumType;
		if (EnumType)
		{
			DefaultValue = static_cast<uint8>(EnumType->GetValueByIndex(0));
		}
	}
}

void FValueOrBBKey_Object::SetBaseClass(UClass* NewBaseClass)
{
	if (NewBaseClass != BaseClass)
	{
		if (!BaseClass || !BaseClass->IsChildOf(NewBaseClass))
		{
			Key = NAME_None;
			DefaultValue = nullptr;
		}
		BaseClass = NewBaseClass;
	}
}

void FValueOrBBKey_Struct::SetStructType(UScriptStruct* NewStructType)
{
	if (NewStructType != DefaultValue.GetScriptStruct())
	{
		DefaultValue.InitializeAs(NewStructType, nullptr);
		Key = NAME_None;
	}
}

#if WITH_EDITOR
void FValueOrBlackboardKeyBase::PreSave(const UObject* Outer, const UBlackboardData& Blackboard, FName PropertyName)
{
	if (!Key.IsNone())
	{
		const FBlackboard::FKey Id = Blackboard.GetKeyID(Key);
		if (Id != FBlackboard::InvalidKey)
		{
			if (const FBlackboardEntry* BlackboardEntry = Blackboard.GetKey(Id))
			{
				if (IsCompatibleType(BlackboardEntry->KeyType))
				{
					return;
				}
				else
				{
					UE_LOGF(LogBehaviorTree, Warning, "%ls in node %ls in bt %ls is bound to key %ls but the type doesn't match in blackboard %ls. Resetting the key to none.", *PropertyName.ToString(), *GetNameSafe(Outer), Outer ? *GetNameSafe(Outer->GetTypedOuter<UBehaviorTree>()) : TEXT("None"), *Key.ToString(), *Blackboard.GetFullName());
				}
			}
			else
			{
				UE_LOGF(LogBehaviorTree, Warning, "%ls in node %ls in bt %ls is bound to key %ls but the key doesn't exist in blackboard %ls. Resetting the key to none.", *PropertyName.ToString(), *GetNameSafe(Outer), Outer ? *GetNameSafe(Outer->GetTypedOuter<UBehaviorTree>()) : TEXT("None"), *Key.ToString(), *Blackboard.GetFullName());
			}
		}
		else
		{
			UE_LOGF(LogBehaviorTree, Warning, "%ls in node %ls in bt %ls is bound to key %ls but the key doesn't exist in blackboard %ls. Resetting the key to none.", *PropertyName.ToString(), *GetNameSafe(Outer), Outer ? *GetNameSafe(Outer->GetTypedOuter<UBehaviorTree>()) : TEXT("None"), *Key.ToString(), *Blackboard.GetFullName());
		}
		Key = NAME_None;
	}
}

bool FValueOrBBKey_Bool::IsCompatibleType(const UBlackboardKeyType* KeyType) const
{
	return KeyType && KeyType->GetClass() == UBlackboardKeyType_Bool::StaticClass();
}

bool FValueOrBBKey_Class::IsCompatibleType(const UBlackboardKeyType* KeyType) const
{
	if (const UBlackboardKeyType_Class* ClassKey = Cast<UBlackboardKeyType_Class>(KeyType))
	{
		if (BaseClass && ClassKey->BaseClass)
		{
			return ClassKey->BaseClass->IsChildOf(BaseClass);
		}
	}
	return false;
}

bool FValueOrBBKey_Enum::IsCompatibleType(const UBlackboardKeyType* KeyType) const
{
	if (const UBlackboardKeyType_Enum* EnumKey = Cast<UBlackboardKeyType_Enum>(KeyType))
	{
		if (EnumType && EnumKey->EnumType)
		{
			return EnumKey->EnumType == EnumType;
		}
	}
	return false;
}

bool FValueOrBBKey_Float::IsCompatibleType(const UBlackboardKeyType* KeyType) const
{
	return KeyType && KeyType->GetClass() == UBlackboardKeyType_Float::StaticClass();
}

bool FValueOrBBKey_Int32::IsCompatibleType(const UBlackboardKeyType* KeyType) const
{
	return KeyType && KeyType->GetClass() == UBlackboardKeyType_Int::StaticClass();
}

bool FValueOrBBKey_Name::IsCompatibleType(const UBlackboardKeyType* KeyType) const
{
	return KeyType && KeyType->GetClass() == UBlackboardKeyType_Name::StaticClass();
}

bool FValueOrBBKey_String::IsCompatibleType(const UBlackboardKeyType* KeyType) const
{
	return KeyType && KeyType->GetClass() == UBlackboardKeyType_String::StaticClass();
}

bool FValueOrBBKey_Object::IsCompatibleType(const UBlackboardKeyType* KeyType) const
{
	if (const UBlackboardKeyType_Object* ObjectKey = Cast<UBlackboardKeyType_Object>(KeyType))
	{
		if (BaseClass && ObjectKey->BaseClass)
		{
			return ObjectKey->BaseClass->IsChildOf(BaseClass);
		}
	}
	return false;
}

bool FValueOrBBKey_Rotator::IsCompatibleType(const UBlackboardKeyType* KeyType) const
{
	return KeyType && KeyType->GetClass() == UBlackboardKeyType_Rotator::StaticClass();
}

bool FValueOrBBKey_Struct::IsCompatibleType(const UBlackboardKeyType* KeyType) const
{
	if (const UBlackboardKeyType_Struct* StructKey = Cast<UBlackboardKeyType_Struct>(KeyType))
	{
		return StructKey->DefaultValue.GetScriptStruct() == DefaultValue.GetScriptStruct();
	}
	return false;
}

bool FValueOrBBKey_Vector::IsCompatibleType(const UBlackboardKeyType* KeyType) const
{
	return KeyType && KeyType->GetClass() == UBlackboardKeyType_Vector::StaticClass();
}
#endif // WITH_EDITOR
