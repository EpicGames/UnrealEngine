// Copyright Epic Games, Inc. All Rights Reserved.

#include "BehaviorTree/ValueOrBBKeyBlueprintUtility.h"

#include "BehaviorTree/ValueOrBBKey.h"
#include "StructUtils/StructView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ValueOrBBKeyBlueprintUtility)

bool UValueOrBBKeyBlueprintUtility::GetBool(const FValueOrBBKey_Bool& Value, const UBehaviorTreeComponent* BehaviorTreeComp)
{
	return Value.GetValue(BehaviorTreeComp);
}

UClass* UValueOrBBKeyBlueprintUtility::GetClass(const FValueOrBBKey_Class& Value, const UBehaviorTreeComponent* BehaviorTreeComp)
{
	return Value.GetValue(BehaviorTreeComp);
}

uint8 UValueOrBBKeyBlueprintUtility::GetEnum(const FValueOrBBKey_Enum& Value, const UBehaviorTreeComponent* BehaviorTreeComp)
{
	return Value.GetValue(BehaviorTreeComp);
}

float UValueOrBBKeyBlueprintUtility::GetFloat(const FValueOrBBKey_Float& Value, const UBehaviorTreeComponent* BehaviorTreeComp)
{
	return Value.GetValue(BehaviorTreeComp);
}

int32 UValueOrBBKeyBlueprintUtility::GetInt32(const FValueOrBBKey_Int32& Value, const UBehaviorTreeComponent* BehaviorTreeComp)
{
	return Value.GetValue(BehaviorTreeComp);
}

FName UValueOrBBKeyBlueprintUtility::GetName(const FValueOrBBKey_Name& Value, const UBehaviorTreeComponent* BehaviorTreeComp)
{
	return Value.GetValue(BehaviorTreeComp);
}

FString UValueOrBBKeyBlueprintUtility::GetString(const FValueOrBBKey_String& Value, const UBehaviorTreeComponent* BehaviorTreeComp)
{
	return Value.GetValue(BehaviorTreeComp);
}

UObject* UValueOrBBKeyBlueprintUtility::GetObject(const FValueOrBBKey_Object& Value, const UBehaviorTreeComponent* BehaviorTreeComp)
{
	return Value.GetValue(BehaviorTreeComp);
}

FRotator UValueOrBBKeyBlueprintUtility::GetRotator(const FValueOrBBKey_Rotator& Value, const UBehaviorTreeComponent* BehaviorTreeComp)
{
	return Value.GetValue(BehaviorTreeComp);
}

FVector UValueOrBBKeyBlueprintUtility::GetVector(const FValueOrBBKey_Vector& Value, const UBehaviorTreeComponent* BehaviorTreeComp)
{
	return Value.GetValue(BehaviorTreeComp);
}

FInstancedStruct UValueOrBBKeyBlueprintUtility::GetStruct(const FValueOrBBKey_Struct& Value, const UBehaviorTreeComponent* BehaviorTreeComp)
{
	return FInstancedStruct(Value.GetValue(BehaviorTreeComp));
}
