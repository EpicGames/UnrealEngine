// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "ValueOrBBKeyBlueprintUtility.generated.h"

struct FValueOrBBKey_Bool;
struct FValueOrBBKey_Class;
struct FValueOrBBKey_Enum;
struct FValueOrBBKey_Float;
struct FValueOrBBKey_Int32;
struct FValueOrBBKey_Name;
struct FValueOrBBKey_String;
struct FValueOrBBKey_Object;
struct FValueOrBBKey_Rotator;
struct FValueOrBBKey_Vector;

UCLASS()
class UValueOrBBKeyBlueprintUtility : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintPure, Category = Blackboard)
	static bool GetBool(const FValueOrBBKey_Bool& Value, const UBehaviorTreeComponent* BehaviorTreeComp);

	UFUNCTION(BlueprintPure, Category = Blackboard)
	static UClass* GetClass(const FValueOrBBKey_Class& Value, const UBehaviorTreeComponent* BehaviorTreeComp);

	UFUNCTION(BlueprintPure, Category = Blackboard)
	static uint8 GetEnum(const FValueOrBBKey_Enum& Value, const UBehaviorTreeComponent* BehaviorTreeComp);

	UFUNCTION(BlueprintPure, Category = Blackboard)
	static float GetFloat(const FValueOrBBKey_Float& Value, const UBehaviorTreeComponent* BehaviorTreeComp);

	UFUNCTION(BlueprintPure, Category = Blackboard)
	static int32 GetInt32(const FValueOrBBKey_Int32& Value, const UBehaviorTreeComponent* BehaviorTreeComp);

	UFUNCTION(BlueprintPure, Category = Blackboard)
	static FName GetName(const FValueOrBBKey_Name& Value, const UBehaviorTreeComponent* BehaviorTreeComp);

	UFUNCTION(BlueprintPure, Category = Blackboard)
	static FString GetString(const FValueOrBBKey_String& Value, const UBehaviorTreeComponent* BehaviorTreeComp);

	UFUNCTION(BlueprintPure, Category = Blackboard)
	static UObject* GetObject(const FValueOrBBKey_Object& Value, const UBehaviorTreeComponent* BehaviorTreeComp);

	UFUNCTION(BlueprintPure, Category = Blackboard)
	static FRotator GetRotator(const FValueOrBBKey_Rotator& Value, const UBehaviorTreeComponent* BehaviorTreeComp);

	UFUNCTION(BlueprintPure, Category = Blackboard)
	static FVector GetVector(const FValueOrBBKey_Vector& Value, const UBehaviorTreeComponent* BehaviorTreeComp);

	UFUNCTION(BlueprintPure, Category = Blackboard)
	static FInstancedStruct GetStruct(const FValueOrBBKey_Struct& Value, const UBehaviorTreeComponent* BehaviorTreeComp);
};