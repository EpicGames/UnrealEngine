// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/AIBlueprintHelperLibrary.h"
#include "EngineGlobals.h"
#include "TimerManager.h"
#include "Engine/World.h"
#include "Engine/Engine.h"
#include "AITypes.h"
#include "AISystem.h"
#include "BrainComponent.h"
#include "Navigation/PathFollowingComponent.h"
#include "AIController.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "Blueprint/AIAsyncTaskBlueprintProxy.h"
#include "Animation/AnimInstance.h"
#include "NavigationPath.h"
#include "NavigationData.h"
#include "NavigationSystem.h"
#include "NavMesh/NavMeshPath.h"
#include "Logging/MessageLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AIBlueprintHelperLibrary)

DEFINE_LOG_CATEGORY_STATIC(LogAIBlueprint, Warning, All);

#define LOCTEXT_NAMESPACE "AIBlueprintHelperLibrary"

//----------------------------------------------------------------------//
// UAIAsyncTaskBlueprintProxy
//----------------------------------------------------------------------//

UAIAsyncTaskBlueprintProxy::UAIAsyncTaskBlueprintProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	MyWorld = Cast<UWorld>(GetOuter());
	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		SetFlags(RF_StrongRefOnFrame);
		UAISystem* const AISystem = UAISystem::GetCurrentSafe(MyWorld.Get());
		if (AISystem)
		{
			AISystem->AddReferenceFromProxyObject(this);
		}
	}
}

void UAIAsyncTaskBlueprintProxy::OnMoveCompleted(FAIRequestID RequestID, EPathFollowingResult::Type MovementResult)
{
	if (RequestID.IsEquivalent(MoveRequestId) && AIController.IsValid(true))
	{
		AIController->ReceiveMoveCompleted.RemoveDynamic(this, &UAIAsyncTaskBlueprintProxy::OnMoveCompleted);

		if (MovementResult == EPathFollowingResult::Success)
		{
			OnSuccess.Broadcast(MovementResult);
		}
		else
		{
			OnFail.Broadcast(MovementResult);
		}

		UAISystem* const AISystem = UAISystem::GetCurrentSafe(MyWorld.Get());
		if (AISystem)
		{
			AISystem->RemoveReferenceToProxyObject(this);
		}
	}
}

void UAIAsyncTaskBlueprintProxy::OnNoPath()
{
	OnFail.Broadcast(EPathFollowingResult::Aborted);
	UAISystem* const AISystem = UAISystem::GetCurrentSafe(MyWorld.Get());
	if (AISystem)
	{
		AISystem->RemoveReferenceToProxyObject(this);
	}
}

void UAIAsyncTaskBlueprintProxy::OnAtGoal()
{
	OnSuccess.Broadcast(EPathFollowingResult::Success);
	UAISystem* const AISystem = UAISystem::GetCurrentSafe(MyWorld.Get());
	if (AISystem)
	{
		AISystem->RemoveReferenceToProxyObject(this);
	}
}

void UAIAsyncTaskBlueprintProxy::BeginDestroy()
{
	UAISystem* const AISystem = UAISystem::GetCurrentSafe(MyWorld.Get());
	if (AISystem)
	{
		AISystem->RemoveReferenceToProxyObject(this);
	}
	Super::BeginDestroy();
}

//----------------------------------------------------------------------//
// UAIAsyncTaskBlueprintProxy
//----------------------------------------------------------------------//

UAIBlueprintHelperLibrary::UAIBlueprintHelperLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UAIAsyncTaskBlueprintProxy* UAIBlueprintHelperLibrary::CreateMoveToProxyObject(UObject* WorldContextObject, APawn* Pawn, FVector Destination, AActor* TargetActor, float AcceptanceRadius, bool bStopOnOverlap)
{
	if (!ensureMsgf(WorldContextObject == nullptr || IsValid(WorldContextObject), TEXT("Unusable WorldContextObject pointer provided (i.e., garbage)")))
	{
		return nullptr;
	}

	if (!ensureMsgf(Pawn == nullptr || IsValid(Pawn), TEXT("Unusable Pawn pointer provided (i.e., garbage)")))
	{
		return nullptr;
	}

	if (WorldContextObject == nullptr)
	{
		if (Pawn != nullptr)
		{
			WorldContextObject = Pawn;
		}
		else
		{
			UE_LOGF(LogAIBlueprint, Warning, "Empty (None) world context as well as Pawn passed in while trying to create a MoveTo proxy");
			return nullptr;
		}
	}

	if (Pawn == nullptr)
	{
		// maybe we can extract the pawn from the world context
		AAIController* AsController = Cast<AAIController>(WorldContextObject);
		if (AsController)
		{
			Pawn = AsController->GetPawn();
		}
	}

	if (Pawn == nullptr
		|| !ensureMsgf(IsValid(Pawn), TEXT("Unusable Pawn pointer fetched from AIController (i.e., garbage)")))
	{
		return nullptr;
	}

	UAIAsyncTaskBlueprintProxy* MyObj = nullptr;

	// Make sure the controller is still a valid Actor that can be stored in a weak pointer.
	// This early validation is a tentative fix for low occurence crashes where the AAIController
	// pointer is considered valid by IsValid() (which validates for non-null ptr and without Garbage flag)
	// but then TWeakObjectPtr<>::Get would return null (potentially unreachable) when trying to access
	// the ReceiveMoveCompleted delegate below.
	// We don't want to perform the MoveTo at all in that case.
	const TWeakObjectPtr WeakAIController = Cast<AAIController>(Pawn->GetController());
	const TStrongObjectPtr<AAIController> AIController = WeakAIController.Pin();
	if (AIController != nullptr)
	{
		UWorld* World = GEngine->GetWorldFromContextObjectChecked(WorldContextObject);
		MyObj = NewObject<UAIAsyncTaskBlueprintProxy>(World);

		FAIMoveRequest MoveReq;
		MoveReq.SetUsePathfinding(true);
		MoveReq.SetAcceptanceRadius(AcceptanceRadius);
		MoveReq.SetReachTestIncludesAgentRadius(bStopOnOverlap);
		if (TargetActor)
		{
			MoveReq.SetGoalActor(TargetActor);
		}
		else
		{
			MoveReq.SetGoalLocation(Destination);
		}
		MoveReq.SetNavigationFilter(AIController->GetDefaultNavigationFilterClass());
		MoveReq.SetCanStrafe(AIController->bAllowStrafe);

		const FPathFollowingRequestResult ResultData = AIController->MoveTo(MoveReq);
		switch (ResultData.Code)
		{
		case EPathFollowingRequestResult::RequestSuccessful:
			MyObj->AIController = WeakAIController;
			MyObj->AIController->ReceiveMoveCompleted.AddDynamic(MyObj, &UAIAsyncTaskBlueprintProxy::OnMoveCompleted);
			MyObj->MoveRequestId = ResultData.MoveId;
			break;

		case EPathFollowingRequestResult::AlreadyAtGoal:
			World->GetTimerManager().SetTimer(MyObj->TimerHandle_OnInstantFinish, MyObj, &UAIAsyncTaskBlueprintProxy::OnAtGoal, 0.1f, false);
			break;

		case EPathFollowingRequestResult::Failed:
		default:
			World->GetTimerManager().SetTimer(MyObj->TimerHandle_OnInstantFinish, MyObj, &UAIAsyncTaskBlueprintProxy::OnNoPath, 0.1f, false);
			break;
		}
	}
	return MyObj;
}

void UAIBlueprintHelperLibrary::SendAIMessage(APawn* Target, FName Message, UObject* MessageSource, bool bSuccess)
{
	FAIMessage::Send(Target, FAIMessage(Message, MessageSource, bSuccess));
}

APawn* UAIBlueprintHelperLibrary::SpawnAIFromClass(UObject* WorldContextObject, TSubclassOf<APawn> PawnClass, UBehaviorTree* BehaviorTree, FVector Location, FRotator Rotation, bool bNoCollisionFail, AActor *Owner)
{
	APawn* NewPawn = NULL;

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (World && *PawnClass)
	{
		FActorSpawnParameters ActorSpawnParams;
		ActorSpawnParams.Owner = Owner;
		ActorSpawnParams.SpawnCollisionHandlingOverride = bNoCollisionFail ? ESpawnActorCollisionHandlingMethod::AlwaysSpawn : ESpawnActorCollisionHandlingMethod::AdjustIfPossibleButDontSpawnIfColliding;

		NewPawn = World->SpawnActor<APawn>(*PawnClass, Location, Rotation, ActorSpawnParams);

		if (NewPawn != NULL)
		{
			if (NewPawn->GetController() == NULL)
			{	// NOTE: SpawnDefaultController ALSO calls Possess() to possess the pawn (if a controller is successfully spawned).
				NewPawn->SpawnDefaultController();
			}

			if (BehaviorTree != NULL)
			{
				AAIController* AIController = Cast<AAIController>(NewPawn->GetController());

				if (AIController != NULL)
				{
					AIController->RunBehaviorTree(BehaviorTree);
				}
			}
		}
	}

	return NewPawn;
}

AAIController* UAIBlueprintHelperLibrary::GetAIController(AActor* ControlledActor)
{
	APawn* AsPawn = Cast<APawn>(ControlledActor);
	if (AsPawn != nullptr)
	{
		return Cast<AAIController>(AsPawn->GetController());
	}
	return Cast<AAIController>(ControlledActor);
}

UBlackboardComponent* UAIBlueprintHelperLibrary::GetBlackboard(AActor* Target)
{
	UBlackboardComponent* BlackboardComp = nullptr;

	if (Target != nullptr)
	{
		APawn* TargetPawn = Cast<APawn>(Target);
		if (TargetPawn && TargetPawn->GetController())
		{
			BlackboardComp = TargetPawn->GetController()->FindComponentByClass<UBlackboardComponent>();
		}

		if (BlackboardComp == nullptr)
		{
			BlackboardComp = Target->FindComponentByClass<UBlackboardComponent>();
		}
	}

	return BlackboardComp;
}

void UAIBlueprintHelperLibrary::LockAIResourcesWithAnimation(UAnimInstance* AnimInstance, bool bLockMovement, bool LockAILogic)
{
	if (AnimInstance == NULL)
	{
		return;
	}

	APawn* PawnOwner = AnimInstance->TryGetPawnOwner();
	if (PawnOwner)
	{
		AAIController* OwningAI = Cast<AAIController>(PawnOwner->GetController());
		if (OwningAI)
		{
			if (bLockMovement && OwningAI->GetPathFollowingComponent())
			{
				OwningAI->GetPathFollowingComponent()->LockResource(EAIRequestPriority::HardScript);
			}
			if (LockAILogic && OwningAI->BrainComponent)
			{
				OwningAI->BrainComponent->LockResource(EAIRequestPriority::HardScript);
			}
		}
	}
}

void UAIBlueprintHelperLibrary::UnlockAIResourcesWithAnimation(UAnimInstance* AnimInstance, bool bUnlockMovement, bool UnlockAILogic)
{
	if (AnimInstance == NULL)
	{
		return;
	}

	APawn* PawnOwner = AnimInstance->TryGetPawnOwner();
	if (PawnOwner)
	{
		AAIController* OwningAI = Cast<AAIController>(PawnOwner->GetController());
		if (OwningAI)
		{
			if (bUnlockMovement && OwningAI->GetPathFollowingComponent())
			{
				OwningAI->GetPathFollowingComponent()->ClearResourceLock(EAIRequestPriority::HardScript);
			}
			if (UnlockAILogic && OwningAI->BrainComponent)
			{
				OwningAI->BrainComponent->ClearResourceLock(EAIRequestPriority::HardScript);
			}
		}
	}
}

bool UAIBlueprintHelperLibrary::IsValidAILocation(FVector Location)
{
	return FAISystem::IsValidLocation(Location);
}

bool UAIBlueprintHelperLibrary::IsValidAIDirection(FVector DirectionVector)
{
	return FAISystem::IsValidDirection(DirectionVector);
}

bool UAIBlueprintHelperLibrary::IsValidAIRotation(FRotator Rotation)
{
	return FAISystem::IsValidRotation(Rotation);
}

UNavigationPath* UAIBlueprintHelperLibrary::GetCurrentPath(AController* Controller)
{
	UNavigationPath* ResultPath = nullptr;
	if (Controller)
	{
		AAIController* AIController = Cast<AAIController>(Controller);
		UPathFollowingComponent* PFComp = nullptr;
		if (AIController)
		{
			PFComp = AIController->GetPathFollowingComponent();
		}
		else
		{
			// player controller, most probably using SimpleMove
			PFComp = Controller->FindComponentByClass<UPathFollowingComponent>();
		}

		if (PFComp != nullptr)
		{
			const FNavPathSharedPtr Path = PFComp->GetPath();
			if (Path.IsValid())
			{
				// we don't care if Path->IsValid(), we're going to retrieve whatever's there
				ResultPath = NewObject<UNavigationPath>();
				ResultPath->SetPath(Path);
			}
		}
	}

	return ResultPath;
}

const TArray<FVector> UAIBlueprintHelperLibrary::GetCurrentPathPoints(AController* Controller)
{
	TArray<FVector> PathPoints;
	
	const UPathFollowingComponent* PFComp = GetPathComp(Controller);
	if (PFComp && PFComp->GetPath().IsValid())
	{
		PathPoints.Reserve(PFComp->GetPath()->GetPathPoints().Num());
		for (const FNavPathPoint& NavPathPoint : PFComp->GetPath()->GetPathPoints())
		{
			PathPoints.Add(NavPathPoint.Location);
		}
	}
	return  PathPoints;
}

UPathFollowingComponent* UAIBlueprintHelperLibrary::GetPathComp(const AController* Controller)
{	
	if (Controller)
	{
		UPathFollowingComponent* PFComp = nullptr;
		const AAIController* AIController = Cast<AAIController>(Controller);
		if (AIController)
		{
			return AIController->GetPathFollowingComponent();
		}
		else
		{
			// No AI Controller means its a player controller, most probably moving using SimpleMove
			return Controller->FindComponentByClass<UPathFollowingComponent>();
		}
	}
	return nullptr;
}

int32 UAIBlueprintHelperLibrary::GetCurrentPathIndex(const AController* Controller)
{
	const UPathFollowingComponent* PFComp = GetPathComp(Controller);
	return PFComp ? static_cast<int32>(PFComp->GetCurrentPathIndex()) : INDEX_NONE;
}

int32 UAIBlueprintHelperLibrary::GetNextNavLinkIndex(const AController* Controller)
{
	if (const UPathFollowingComponent* PFComp = GetPathComp(Controller))
	{
		const FNavPathSharedPtr Path = PFComp->GetPath();
		if (Path.IsValid())
		{
			const TArray<FNavPathPoint>& PathPoints = Path->GetPathPoints();
			for (int32 i = PFComp->GetCurrentPathIndex(); i < PathPoints.Num(); ++i)
			{
				if (FNavMeshNodeFlags(PathPoints[i].Flags).IsNavLink())
				{
					return i;
				}
			}
		}
	}

	return INDEX_NONE;
}

namespace
{
	UPathFollowingComponent* InitNavigationControl(AController& Controller)
	{
		AAIController* AsAIController = Cast<AAIController>(&Controller);
		UPathFollowingComponent* PathFollowingComp = nullptr;

		if (AsAIController)
		{
			PathFollowingComp = AsAIController->GetPathFollowingComponent();
		}
		else
		{
			PathFollowingComp = Controller.FindComponentByClass<UPathFollowingComponent>();
			if (PathFollowingComp == nullptr)
			{
				PathFollowingComp = NewObject<UPathFollowingComponent>(&Controller);
				PathFollowingComp->RegisterComponentWithWorld(Controller.GetWorld());
				PathFollowingComp->Initialize();
			}
		}

		return PathFollowingComp;
	}
}

void UAIBlueprintHelperLibrary::SimpleMoveToActor(AController* Controller, const AActor* Goal)
{
	UNavigationSystemV1* NavSys = Controller ? FNavigationSystem::GetCurrent<UNavigationSystemV1>(Controller->GetWorld()) : nullptr;
	if (NavSys == nullptr || Goal == nullptr || Controller == nullptr || Controller->GetPawn() == nullptr)
	{
		UE_LOGF(LogNavigation, Warning, "UNavigationSystemV1::SimpleMoveToActor called for NavSys:%ls Controller:%ls controlling Pawn:%ls with goal actor %ls (if any of these is None then there's your problem",
			*GetNameSafe(NavSys), *GetNameSafe(Controller), Controller ? *GetNameSafe(Controller->GetPawn()) : TEXT("NULL"), *GetNameSafe(Goal));
		return;
	}

	UPathFollowingComponent* PFollowComp = InitNavigationControl(*Controller);

	if (PFollowComp == nullptr)
	{
		FMessageLog("PIE").Warning(FText::Format(
			LOCTEXT("SimpleMoveErrorNoComp", "SimpleMove failed for {0}: missing components"),
			FText::FromName(Controller->GetFName())
		));
		return;
	}

	if (!PFollowComp->IsPathFollowingAllowed())
	{
		FMessageLog("PIE").Warning(FText::Format(
			LOCTEXT("SimpleMoveErrorMovement", "SimpleMove failed for {0}: movement not allowed"),
			FText::FromName(Controller->GetFName())
		));
		return;
	}

	const bool bAlreadyAtGoal = PFollowComp->HasReached(*Goal, EPathFollowingReachMode::OverlapAgentAndGoal);

	// script source, keep only one move request at time
	if (PFollowComp->GetStatus() != EPathFollowingStatus::Idle)
	{
		PFollowComp->AbortMove(*NavSys, FPathFollowingResultFlags::ForcedScript | FPathFollowingResultFlags::NewRequest
			, FAIRequestID::AnyRequest, bAlreadyAtGoal ? EPathFollowingVelocityMode::Reset : EPathFollowingVelocityMode::Keep);
	}

	if (bAlreadyAtGoal)
	{
		PFollowComp->RequestMoveWithImmediateFinish(EPathFollowingResult::Success);
	}
	else
	{
		const FVector AgentNavLocation = Controller->GetNavAgentLocation();
		const ANavigationData* NavData = NavSys->GetNavDataForProps(Controller->GetNavAgentPropertiesRef(), AgentNavLocation);
		if (NavData)
		{
			FPathFindingQuery Query(Controller, *NavData, AgentNavLocation, Goal->GetActorLocation());
			FPathFindingResult Result = NavSys->FindPathSync(Query);
			if (Result.IsSuccessful())
			{
				Result.Path->SetGoalActorObservation(*Goal, 100.0f);
				PFollowComp->RequestMove(FAIMoveRequest(Goal), Result.Path);
			}
			else if (PFollowComp->GetStatus() != EPathFollowingStatus::Idle)
			{
				PFollowComp->RequestMoveWithImmediateFinish(EPathFollowingResult::Invalid);
			}
		}
	}
}

void UAIBlueprintHelperLibrary::SimpleMoveToLocation(AController* Controller, const FVector& GoalLocation)
{
	UNavigationSystemV1* NavSys = Controller ? FNavigationSystem::GetCurrent<UNavigationSystemV1>(Controller->GetWorld()) : nullptr;
	if (NavSys == nullptr || Controller == nullptr || Controller->GetPawn() == nullptr)
	{
		UE_LOGF(LogNavigation, Warning, "UNavigationSystemV1::SimpleMoveToActor called for NavSys:%ls Controller:%ls controlling Pawn:%ls (if any of these is None then there's your problem",
			*GetNameSafe(NavSys), *GetNameSafe(Controller), Controller ? *GetNameSafe(Controller->GetPawn()) : TEXT("NULL"));
		return;
	}

	UPathFollowingComponent* PFollowComp = InitNavigationControl(*Controller);

	if (PFollowComp == nullptr)
	{
		FMessageLog("PIE").Warning(FText::Format(
			LOCTEXT("SimpleMoveErrorNoComp", "SimpleMove failed for {0}: missing components"),
			FText::FromName(Controller->GetFName())
		));
		return;
	}

	if (!PFollowComp->IsPathFollowingAllowed())
	{
		FMessageLog("PIE").Warning(FText::Format(
			LOCTEXT("SimpleMoveErrorMovement", "SimpleMove failed for {0}: movement not allowed"),
			FText::FromName(Controller->GetFName())
		));
		return;
	}

	const bool bAlreadyAtGoal = PFollowComp->HasReached(GoalLocation, EPathFollowingReachMode::OverlapAgent);

	// script source, keep only one move request at time
	if (PFollowComp->GetStatus() != EPathFollowingStatus::Idle)
	{
		PFollowComp->AbortMove(*NavSys, FPathFollowingResultFlags::ForcedScript | FPathFollowingResultFlags::NewRequest
			, FAIRequestID::AnyRequest, bAlreadyAtGoal ? EPathFollowingVelocityMode::Reset : EPathFollowingVelocityMode::Keep);
	}

	// script source, keep only one move request at time
	if (PFollowComp->GetStatus() != EPathFollowingStatus::Idle)
	{
		PFollowComp->AbortMove(*NavSys, FPathFollowingResultFlags::ForcedScript | FPathFollowingResultFlags::NewRequest);
	}

	if (bAlreadyAtGoal)
	{
		PFollowComp->RequestMoveWithImmediateFinish(EPathFollowingResult::Success);
	}
	else
	{
		const FVector AgentNavLocation = Controller->GetNavAgentLocation();
		const ANavigationData* NavData = NavSys->GetNavDataForProps(Controller->GetNavAgentPropertiesRef(), AgentNavLocation);
		if (NavData)
		{
			FPathFindingQuery Query(Controller, *NavData, AgentNavLocation, GoalLocation);
			FPathFindingResult Result = NavSys->FindPathSync(Query);
			if (Result.IsSuccessful())
			{
				PFollowComp->RequestMove(FAIMoveRequest(GoalLocation), Result.Path);
			}
			else if (PFollowComp->GetStatus() != EPathFollowingStatus::Idle)
			{
				PFollowComp->RequestMoveWithImmediateFinish(EPathFollowingResult::Invalid);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
