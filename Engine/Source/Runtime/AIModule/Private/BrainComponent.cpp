// Copyright Epic Games, Inc. All Rights Reserved.

#include "BrainComponent.h"
#include "UObject/Package.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "AIController.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "VisualLogger/VisualLoggerTypes.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BrainComponent)

const FName UBrainComponent::AIMessage_MoveFinished = TEXT("MoveFinished");
const FName UBrainComponent::AIMessage_RepathFailed = TEXT("RepathFailed");
const FName UBrainComponent::AIMessage_QueryFinished = TEXT("QueryFinished");

DEFINE_LOG_CATEGORY(LogBrain);

//////////////////////////////////////////////////////////////////////////
// Messages

static UBrainComponent* FindBrainComponentHelper(const AController* Controller)
{
	return Controller ? Controller->FindComponentByClass<UBrainComponent>() : NULL;
}

static UBrainComponent* FindBrainComponentHelper(const APawn* Pawn)
{
	if (Pawn == NULL)
	{
		return NULL;
	}

	UBrainComponent* Comp = NULL;

	if (Pawn->GetController())
	{
		Comp = FindBrainComponentHelper(Pawn->GetController());
	}

	if (Pawn && Comp == NULL)
	{
		Comp = Pawn->FindComponentByClass<UBrainComponent>();
	}

	return Comp;
}

void FAIMessage::Send(AController* Controller, const FAIMessage& Message)
{
	UBrainComponent* BrainComp = FindBrainComponentHelper(Controller);
	Send(BrainComp, Message);
}

void FAIMessage::Send(APawn* Pawn, const FAIMessage& Message)
{
	UBrainComponent* BrainComp = FindBrainComponentHelper(Pawn);
	Send(BrainComp, Message);
}

void FAIMessage::Send(UBrainComponent* BrainComp, const FAIMessage& Message)
{
	if (BrainComp)
	{
		BrainComp->HandleMessage(Message);
	}
}

void FAIMessage::Broadcast(UObject* WorldContextObject, const FAIMessage& Message)
{
	UWorld* MyWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (MyWorld)
	{
		for (FConstControllerIterator It = MyWorld->GetControllerIterator(); It; ++It)
		{
			FAIMessage::Send(It->Get(), Message);
		}
	}
}

FAIMessageObserver::FAIMessageObserver()
{
}

FAIMessageObserverHandle FAIMessageObserver::Create(AController* Controller, FName MessageType, FOnAIMessage const& Delegate)
{
	UBrainComponent* BrainComp = FindBrainComponentHelper(Controller);
	return Create(BrainComp, MessageType, Delegate);
}

FAIMessageObserverHandle FAIMessageObserver::Create(AController* Controller, FName MessageType, FAIRequestID MessageID, FOnAIMessage const& Delegate)
{
	UBrainComponent* BrainComp = FindBrainComponentHelper(Controller);
	return Create(BrainComp, MessageType, MessageID, Delegate);
}

FAIMessageObserverHandle FAIMessageObserver::Create(APawn* Pawn, FName MessageType, FOnAIMessage const& Delegate)
{
	UBrainComponent* BrainComp = FindBrainComponentHelper(Pawn);
	return Create(BrainComp, MessageType, Delegate);
}

FAIMessageObserverHandle FAIMessageObserver::Create(APawn* Pawn, FName MessageType, FAIRequestID MessageID, FOnAIMessage const& Delegate)
{
	UBrainComponent* BrainComp = FindBrainComponentHelper(Pawn);
	return Create(BrainComp, MessageType, MessageID, Delegate);
}

FAIMessageObserverHandle FAIMessageObserver::Create(UBrainComponent* BrainComp, FName MessageType, FOnAIMessage const& Delegate)
{
	FAIMessageObserverHandle ObserverHandle;
	if (BrainComp)
	{
		FAIMessageObserver* NewObserver = new FAIMessageObserver();
		NewObserver->MessageType = MessageType;
		NewObserver->bFilterByID = false;
		NewObserver->ObserverDelegate = Delegate;
		NewObserver->Register(BrainComp);

		ObserverHandle = MakeShareable(NewObserver);
	}

	return ObserverHandle;
}

FAIMessageObserverHandle FAIMessageObserver::Create(UBrainComponent* BrainComp, FName MessageType, FAIRequestID MessageID, FOnAIMessage const& Delegate)
{
	FAIMessageObserverHandle ObserverHandle;
	if (BrainComp)
	{
		FAIMessageObserver* NewObserver = new FAIMessageObserver();
		NewObserver->MessageType = MessageType;
		NewObserver->MessageID = MessageID;
		NewObserver->bFilterByID = true;
		NewObserver->ObserverDelegate = Delegate;
		NewObserver->Register(BrainComp);

		ObserverHandle = MakeShareable(NewObserver);
	}

	return ObserverHandle;
}

FAIMessageObserver::~FAIMessageObserver()
{
	Unregister();
}

void FAIMessageObserver::Register(UBrainComponent* OwnerComp)
{
	OwnerComp->MessageObservers.Add(this);
	Owner = OwnerComp;
}

void FAIMessageObserver::Unregister()
{
	UBrainComponent* OwnerComp = Owner.Get();
	if (OwnerComp)
	{
		OwnerComp->MessageObservers.RemoveSingle(this);
	}
}

void FAIMessageObserver::OnMessage(const FAIMessage& Message)
{
	if (Message.MessageName == MessageType)
	{
		if (!bFilterByID || Message.RequestID.IsEquivalent(MessageID))
		{
			ObserverDelegate.ExecuteIfBound(Owner.Get(), Message);
		}
	}
}

FString FAIMessageObserver::DescribeObservedMessage() const
{
	FString Description = MessageType.ToString();
	if (bFilterByID)
	{
		Description.AppendChar(TEXT(':'));
		Description.AppendInt(MessageID.GetID());
	}

	return Description;
}

//////////////////////////////////////////////////////////////////////////
// Brain component

UBrainComponent::UBrainComponent(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;
	bDoLogicRestartOnUnlock = false;
}

#if ENABLE_VISUAL_LOG
void UBrainComponent::DescribeSelfToVisLog(FVisualLogEntry* Snapshot) const
{
	const static UEnum* PriorityEnum = StaticEnum<EAIRequestPriority::Type>();

	if (!IsValidChecked(this))
	{
		return;
	}

	FVisualLogStatusCategory StatusCategory;
	StatusCategory.Category = FString::Printf(TEXT("Resource lock: %s"), *ResourceLock.GetLockPriorityName());
	for (int32 LockLevel = 0; LockLevel < int32(EAIRequestPriority::MAX); ++LockLevel)
	{
		StatusCategory.Add(*PriorityEnum->GetNameStringByValue(LockLevel), ResourceLock.IsLockedBy(EAIRequestPriority::Type(LockLevel)) ? TEXT("Locked") : TEXT("Unlocked"));
	}
	Snapshot->Status.Add(StatusCategory);

	if (BlackboardComp)
	{
		BlackboardComp->DescribeSelfToVisLog(Snapshot);
	}
}
#endif // ENABLE_VISUAL_LOG

void UBrainComponent::LockResource(EAIRequestPriority::Type LockSource)
{
	const bool bWasLocked = ResourceLock.IsLocked();
	ResourceLock.SetLock(LockSource);
	if (bWasLocked == false)
	{
		PauseLogic(FString::Printf(TEXT("Locking Resource with source %s"), *ResourceLock.GetLockPriorityName()));
	}
}

void UBrainComponent::ClearResourceLock(EAIRequestPriority::Type LockSource)
{
	ResourceLock.ClearLock(LockSource);

	if (ResourceLock.IsLocked() == false)
	{
		ResumeLogic(TEXT("unlocked"));
	}
}

void UBrainComponent::ForceUnlockResource()
{
	ResourceLock.ForceClearAllLocks();
	ResumeLogic(TEXT("unlocked: forced"));
}

bool UBrainComponent::IsResourceLocked() const
{
	return ResourceLock.IsLocked();
}

void UBrainComponent::OnRegister() 
{
	Super::OnRegister();

	AIOwner = Cast<AAIController>(GetOwner());
}

void UBrainComponent::InitializeComponent()
{
	Super::InitializeComponent();

	// cache blackboard component if owner has one
	// note that it's a valid scenario for this component to not have an owner at all (at least in terms of unittesting)
	AActor* Owner = GetOwner();
	if (Owner)
	{
		BlackboardComp = GetOwner()->FindComponentByClass<UBlackboardComponent>();
		if (BlackboardComp)
		{
			BlackboardComp->CacheBrainComponent(*this);
		}
	}
}

void UBrainComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	if (MessagesToProcess.Num() > 0)
	{
		const int32 NumMessages = MessagesToProcess.Num();
		for (int32 Idx = 0; Idx < NumMessages; Idx++)
		{
			// create a copy of message in case MessagesToProcess is changed during loop
			const FAIMessage MessageCopy(MessagesToProcess[Idx]);

			for (int32 ObserverIndex = 0; ObserverIndex < MessageObservers.Num(); ObserverIndex++)
			{
				MessageObservers[ObserverIndex]->OnMessage(MessageCopy);
			}
		}
		MessagesToProcess.RemoveAt(0, NumMessages, EAllowShrinking::No);
	}
}

void UBrainComponent::CacheBlackboardComponent(UBlackboardComponent* BBComp)
{
	if (BBComp)
	{
		BlackboardComp = BBComp;
	}
}

void UBrainComponent::HandleMessage(const FAIMessage& Message)
{
	MessagesToProcess.Add(Message);
}

void UBrainComponent::RequestLogicRestartOnUnlock()
{
	if (IsResourceLocked())
	{
		UE_VLOG(GetOwner(), LogBrain, Log, TEXT("Scheduling Logic Restart on next braing unlocking"));
		bDoLogicRestartOnUnlock = true;
	}
}

EAILogicResuming::Type UBrainComponent::ResumeLogic(const FString& Reason)
{
	UE_VLOG(GetOwner(), LogBrain, Log, TEXT("Execution updates: RESUMED (%s)"), *Reason);

	if (bDoLogicRestartOnUnlock == true)
	{
		bDoLogicRestartOnUnlock = false;
		RestartLogic();
		// let child implementations not to continue
		return EAILogicResuming::RestartedInstead;
	}

	return EAILogicResuming::Continue;
}

void UBrainComponent::StartLogic()
{

}

void UBrainComponent::RestartLogic()
{

}

void UBrainComponent::StopLogic(const FString& Reason)
{

}

bool UBrainComponent::IsRunning() const 
{ 
	return false; 
}

bool UBrainComponent::IsPaused() const 
{ 
	return false; 
}
