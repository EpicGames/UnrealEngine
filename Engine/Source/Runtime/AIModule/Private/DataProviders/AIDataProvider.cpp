// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataProviders/AIDataProvider.h"
#include "UObject/CoreObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AIDataProvider)

//////////////////////////////////////////////////////////////////////////
// FAIDataProviderValue

bool FAIDataProviderValue::IsMatchingType(FProperty* PropType) const
{
	return true;
}

void FAIDataProviderValue::GetMatchingProperties(TArray<FName>& MatchingProperties) const
{
	if (DataBinding)
	{
		for (FProperty* Prop = DataBinding->GetClass()->PropertyLink; Prop; Prop = Prop->PropertyLinkNext)
		{
			if (Prop->HasAnyPropertyFlags(CPF_Edit))
			{
				continue;
			}

			if (IsMatchingType(Prop))
			{
				MatchingProperties.Add(Prop->GetFName());
			}
		}
	}
}

void FAIDataProviderValue::BindData(const UObject* Owner, int32 RequestId) const
{
	if (DataBinding && ensure(Owner))
	{
		CachedProperty = nullptr;
		DataBinding->BindData(*Owner, RequestId);
		if (!DataField.IsNone())
		{
			FProperty* Prop = DataBinding->GetClass()->FindPropertyByName(DataField);
			if (ensureAlwaysMsgf(Prop != nullptr && IsMatchingType(Prop)
				, TEXT("We expect UEnvQueryNode::PostEditChangeProperty to prevent copying an incompatible provider value on top of an existing one.")))
			{
				CachedProperty = Prop;
			}
		}
	}
}

FString FAIDataProviderValue::ToString() const
{
	return IsDynamic() ? DataBinding->ToString(DataField) : ValueToString();
}

FString FAIDataProviderValue::ValueToString() const
{
	return TEXT("??");
}

//////////////////////////////////////////////////////////////////////////
// FAIDataProviderTypedValue

bool FAIDataProviderTypedValue::IsMatchingType(FProperty* PropType) const
{
	return PropType->GetClass() == PropertyType;
}

bool FAIDataProviderTypedValue::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FCoreObjectVersion::GUID);
	
	UScriptStruct* Struct = FAIDataProviderTypedValue::StaticStruct();
	
	if (Ar.IsLoading() || Ar.IsSaving())
	{
		Struct->SerializeTaggedProperties(Ar, (uint8*)this, Struct, nullptr);
	}

	if (Ar.CustomVer(FCoreObjectVersion::GUID) >= FCoreObjectVersion::FProperties)
	{
		Ar << PropertyType;
	}
	else if (Ar.IsLoading())
	{		
		if (PropertyType_DEPRECATED)
		{
			PropertyType = FFieldClass::GetNameToFieldClassMap().FindRef(PropertyType_DEPRECATED->GetFName());
			PropertyType_DEPRECATED = nullptr;
		}
		else
		{
			PropertyType = nullptr;
		}
	}

	return true;
}

//////////////////////////////////////////////////////////////////////////
// FAIDataProviderStructValue

bool FAIDataProviderStructValue::IsMatchingType(FProperty* PropType) const
{
	FStructProperty* StructProp = CastField<FStructProperty>(PropType);
	if (StructProp)
	{
		// skip inital "struct " 
		FString CPPType = StructProp->GetCPPType(nullptr, CPPF_None).Mid(8);
		return CPPType == StructName;
	}

	return false;
}

//////////////////////////////////////////////////////////////////////////
// FAIDataProviderIntValue

FAIDataProviderIntValue::FAIDataProviderIntValue()
	: DefaultValue(0)
{
	PropertyType = FIntProperty::StaticClass();
}

int32 FAIDataProviderIntValue::GetValue() const
{
	int32* PropValue = GetRawValuePtr<int32>();
	return PropValue ? *PropValue : DefaultValue;
}

FString FAIDataProviderIntValue::ValueToString() const
{
	return TTypeToString<int32>().ToSanitizedString(DefaultValue);
}

//////////////////////////////////////////////////////////////////////////
// FAIDataProviderFloatValue

FAIDataProviderFloatValue::FAIDataProviderFloatValue()
	: DefaultValue(0.0f)
{
	PropertyType = FFloatProperty::StaticClass();
}

float FAIDataProviderFloatValue::GetValue() const
{
	float* PropValue = GetRawValuePtr<float>();
	return PropValue ? *PropValue : DefaultValue;
}

FString FAIDataProviderFloatValue::ValueToString() const
{
	return TTypeToString<float>().ToSanitizedString(DefaultValue);
}

//////////////////////////////////////////////////////////////////////////
// FAIDataProviderBoolValue

FAIDataProviderBoolValue::FAIDataProviderBoolValue()
	: DefaultValue(false)
{
	PropertyType = FBoolProperty::StaticClass();
}

bool FAIDataProviderBoolValue::GetValue() const
{
	bool* PropValue = GetRawValuePtr<bool>();
	return PropValue ? *PropValue : DefaultValue;
}

FString FAIDataProviderBoolValue::ValueToString() const
{
	const FCoreTexts& CoreTexts = FCoreTexts::Get();

	return DefaultValue ? CoreTexts.True.ToString() : CoreTexts.False.ToString();
}

//////////////////////////////////////////////////////////////////////////
// UAIDataProvider

UAIDataProvider::UAIDataProvider(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

void UAIDataProvider::BindData(const UObject& Owner, int32 RequestId)
{
	// empty in base class
}

FString UAIDataProvider::ToString(FName PropName) const
{
	FString ProviderName = GetClass()->GetName();
	int32 SplitIdx = 0;
	const bool bFound = ProviderName.FindChar(TEXT('_'), SplitIdx);
	if (bFound)
	{
		ProviderName.MidInline(SplitIdx + 1, MAX_int32, EAllowShrinking::No);
	}

	ProviderName += TEXT('.');
	ProviderName += PropName.ToString();
	return ProviderName;
}

