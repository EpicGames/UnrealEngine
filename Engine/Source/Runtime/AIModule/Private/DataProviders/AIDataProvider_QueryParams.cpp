// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataProviders/AIDataProvider_QueryParams.h"
#include "EnvironmentQuery/EnvQueryManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AIDataProvider_QueryParams)

void UAIDataProvider_QueryParams::BindData(const UObject& Owner, int32 RequestId)
{
	UEnvQueryManager* QueryManager = UEnvQueryManager::GetCurrent(&Owner);
	if (QueryManager)
	{
		FloatValue = QueryManager->FindNamedParam(RequestId, ParamName);

		// int param was encoded directly in the float value
		IntValue = *((int32*)&FloatValue);

		// bool param was encoded as -1.0f (false) or 1.0f (true) in the float value
		BoolValue = FloatValue > 0.f;
	}
	else
	{
		IntValue = 0;
		FloatValue = 0.0f;
		BoolValue = false;
	}
}

FString UAIDataProvider_QueryParams::ToString(FName PropName) const
{
	return FString::Printf(TEXT("QueryParam.%s"), *ParamName.ToString());
}

