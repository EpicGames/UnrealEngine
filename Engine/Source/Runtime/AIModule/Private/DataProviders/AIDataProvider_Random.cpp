// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataProviders/AIDataProvider_Random.h"
#include "AISystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AIDataProvider_Random)

UAIDataProvider_Random::UAIDataProvider_Random(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Min = 0.f;
	Max = 1.f;
	bInteger = false;
}

void UAIDataProvider_Random::BindData(const UObject& Owner, int32 RequestId)
{
	const float RandNumber = UAISystem::GetRandomStream().GetFraction();
	const float ReturnValue = RandNumber * (Max - Min) + Min;

	IntValue = FMath::RoundToInt(ReturnValue);
	BoolValue = (IntValue != 0);
	
	if (bInteger)
	{		
		FloatValue = float(IntValue);
	}
	else
	{
		FloatValue = ReturnValue;
	}
}

FString UAIDataProvider_Random::ToString(FName PropName) const
{
	return FString::Printf(TEXT("Random number"));
}
