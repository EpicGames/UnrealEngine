// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Contexts/EnvQueryContext_BlueprintBase.h"
#include "GameFramework/Actor.h"
#include "AITypes.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_Actor.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_Point.h"
#include "EnvironmentQuery/EnvQueryManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryContext_BlueprintBase)

namespace
{
	FORCEINLINE bool DoesImplementBPFunction(FName FuncName, const UObject* Ob, const UClass* StopAtClass)
	{
		return (Ob->GetClass()->FindFunctionByName(FuncName)->GetOuter() != StopAtClass);
	}
}

UEnvQueryContext_BlueprintBase::UEnvQueryContext_BlueprintBase(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer), CallMode(UEnvQueryContext_BlueprintBase::InvalidCallMode)
{
	UClass* StopAtClass = UEnvQueryContext_BlueprintBase::StaticClass();	
	bool bImplementsProvideSingleActor = DoesImplementBPFunction(GET_FUNCTION_NAME_CHECKED(UEnvQueryContext_BlueprintBase, ProvideSingleActor), this, StopAtClass);
	bool bImplementsProvideSingleLocation = DoesImplementBPFunction(GET_FUNCTION_NAME_CHECKED(UEnvQueryContext_BlueprintBase, ProvideSingleLocation), this, StopAtClass);
	bool bImplementsProvideActorSet = DoesImplementBPFunction(GET_FUNCTION_NAME_CHECKED(UEnvQueryContext_BlueprintBase, ProvideActorsSet), this, StopAtClass);
	bool bImplementsProvideLocationsSet = DoesImplementBPFunction(GET_FUNCTION_NAME_CHECKED(UEnvQueryContext_BlueprintBase, ProvideLocationsSet), this, StopAtClass);

	int32 ImplementationsCount = 0;
	if (bImplementsProvideSingleActor)
	{
		++ImplementationsCount;
		CallMode = SingleActor;
	}

	if (bImplementsProvideSingleLocation)
	{
		++ImplementationsCount;
		CallMode = SingleLocation;
	}

	if (bImplementsProvideActorSet)
	{
		++ImplementationsCount;
		CallMode = ActorSet;
	}

	if (bImplementsProvideLocationsSet)
	{
		++ImplementationsCount;
		CallMode = LocationSet;
	}

	if (ImplementationsCount != 1)
	{

	}
}

void UEnvQueryContext_BlueprintBase::ProvideContext(FEnvQueryInstance& QueryInstance, FEnvQueryContextData& ContextData) const
{
	UObject* QuerierObject = QueryInstance.Owner.Get();
	if ((QuerierObject == nullptr) || (CallMode == InvalidCallMode))
	{
		return;
	}

	// NOTE: QuerierActor is redundant with QuerierObject and should be removed in the future.  It's here for now for
	// backwards compatibility.
	AActor* QuerierActor = Cast<AActor>(QuerierObject);
	switch (CallMode)
	{
	case SingleActor:
		{
			AActor* ResultingActor = NULL;
			ProvideSingleActor(QuerierObject, QuerierActor, ResultingActor);
			// store the result only if it's a valid actor, otherwise the FEnvQueryContextData will think it succeeded 
			// and proceed with the query
			if (ResultingActor)
			{
				UEnvQueryItemType_Actor::SetContextHelper(ContextData, ResultingActor);
			}
		}
		break;
	case SingleLocation:
		{
			FVector ResultingLocation = FAISystem::InvalidLocation;
			ProvideSingleLocation(QuerierObject, QuerierActor, ResultingLocation);
			// filter out invalid locations as they don't represent valid context
			if (FAISystem::IsValidLocation(ResultingLocation))
			{
				UEnvQueryItemType_Point::SetContextHelper(ContextData, ResultingLocation);
			}
		}
		break;
	case ActorSet:
		{
			TArray<AActor*> ActorSet;
			ProvideActorsSet(QuerierObject, QuerierActor, ActorSet);
			// remove nulls as they don't represent valid information, i.e. null actors are invalid rather than an "is missing" information
			ActorSet.Remove(nullptr);
			if (ActorSet.Num())
			{
				UEnvQueryItemType_Actor::SetContextHelper(ContextData, ActorSet);
			}
		}
		break;
	case LocationSet:
		{
			TArray<FVector> LocationSet;
			ProvideLocationsSet(QuerierObject, QuerierActor, LocationSet);
			// filter out invalid locations as they don't represent valid context
			LocationSet.RemoveAll([](const FVector& Element) { return !FAISystem::IsValidLocation(Element); });
			if (LocationSet.Num())
			{
				UEnvQueryItemType_Point::SetContextHelper(ContextData, LocationSet);
			}
		}
		break;
	default:
		break;
	}
}

UWorld* UEnvQueryContext_BlueprintBase::GetWorld() const
{
	check(GetOuter() != NULL);
	
	UEnvQueryManager* EnvQueryManager = Cast<UEnvQueryManager>(GetOuter());
	if (EnvQueryManager != NULL)
	{
		return EnvQueryManager->GetWorld();
	}

	// Outer should always be UEnvQueryManager* in the game, which implements GetWorld() and therefore makes this
	// a correct world context.  However, in the editor the outer is /Script/AIModule (at compile time), which
	// does not explicitly implement GetWorld().  For that reason, calling GetWorld() generically in that case on the
	// AIModule calls to the base implementation, which causes a blueprint compile warning in the Editor
	// which states that the function isn't safe to call on this (due to requiring WorldContext which it doesn't
	// provide).  Simply returning NULL in this case fixes those erroneous blueprint compile warnings.
	return NULL;
}

