// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Contexts/EnvQueryContext_Item.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryContext_Item)

UEnvQueryContext_Item::UEnvQueryContext_Item(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	// special case, don't set delegate
}

