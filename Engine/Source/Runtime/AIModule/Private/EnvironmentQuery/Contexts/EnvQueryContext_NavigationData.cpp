// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Contexts/EnvQueryContext_NavigationData.h"
#include "EnvironmentQuery/EnvQueryManager.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_Actor.h"
#include "NavigationSystem.h"
#include "NavigationSystemTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryContext_NavigationData)

UEnvQueryContext_NavigationData::UEnvQueryContext_NavigationData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UEnvQueryContext_NavigationData::ProvideContext(FEnvQueryInstance& QueryInstance, FEnvQueryContextData& ContextData) const
{
	const UNavigationSystemV1* NavigationSystem = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
	const ANavigationData* NavigationData = NavigationSystem ? NavigationSystem->GetNavDataForProps(NavAgentProperties) : nullptr;
	
	if (NavigationData)
	{
		UEnvQueryItemType_Actor::SetContextHelper(ContextData, NavigationData);
	}
}
