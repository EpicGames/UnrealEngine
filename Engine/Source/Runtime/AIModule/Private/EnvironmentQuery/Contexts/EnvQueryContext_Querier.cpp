// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Contexts/EnvQueryContext_Querier.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Controller.h"
#include "AISystem.h"
#include "VisualLogger/VisualLogger.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryContext_Querier)

UEnvQueryContext_Querier::UEnvQueryContext_Querier(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

void UEnvQueryContext_Querier::ProvideContext(FEnvQueryInstance& QueryInstance, FEnvQueryContextData& ContextData) const
{
	AActor* QueryOwner = Cast<AActor>(QueryInstance.Owner.Get());
	UE_CVLOG(GET_AI_CONFIG_VAR(bAllowControllersAsEQSQuerier) == false && Cast<AController>(QueryOwner) != nullptr, QueryOwner, LogEQS, Warning, TEXT("Using Controller as query's owner is dangerous since Controller's location is usually not what you expect it to be!"));
	UEnvQueryItemType_Actor::SetContextHelper(ContextData, QueryOwner);
}

