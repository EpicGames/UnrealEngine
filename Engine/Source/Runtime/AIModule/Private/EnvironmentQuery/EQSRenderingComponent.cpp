// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/EQSRenderingComponent.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_VectorBase.h"
#include "Engine/Canvas.h"
#include "EnvironmentQuery/EQSQueryResultSourceInterface.h"
#include "SceneInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EQSRenderingComponent)

static constexpr int32 EQSMaxItemsDrawn = 10000;

namespace FEQSRenderingHelper
{
	FVector ExtractLocation(TSubclassOf<UEnvQueryItemType> ItemType, const TArray<uint8>& RawData, const TArray<FEnvQueryItem>& Items, int32 Index)
	{
		if (Items.IsValidIndex(Index) &&
			ItemType->IsChildOf(UEnvQueryItemType_VectorBase::StaticClass()))
		{
			const UEnvQueryItemType_VectorBase* DefTypeOb = ItemType->GetDefaultObject<UEnvQueryItemType_VectorBase>();
			return DefTypeOb->GetItemLocation(RawData.GetData() + Items[Index].DataOffset);
		}
		return FVector::ZeroVector;
	}
}

//----------------------------------------------------------------------//
// FEQSSceneProxy
//----------------------------------------------------------------------//
const FVector3f FEQSSceneProxy::ItemDrawRadius(30.f,30.f,30.f);

SIZE_T FEQSSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

FEQSSceneProxy::FEQSSceneProxy(const UPrimitiveComponent& InComponent, const FString& InViewFlagName, const TArray<FSphere>& InSpheres, const TArray<FText3d>& InTexts)
	: FDebugRenderSceneProxy(&InComponent)
	, ActorOwner(nullptr)
	, QueryDataSource(nullptr)
{
	DrawType = SolidAndWireMeshes;
	TextWithoutShadowDistance = 1500;
	ViewFlagIndex = uint32(FEngineShowFlags::FindIndexByName(*InViewFlagName));
	ViewFlagName = InViewFlagName;
	bWantsSelectionOutline = false;

	Spheres = InSpheres;
	Texts = InTexts;
	
	const UEQSRenderingComponent* MyRenderComp = Cast<const UEQSRenderingComponent>(&InComponent);
	bDrawOnlyWhenSelected = MyRenderComp && MyRenderComp->bDrawOnlyWhenSelected;

	ActorOwner = InComponent.GetOwner();
	QueryDataSource = Cast<const IEQSQueryResultSourceInterface>(ActorOwner);
	if (QueryDataSource == nullptr)
	{
		QueryDataSource = Cast<const IEQSQueryResultSourceInterface>(&InComponent);
	}

#if  USE_EQS_DEBUGGER 
	if (Spheres.Num() == 0 && Texts.Num() == 0 && QueryDataSource != nullptr)
	{
		TArray<EQSDebug::FDebugHelper> DebugItems;
		FEQSSceneProxy::CollectEQSData(&InComponent, QueryDataSource, Spheres, Texts, DebugItems);
	}
#endif
}

#if  USE_EQS_DEBUGGER
void FEQSSceneProxy::CollectEQSData(const UPrimitiveComponent* InComponent, const IEQSQueryResultSourceInterface* InQueryDataSource, TArray<FSphere>& Spheres, TArray<FText3d>& Texts, TArray<EQSDebug::FDebugHelper>& DebugItems)
{
	AActor* ActorOwner = InComponent ? InComponent->GetOwner() : nullptr;
	IEQSQueryResultSourceInterface* QueryDataSource = const_cast<IEQSQueryResultSourceInterface*>(InQueryDataSource);
	if (QueryDataSource == nullptr)
	{
		QueryDataSource = Cast<IEQSQueryResultSourceInterface>(ActorOwner);
		if (QueryDataSource == nullptr)
		{
			QueryDataSource = Cast<IEQSQueryResultSourceInterface>(const_cast<UPrimitiveComponent*>(InComponent));
			if (QueryDataSource == nullptr)
			{
				return;
			}
		}
	}

	const FEnvQueryResult* ResultItems = QueryDataSource->GetQueryResult();
	const FEnvQueryInstance* QueryInstance = QueryDataSource->GetQueryInstance();

	FEQSSceneProxy::CollectEQSData(ResultItems, QueryInstance,
		QueryDataSource->GetHighlightRangePct(), QueryDataSource->GetShouldDrawFailedItems(),
		Spheres, Texts, DebugItems);

	if (ActorOwner && Spheres.Num() > EQSMaxItemsDrawn)
	{
		UE_VLOG(ActorOwner, LogEQS, Warning, TEXT("EQS drawing: too much items to draw! Drawing first %d from set of %d")
			, EQSMaxItemsDrawn
			, Spheres.Num()
			);
	}
}

void FEQSSceneProxy::CollectEQSData(const FEnvQueryResult* ResultItems, const FEnvQueryInstance* QueryInstance, float HighlightRangePct, bool ShouldDrawFailedItems, TArray<FSphere>& Spheres, TArray<FText3d>& Texts, TArray<EQSDebug::FDebugHelper>& DebugItems)
{
	if (ResultItems == nullptr)
	{
		if (QueryInstance == nullptr)
		{
			return;
		}
		else
		{
			ResultItems = QueryInstance;
		}
	}

	// using "mid-results" requires manual normalization
	const bool bUseMidResults = QueryInstance && (QueryInstance->Items.Num() < QueryInstance->DebugData.DebugItems.Num());
	// no point in checking if QueryInstance != null since bUseMidResults == true guarantees that, PVS-Studio doesn't
	// understand that invariant though, and we need to disable V595:
	const TArray<FEnvQueryItem>& Items = bUseMidResults ? QueryInstance->DebugData.DebugItems : ResultItems->Items;
	const TArray<uint8>& RawData = bUseMidResults ? QueryInstance->DebugData.RawData : ResultItems->RawData;
	const int32 ItemCountLimit = FMath::Clamp(Items.Num(), 0, EQSMaxItemsDrawn);
	const bool bNoTestsPerformed = QueryInstance != nullptr && QueryInstance->CurrentTest <= 0;
	
	const bool bSingleItemResult = QueryInstance != nullptr && QueryInstance->DebugData.bSingleItemResult;

	float MinScore = 0.f;
	float MaxScore = -BIG_NUMBER;
	if (bUseMidResults || HighlightRangePct < 1.0f)
	{
		const FEnvQueryItem* ItemInfo = Items.GetData();
		for (int32 ItemIndex = 0; ItemIndex < Items.Num(); ItemIndex++, ItemInfo++)
		{
			if (ItemInfo->IsValid())
			{
				MinScore = FMath::Min(MinScore, ItemInfo->Score);
				MaxScore = FMath::Max(MaxScore, ItemInfo->Score);
			}
		}
	}
	
	const float ScoreNormalizer = bUseMidResults && (MaxScore != MinScore) ? (1.f / (MaxScore - MinScore)) : 1.f;
	const float HighlightThreshold = (HighlightRangePct < 1.0f) ? (MaxScore * HighlightRangePct) : FLT_MAX;

	if (bSingleItemResult == false)
	{
		for (int32 ItemIndex = 0; ItemIndex < ItemCountLimit; ++ItemIndex)
		{
			if (Items[ItemIndex].IsValid())
			{
				const float NormalizedScore = bNoTestsPerformed ? 1 : (Items[ItemIndex].Score - MinScore) * ScoreNormalizer;
				const bool bLowRadius = (HighlightThreshold < FLT_MAX) && (bNoTestsPerformed || (Items[ItemIndex].Score < HighlightThreshold));
				const float Radius = ItemDrawRadius.X * (bLowRadius ? 0.2f : 1.0f);
				const FVector Loc = FEQSRenderingHelper::ExtractLocation(ResultItems->ItemType, RawData, Items, ItemIndex);
				Spheres.Add(FSphere(Radius, Loc, bNoTestsPerformed == false
					? FLinearColor(FColor::MakeRedToGreenColorFromScalar(NormalizedScore))
					: FLinearColor(0.2f, 1.0f, 1.0f, 1)));

				DebugItems.Add(EQSDebug::FDebugHelper(Loc, Radius));

				const FString Label = bNoTestsPerformed ? TEXT("") : FString::Printf(TEXT("%.2f"), Items[ItemIndex].Score);
				Texts.Add(FText3d(Label, Loc, FLinearColor::White));
			}
		}
	}
	else if (ItemCountLimit > 0)
	{
		if (Items[0].IsValid())
		{
			const float Score = Items[0].Score;
			constexpr bool bLowRadius = false;
			const float Radius = ItemDrawRadius.X * (bLowRadius ? 0.2f : 1.0f);
			const FVector Loc = FEQSRenderingHelper::ExtractLocation(ResultItems->ItemType, RawData, Items, 0);
			Spheres.Add(FSphere(Radius, Loc, FLinearColor(0.0f, 1.0f, 0.12f, 1)));

			DebugItems.Add(EQSDebug::FDebugHelper(Loc, Radius));

			const FString Label = FString::Printf(TEXT("Winner %.2f"), Score);
			Texts.Add(FText3d(Label, Loc, FLinearColor::White));
		}

		for (int32 ItemIndex = 1; ItemIndex < ItemCountLimit; ++ItemIndex)
		{
			if (Items[ItemIndex].IsValid())
			{
				const float Score = bNoTestsPerformed ? 1 : Items[ItemIndex].Score;
				const bool bLowRadius = (HighlightThreshold < FLT_MAX) && (bNoTestsPerformed || (Items[ItemIndex].Score < HighlightThreshold));
				const float Radius = ItemDrawRadius.X * (bLowRadius ? 0.2f : 1.0f);
				const FVector Loc = FEQSRenderingHelper::ExtractLocation(ResultItems->ItemType, RawData, Items, ItemIndex);
				Spheres.Add(FSphere(Radius, Loc, FLinearColor(0.0, 0.2f, 0.025f, 1)));

				DebugItems.Add(EQSDebug::FDebugHelper(Loc, Radius));
				
				const FString Label = bNoTestsPerformed ? TEXT("") : FString::Printf(TEXT("%.2f"), Score);
				Texts.Add(FText3d(Label, Loc, FLinearColor::White));
			}
		}
	}

	if (ShouldDrawFailedItems && QueryInstance)
	{
		const FEnvQueryDebugData& InstanceDebugData = QueryInstance->DebugData;
		const TArray<FEnvQueryItem>& DebugQueryItems = InstanceDebugData.DebugItems;
		const TArray<FEnvQueryItemDetails>& Details = InstanceDebugData.DebugItemDetails;

		const int32 DebugItemCountLimit = DebugQueryItems.Num() == Details.Num() ? FMath::Clamp(DebugQueryItems.Num(), 0, EQSMaxItemsDrawn) : 0;

		for (int32 ItemIndex = 0; ItemIndex < DebugItemCountLimit; ++ItemIndex)
		{
			if (DebugQueryItems[ItemIndex].IsValid())
			{
				continue;
			}

			const bool bLowRadius = (HighlightThreshold < FLT_MAX) && (bNoTestsPerformed || (Items[ItemIndex].Score < HighlightThreshold));
			const float Radius = ItemDrawRadius.X * (bLowRadius ? 0.2f : 1.0f);
			const FVector Loc = FEQSRenderingHelper::ExtractLocation(QueryInstance->ItemType, InstanceDebugData.RawData, DebugQueryItems, ItemIndex);
			Spheres.Add(FSphere(Radius, Loc, FLinearColor(0.0f, 0.0f, 0.6f, 0.6f)));

			auto& DebugHelper = DebugItems[DebugItems.Add(EQSDebug::FDebugHelper(Loc, Radius))];
			DebugHelper.AdditionalInformation = Details[ItemIndex].FailedDescription;
			if (Details[ItemIndex].FailedTestIndex != INDEX_NONE)
			{
				DebugHelper.FailedTestIndex = Details[ItemIndex].FailedTestIndex;
				DebugHelper.FailedScore = Details[ItemIndex].TestResults[DebugHelper.FailedTestIndex];

				const FString Label = InstanceDebugData.PerformedTestNames[DebugHelper.FailedTestIndex] + FString::Printf(TEXT("(%d)"), DebugHelper.FailedTestIndex);
				Texts.Add(FText3d(Label, Loc, FLinearColor::White));
			}
		}
	}
}

void FEQSRenderingDebugDrawDelegateHelper::DrawDebugLabels(UCanvas* Canvas, APlayerController* PC)
{
	if ( !ActorOwner
		 || (ActorOwner->IsSelected() == false && bDrawOnlyWhenSelected == true)
		 || (QueryDataSource && QueryDataSource->GetShouldDebugDrawLabels() == false))
	{
		return;
	}

	FDebugDrawDelegateHelper::DrawDebugLabels(Canvas, PC);
}
#endif //USE_EQS_DEBUGGER

bool FEQSSceneProxy::SafeIsActorSelected() const
{
	if(ActorOwner)
	{
		return ActorOwner->IsSelected();
	}

	return false;
}

FPrimitiveViewRelevance FEQSSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	FPrimitiveViewRelevance Result;
	Result.bDrawRelevance = View->Family->EngineShowFlags.GetSingleFlag(ViewFlagIndex) && IsShown(View) 
		&& (!bDrawOnlyWhenSelected || SafeIsActorSelected());
	Result.bDynamicRelevance = true;
	// ideally the TranslucencyRelevance should be filled out by the material, here we do it conservative
	Result.bSeparateTranslucency = Result.bNormalTranslucency = IsShown(View);
	return Result;
}

//----------------------------------------------------------------------//
// UEQSRenderingComponent
//----------------------------------------------------------------------//
UEQSRenderingComponent::UEQSRenderingComponent(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer)
	, DrawFlagName("GameplayDebug")
	, bDrawOnlyWhenSelected(true)
{
}

#if UE_ENABLE_DEBUG_DRAWING && USE_EQS_DEBUGGER
FDebugRenderSceneProxy* UEQSRenderingComponent::CreateDebugSceneProxy()
{
	FEQSSceneProxy* NewSceneProxy = new FEQSSceneProxy(*this, DrawFlagName, DebugDataSolidSpheres, DebugDataTexts);
  	EQSRenderingDebugDrawDelegateHelper.SetupFromProxy(NewSceneProxy);
	return NewSceneProxy;
}
#endif

void UEQSRenderingComponent::ClearStoredDebugData()
{
	DebugDataSolidSpheres.Reset();
	DebugDataTexts.Reset();
#if  USE_EQS_DEBUGGER
	EQSRenderingDebugDrawDelegateHelper.Reset();
#endif
	MarkRenderStateDirty();
}

#if  USE_EQS_DEBUGGER || ENABLE_VISUAL_LOG
void UEQSRenderingComponent::StoreDebugData(const EQSDebug::FQueryData& DebugData)
{
	DebugDataSolidSpheres = DebugData.SolidSpheres;
	DebugDataTexts = DebugData.Texts;
	MarkRenderStateDirty();
}
#endif  // USE_EQS_DEBUGGER || ENABLE_VISUAL_LOG

FBoxSphereBounds UEQSRenderingComponent::CalcBounds(const FTransform& LocalToWorld) const
{
#if USE_EQS_DEBUGGER 
	TArray<FDebugRenderSceneProxy::FSphere> Spheres;
	Spheres.Append(DebugDataSolidSpheres);
	const IEQSQueryResultSourceInterface* QueryDataSource = Cast<const IEQSQueryResultSourceInterface>(GetOwner());
	if (QueryDataSource == nullptr)
	{
		QueryDataSource = Cast<const IEQSQueryResultSourceInterface>(this);
	}

	if (QueryDataSource != nullptr)
	{
		TArray<EQSDebug::FDebugHelper> DebugItems;

		TArray<FDebugRenderSceneProxy::FText3d> Texts;
		FEQSSceneProxy::CollectEQSData(this, QueryDataSource, Spheres, Texts, DebugItems);	
	}

	if (Spheres.Num())
	{
		FBoxSphereBounds::Builder DebugBoundsBuilder;
		for (int32 Index = 0; Index < Spheres.Num(); ++Index)
		{
			DebugBoundsBuilder += FSphere(Spheres[Index].Location, Spheres[Index].Radius);
		}

		return DebugBoundsBuilder;
	}
#endif

	static FSphere BoundingSphere(FVector::ZeroVector, 0.f);
	return FBoxSphereBounds(BoundingSphere).TransformBy(LocalToWorld);
}

//----------------------------------------------------------------------//
// UEQSQueryResultSourceInterface
//----------------------------------------------------------------------//
UEQSQueryResultSourceInterface::UEQSQueryResultSourceInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

