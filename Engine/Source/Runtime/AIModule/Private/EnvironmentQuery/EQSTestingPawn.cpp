// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/EQSTestingPawn.h"
#include "UObject/ConstructorHelpers.h"
#include "AISystem.h"
#include "Components/CapsuleComponent.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "EnvironmentQuery/EnvQuery.h"
#include "EnvironmentQuery/EnvQueryManager.h"
#include "Engine/Texture2D.h"
#include "EnvironmentQuery/EQSRenderingComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "BehaviorTree/BlackboardComponent.h"

#if WITH_EDITORONLY_DATA
#include "Components/ArrowComponent.h"
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
#include "Misc/TransactionObjectEvent.h"
#include "Editor/EditorEngine.h"
extern UNREALED_API UEditorEngine* GEditor;
#endif // WITH_EDITOR

#include "Engine/Selection.h"
#include "Components/BillboardComponent.h"
#include "EngineUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EQSTestingPawn)

//----------------------------------------------------------------------//
// AEQSTestingPawn
//----------------------------------------------------------------------//
AEQSTestingPawn::AEQSTestingPawn(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, TimeLimitPerStep(-1)
	, StepToDebugDraw(0)
	, bDrawLabels(true)
	, bDrawFailedItems(true)
	, bReRunQueryOnlyOnFinishedMove(true)
	, bRunQueryOnSelectionChanged(false)
	, QueryingMode(EEnvQueryRunMode::AllMatching)
{
	static FName CollisionProfileName("NoCollision");
	GetCapsuleComponent()->SetCollisionProfileName(CollisionProfileName);

	NavAgentProperties = FNavAgentProperties::DefaultProperties;

#if WITH_EDITORONLY_DATA
	EdRenderComp = CreateEditorOnlyDefaultSubobject<UEQSRenderingComponent>(TEXT("EQSRender"));
	if (EdRenderComp)
	{
		EdRenderComp->SetCollisionEnabled(ECollisionEnabled::NoCollision);
	}

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		UArrowComponent* ArrowComp = FindComponentByClass<UArrowComponent>();
		if (ArrowComp != NULL)
		{
			ArrowComp->SetRelativeScale3D(FVector(2, 2, 2));
			ArrowComp->bIsScreenSizeScaled = true;
		}

		UBillboardComponent* SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));
		if (!IsRunningCommandlet() && (SpriteComponent != nullptr))
		{
			struct FConstructorStatics
			{
				ConstructorHelpers::FObjectFinderOptional<UTexture2D> TextureObject;
				FName ID_Misc;
				FText NAME_Misc;
				FConstructorStatics()
					: TextureObject(TEXT("/Engine/EditorResources/S_Pawn"))
					, ID_Misc(TEXT("Misc"))
					, NAME_Misc(NSLOCTEXT("SpriteCategory", "Misc", "Misc"))
				{
				}
			};
			static FConstructorStatics ConstructorStatics;

			SpriteComponent->Sprite = ConstructorStatics.TextureObject.Get();
			SpriteComponent->SetRelativeScale3D(FVector(1, 1, 1));
			SpriteComponent->bHiddenInGame = true;
			//SpriteComponent->Mobility = EComponentMobility::Static;
			SpriteComponent->SpriteInfo.Category = ConstructorStatics.ID_Misc;
			SpriteComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Misc;
			SpriteComponent->SetupAttachment(RootComponent);
			SpriteComponent->bIsScreenSizeScaled = true;
		}
	}
#endif // WITH_EDITORONLY_DATA

	// Default to no tick function, but if we set 'never ticks' to false (so there is a tick function) it is enabled by default
	PrimaryActorTick.bCanEverTick = false;
	PrimaryActorTick.bStartWithTickEnabled = true;

	if (GetCharacterMovement())
	{
		GetCharacterMovement()->DefaultLandMovementMode = MOVE_None;
	}

#if WITH_EDITOR
	if (HasAnyFlags(RF_ClassDefaultObject) && GetClass() == StaticClass())
	{
		USelection::SelectObjectEvent.AddStatic(&AEQSTestingPawn::OnEditorSelectionChanged);
		USelection::SelectionChangedEvent.AddStatic(&AEQSTestingPawn::OnEditorSelectionChanged);
	}
#endif // WITH_EDITOR
}

float AEQSTestingPawn::GetHighlightRangePct() const
{
	return (HighlightMode == EEnvQueryHightlightMode::Best25Pct) ? 0.75f : (HighlightMode == EEnvQueryHightlightMode::Best5Pct) ? 0.95f : 1.0f;
}

const FNavAgentProperties& AEQSTestingPawn::GetNavAgentPropertiesRef() const 
{
	return NavAgentProperties;
}

#if WITH_EDITOR
void AEQSTestingPawn::OnEditorSelectionChanged(UObject* NewSelection)
{
	TArray<AEQSTestingPawn*> SelectedPawns;
	AEQSTestingPawn* SelectedPawn = Cast<AEQSTestingPawn>(NewSelection);
	if (SelectedPawn)
	{
		SelectedPawns.Add(Cast<AEQSTestingPawn>(NewSelection));
	}
	else 
	{
		USelection* Selection = Cast<USelection>(NewSelection);
		if (Selection != NULL)
		{
			Selection->GetSelectedObjects<AEQSTestingPawn>(SelectedPawns);
		}
	}

	for (AEQSTestingPawn* EQSPawn : SelectedPawns)
	{
		if (EQSPawn->QueryTemplate != nullptr && EQSPawn->QueryInstance.IsValid() == false)
		{
			EQSPawn->QueryTemplate->CollectQueryParams(*EQSPawn, EQSPawn->QueryConfig);
			EQSPawn->RunEQSQuery();
		}
	}

	if (UWorld* SelectionWorld = NewSelection ? NewSelection->GetWorld() : nullptr)
	{
		for (TActorIterator<AEQSTestingPawn> EQSPawnIterator(SelectionWorld); EQSPawnIterator; ++EQSPawnIterator)
		{
			if (!SelectedPawns.Contains(*EQSPawnIterator) && EQSPawnIterator->bRunQueryOnSelectionChanged)
			{
				EQSPawnIterator->QueryTemplate->CollectQueryParams(**EQSPawnIterator, EQSPawnIterator->QueryConfig);
				EQSPawnIterator->RunEQSQuery();
			}
		}
	}
}
#endif // WITH_EDITOR

void AEQSTestingPawn::TickActor(float DeltaTime, enum ELevelTick TickType, FActorTickFunction& ThisTickFunction)
{
	Super::Tick(DeltaTime);

	if (QueryTemplate && !QueryInstance.IsValid())
	{
		UAISystem* AISys = UAISystem::GetCurrentSafe(GetWorld());
		if (AISys)
		{
			RunEQSQuery();
		}
	}

	if (QueryInstance.IsValid() && QueryInstance->IsFinished() == false)
	{
		MakeOneStep();
	}
}

void AEQSTestingPawn::PostLoad() 
{
	Super::PostLoad();

	UBillboardComponent* SpriteComponent = FindComponentByClass<UBillboardComponent>();
	if (SpriteComponent != NULL)
	{
		SpriteComponent->bHiddenInGame = !bShouldBeVisibleInGame;
	}

	if (QueryParams.Num() > 0)
	{
		// handle legacy props reading
		for (FEnvNamedValue& NamedValue : QueryParams)
		{
			if (uint8(NamedValue.ParamType) >= uint8(EAIParamType::MAX))
			{
				NamedValue.ParamType = EAIParamType::Float;
			}
		}

		FAIDynamicParam::GenerateConfigurableParamsFromNamedValues(*this, QueryConfig, QueryParams);
		QueryParams.Empty();
	}

	UWorld* World = GetWorld();
	PrimaryActorTick.bCanEverTick = World && ((World->IsGameWorld() == false) || bTickDuringGame);

	if (PrimaryActorTick.bCanEverTick == false)
	{
		// Also disable components that may tick
		if (GetCharacterMovement())
		{
			GetCharacterMovement()->PrimaryComponentTick.bCanEverTick = false;
		}
		if (GetMesh())
		{
			GetMesh()->PrimaryComponentTick.bCanEverTick = false;
		}
	}
}

void AEQSTestingPawn::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	const AEQSTestingPawn* This = CastChecked<AEQSTestingPawn>(InThis);
	if (This->QueryInstance)
	{
		Collector.AddPropertyReferencesWithStructARO(FEnvQueryInstance::StaticStruct(), This->QueryInstance.Get(), This);
	}

	Super::AddReferencedObjects(InThis, Collector);
}

void AEQSTestingPawn::RunEQSQuery()
{
	if (HasAnyFlags(RF_ClassDefaultObject) == true)
	{
		return;
	}

	Reset();
	
	// make one step if TimeLimitPerStep > 0.f, else all steps
	do
	{
		MakeOneStep();
	} while (TimeLimitPerStep <= 0.f && QueryInstance.IsValid() == true && QueryInstance->IsFinished() == false);
}

void AEQSTestingPawn::Reset()
{
	QueryInstance = NULL;
	StepToDebugDraw = 0;
	StepResults.Reset();

	if (GetController() == nullptr)
	{
		SpawnDefaultController();
	}
}

void AEQSTestingPawn::MakeOneStep()
{
	UEnvQueryManager* EQS = UEnvQueryManager::GetCurrent(GetWorld());
	if (EQS == NULL)
	{
		return;
	}

	if (QueryInstance.IsValid() == false && QueryTemplate != NULL)
	{
		UBlackboardComponent* BlackboardComponent = FindComponentByClass<UBlackboardComponent>();
		FEnvQueryRequest QueryRequest(QueryTemplate, this);
		for (FAIDynamicParam& RuntimeParam : QueryConfig)
		{
			if (ensureMsgf(RuntimeParam.BBKey.IsSet() == false || BlackboardComponent, TEXT("BBKey.IsSet but no BlackboardComponent provided")))
			{
				QueryRequest.SetDynamicParam(RuntimeParam, BlackboardComponent);
			}
		}
		QueryInstance = EQS->PrepareQueryInstance(QueryRequest, QueryingMode);
		if (QueryInstance.IsValid())
		{
			EQS->RegisterExternalQuery(QueryInstance);
		}
	}

	// possible still not valid 
	if (QueryInstance.IsValid() == true && QueryInstance->IsFinished() == false)
	{
		QueryInstance->ExecuteOneStep(TimeLimitPerStep);
		StepResults.Add(*(QueryInstance.Get()));

		if (QueryInstance->IsFinished())
		{
			StepToDebugDraw = StepResults.Num() - 1;
			UpdateDrawing();
		}
	}
}

void AEQSTestingPawn::UpdateDrawing()
{
#if WITH_EDITORONLY_DATA
	if (HasAnyFlags(RF_ClassDefaultObject) == true)
	{
		return;
	}

	UBillboardComponent* SpriteComponent = FindComponentByClass<UBillboardComponent>();
	if (SpriteComponent != NULL)
	{
		SpriteComponent->MarkRenderStateDirty();
	}	

	if (EdRenderComp != NULL && EdRenderComp->GetVisibleFlag())
	{
		EdRenderComp->MarkRenderStateDirty();

#if WITH_EDITOR
		if (GEditor != NULL)
		{
			GEditor->RedrawLevelEditingViewports();
		}
#endif // WITH_EDITOR
	}
#endif // WITH_EDITORONLY_DATA
}

const FEnvQueryResult* AEQSTestingPawn::GetQueryResult() const 
{
	return StepResults.IsValidIndex(StepToDebugDraw) ? &StepResults[StepToDebugDraw] : nullptr;
}

const FEnvQueryInstance* AEQSTestingPawn::GetQueryInstance() const 
{
	return StepResults.IsValidIndex(StepToDebugDraw) ? &StepResults[StepToDebugDraw] : nullptr;
}

#if WITH_EDITOR

void AEQSTestingPawn::OnPropertyChanged(const FName PropName)
{
	static const FName NAME_QueryTemplate = GET_MEMBER_NAME_CHECKED(AEQSTestingPawn, QueryTemplate);
	static const FName NAME_StepToDebugDraw = GET_MEMBER_NAME_CHECKED(AEQSTestingPawn, StepToDebugDraw);
	static const FName NAME_QueryConfig = GET_MEMBER_NAME_CHECKED(AEQSTestingPawn, QueryConfig);
	static const FName NAME_ShouldBeVisibleInGame = GET_MEMBER_NAME_CHECKED(AEQSTestingPawn, bShouldBeVisibleInGame);
	static const FName NAME_QueryingMode = GET_MEMBER_NAME_CHECKED(AEQSTestingPawn, QueryingMode);
	static const FName NAME_bRunQueryOnSelectionChanged = GET_MEMBER_NAME_CHECKED(AEQSTestingPawn, bRunQueryOnSelectionChanged);

	if (PropName == NAME_QueryTemplate || PropName == NAME_QueryConfig)
	{
		if (QueryTemplate)
		{
			QueryTemplate->CollectQueryParams(*this, QueryConfig);
		}

		RunEQSQuery();
	}
	else if (PropName == NAME_StepToDebugDraw)
	{
		StepToDebugDraw  = FMath::Clamp(StepToDebugDraw, 0, StepResults.Num() - 1 );
		UpdateDrawing();
	}
	else if (PropName == GET_MEMBER_NAME_CHECKED(AEQSTestingPawn, bDrawFailedItems) ||
		PropName == GET_MEMBER_NAME_CHECKED(AEQSTestingPawn, HighlightMode))
	{
		UpdateDrawing();
	}
	else if (PropName == NAME_ShouldBeVisibleInGame)
	{
		UBillboardComponent* SpriteComponent = FindComponentByClass<UBillboardComponent>();
		if (SpriteComponent != NULL)
		{
			SpriteComponent->bHiddenInGame = !bShouldBeVisibleInGame;
		}
	}
	else if (PropName == NAME_QueryingMode)
	{
		RunEQSQuery();
	}
	else if (PropName == NAME_bRunQueryOnSelectionChanged)
	{
		if (bRunQueryOnSelectionChanged)
		{
			EdRenderComp->bDrawOnlyWhenSelected = false;
		}
		UpdateDrawing();
	}
}

void AEQSTestingPawn::PostEditChangeProperty( FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property != nullptr)
	{
		OnPropertyChanged(PropertyChangedEvent.MemberProperty->GetFName());
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void AEQSTestingPawn::PostEditMove(bool bFinished)
{
	Super::PostEditMove(bFinished);

	if (bFinished || !bReRunQueryOnlyOnFinishedMove)
	{
		RunEQSQuery();
	}
}

void AEQSTestingPawn::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);

	if (TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		if (TransactionEvent.GetChangedProperties().Num() > 0)
		{
			// targeted update
			for (const FName PropertyName : TransactionEvent.GetChangedProperties())
			{
				OnPropertyChanged(PropertyName);
			}
		}
		else
		{
			// fallback - make sure the results are up to date
			RunEQSQuery();
		}
	}
}

#endif // WITH_EDITOR

