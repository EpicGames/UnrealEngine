// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/EnvQuery.h"
#include "UObject/UnrealType.h"
#include "DataProviders/AIDataProvider.h"
#include "DataProviders/AIDataProvider_QueryParams.h"
#include "EnvironmentQuery/EnvQueryGenerator.h"
#include "EnvironmentQuery/EnvQueryTest.h"
#include "EnvironmentQuery/EnvQueryOption.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQuery)

namespace FEQSParamsExporter
{
	bool HasNamedValue(const FName& ParamName, const TArray<FAIDynamicParam>& NamedValues)
	{
		for (int32 ValueIndex = 0; ValueIndex < NamedValues.Num(); ValueIndex++)
		{
			if (NamedValues[ValueIndex].ParamName == ParamName)
			{
				return true;
			}
		}

		return false;
	}

	void AddNamedValue(UObject& QueryOwner, const FName& ParamName, const EAIParamType ParamType, float Value, TArray<FAIDynamicParam>& NamedValues, TArray<FName>& RequiredParams) 
	{
		check(ParamName.IsNone() == false);

		if (HasNamedValue(ParamName, NamedValues) == false)
		{
			FAIDynamicParam& NewValue = NamedValues[NamedValues.Add(FAIDynamicParam())];
			NewValue.ParamName = ParamName;
			NewValue.ParamType = ParamType;
			NewValue.Value = Value;
			NewValue.ConfigureBBKey(QueryOwner);
		}

		RequiredParams.AddUnique(ParamName);
	}

#define GET_STRUCT_NAME_CHECKED(StructName) \
	((void)sizeof(StructName), TEXT(#StructName))

	void AddNamedValuesFromObject(UObject& QueryOwner, const UEnvQueryNode& QueryNode, TArray<FAIDynamicParam>& NamedValues, TArray<FName>& RequiredParams)
	{
		for (FProperty* TestProperty = QueryNode.GetClass()->PropertyLink; TestProperty; TestProperty = TestProperty->PropertyLinkNext)
		{
			FArrayProperty* TestArray = CastField<FArrayProperty>(TestProperty);
			if (TestArray)
			{
				FObjectPropertyBase* ArrayInnerProperty = CastField<FObjectPropertyBase>(TestArray->Inner);
				if (ArrayInnerProperty == nullptr)
				{
					continue;
				}

				if (ArrayInnerProperty->PropertyClass == nullptr || ArrayInnerProperty->PropertyClass->IsChildOf<UEnvQueryNode>() == false)
				{
					continue;
				}

				FScriptArrayHelper ArrayHelper(TestArray, TestArray->ContainerPtrToValuePtr<void>(&QueryNode));

				for (int32 SubNodeIndex = 0; SubNodeIndex < ArrayHelper.Num(); ++SubNodeIndex)
				{
					const UEnvQueryNode** SubNode = reinterpret_cast<const UEnvQueryNode**>(ArrayHelper.GetRawPtr(SubNodeIndex));
					if (SubNode && *SubNode)
					{
						AddNamedValuesFromObject(QueryOwner, **SubNode, NamedValues, RequiredParams);	
					}
				}
			}

			FStructProperty* TestStruct = CastField<FStructProperty>(TestProperty);
			if (TestStruct == NULL)
			{
				continue;
			}

			const FString TypeDesc = TestStruct->GetCPPType(NULL, CPPF_None);
			if (TypeDesc.Contains(GET_STRUCT_NAME_CHECKED(FAIDataProviderIntValue)))
			{
				const FAIDataProviderIntValue* PropertyValue = TestStruct->ContainerPtrToValuePtr<FAIDataProviderIntValue>(&QueryNode);
				const UAIDataProvider_QueryParams* QueryParamProvider = PropertyValue ? Cast<const UAIDataProvider_QueryParams>(PropertyValue->DataBinding) : nullptr;
				if (QueryParamProvider && !QueryParamProvider->ParamName.IsNone())
				{
					AddNamedValue(QueryOwner, QueryParamProvider->ParamName, EAIParamType::Int, *((float*)&PropertyValue->DefaultValue), NamedValues, RequiredParams);
				}
			}
			else if (TypeDesc.Contains(GET_STRUCT_NAME_CHECKED(FAIDataProviderFloatValue)))
			{
				const FAIDataProviderFloatValue* PropertyValue = TestStruct->ContainerPtrToValuePtr<FAIDataProviderFloatValue>(&QueryNode);
				const UAIDataProvider_QueryParams* QueryParamProvider = PropertyValue ? Cast<const UAIDataProvider_QueryParams>(PropertyValue->DataBinding) : nullptr;
				if (QueryParamProvider && !QueryParamProvider->ParamName.IsNone())
				{
					AddNamedValue(QueryOwner, QueryParamProvider->ParamName, EAIParamType::Float, PropertyValue->DefaultValue, NamedValues, RequiredParams);
				}
			}
			else if (TypeDesc.Contains(GET_STRUCT_NAME_CHECKED(FAIDataProviderBoolValue)))
			{
				const FAIDataProviderBoolValue* PropertyValue = TestStruct->ContainerPtrToValuePtr<FAIDataProviderBoolValue>(&QueryNode);
				const UAIDataProvider_QueryParams* QueryParamProvider = PropertyValue ? Cast<const UAIDataProvider_QueryParams>(PropertyValue->DataBinding) : nullptr;
				if (QueryParamProvider && !QueryParamProvider->ParamName.IsNone())
				{
					AddNamedValue(QueryOwner, QueryParamProvider->ParamName, EAIParamType::Bool, PropertyValue->DefaultValue ? 1.0f : -1.0f, NamedValues, RequiredParams);
				}
			}
		}
	}

#undef GET_STRUCT_NAME_CHECKED
}

UEnvQuery::UEnvQuery(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

void UEnvQuery::CollectQueryParams(UObject& QueryOwner, TArray<FAIDynamicParam>& NamedValues) const
{
	TArray<FName> RequiredParams;

	// collect all params
	for (int32 OptionIndex = 0; OptionIndex < Options.Num(); OptionIndex++)
	{
		const UEnvQueryOption* Option = Options[OptionIndex];
		if (Option->Generator == nullptr)
		{
			continue;
		}

		FEQSParamsExporter::AddNamedValuesFromObject(QueryOwner, *(Option->Generator), NamedValues, RequiredParams);

		for (int32 TestIndex = 0; TestIndex < Option->Tests.Num(); TestIndex++)
		{
			const UEnvQueryTest* TestOb = Option->Tests[TestIndex];
			if (TestOb)
			{
				FEQSParamsExporter::AddNamedValuesFromObject(QueryOwner, *TestOb, NamedValues, RequiredParams);
			}
		}
	}

	// remove unnecessary params
	for (int32 ValueIndex = NamedValues.Num() - 1; ValueIndex >= 0; ValueIndex--)
	{
		if (!RequiredParams.Contains(NamedValues[ValueIndex].ParamName))
		{
			NamedValues.RemoveAt(ValueIndex);
		}
	}
}

void UEnvQuery::PostInitProperties()
{
	Super::PostInitProperties();
	QueryName = GetFName();
}

void UEnvQuery::PostLoad()
{
	Super::PostLoad();

	if (QueryName == NAME_None || QueryName.IsValid() == false)
	{
		QueryName = GetFName();
	}
}

#if WITH_EDITOR
void UEnvQuery::PostRename(UObject* OldOuter, const FName OldName)
{
	Super::PostRename(OldOuter, OldName);

	QueryName = GetFName();
}

void UEnvQuery::PostDuplicate(bool bDuplicateForPIE)
{
	if (bDuplicateForPIE == false)
	{
		QueryName = GetFName();
	}

	Super::PostDuplicate(bDuplicateForPIE);
}
#endif // WITH_EDITOR

