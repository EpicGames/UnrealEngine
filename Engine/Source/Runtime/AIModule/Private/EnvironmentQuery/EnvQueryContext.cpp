// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/EnvQueryContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryContext)

UEnvQueryContext::UEnvQueryContext(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

void UEnvQueryContext::ProvideContext(FEnvQueryInstance& QueryInstance, FEnvQueryContextData& ContextData) const
{
	// empty in base class
}

