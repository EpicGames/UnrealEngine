// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/EnvQueryDebugHelpers.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "EnvironmentQuery/EnvQueryGenerator.h"
#include "EnvironmentQuery/EnvQueryTest.h"
#include "VisualLoggerExtension.h"
#include "EnvironmentQuery/EQSRenderingComponent.h"
#include "Misc/Compression.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryDebugHelpers)

#if USE_EQS_DEBUGGER
void UEnvQueryDebugHelpers::QueryToBlobArray(FEnvQueryInstance& Query, TArray<uint8>& BlobArray, bool bUseCompression)
{
	EQSDebug::FQueryData EQSLocalData;
	UEnvQueryDebugHelpers::QueryToDebugData(Query, EQSLocalData);

	DebugDataToBlobArray(EQSLocalData, BlobArray, bUseCompression);
}

void UEnvQueryDebugHelpers::DebugDataToBlobArray(EQSDebug::FQueryData& EQSLocalData, TArray<uint8>& BlobArray, bool bUseCompression)
{
	if (!bUseCompression)
	{
		FMemoryWriter ArWriter(BlobArray);
		ArWriter << EQSLocalData;
	}
	else
	{
		TArray<uint8> UncompressedBuffer;
		FMemoryWriter ArWriter(UncompressedBuffer);
		ArWriter << EQSLocalData;

		const int32 UncompressedSize = UncompressedBuffer.Num();
		const int32 HeaderSize = sizeof(int32);
		BlobArray.Init(0, HeaderSize + FMath::TruncToInt(1.1f * UncompressedSize));

		int32 CompressedSize = BlobArray.Num() - HeaderSize;
		uint8* DestBuffer = BlobArray.GetData();
		FMemory::Memcpy(DestBuffer, &UncompressedSize, HeaderSize);
		DestBuffer += HeaderSize;

		FCompression::CompressMemory(NAME_Zlib, (void*)DestBuffer, CompressedSize, (void*)UncompressedBuffer.GetData(), UncompressedSize, COMPRESS_BiasMemory);

		BlobArray.SetNum(CompressedSize + HeaderSize, EAllowShrinking::No);
	}
}

void UEnvQueryDebugHelpers::QueryToDebugData(FEnvQueryInstance& Query, EQSDebug::FQueryData& EQSLocalData, int32 MaxItemsToStore)
{
	// step 1: data for rendering component
	EQSLocalData.Reset();

	FEQSSceneProxy::CollectEQSData(&Query, &Query, 1.0f, true, EQSLocalData.SolidSpheres, EQSLocalData.Texts, EQSLocalData.RenderDebugHelpers);

	// step 2: detailed scoring data for HUD
	const int32 FirstItemIndex = 0;

	const int32 NumTests = Query.ItemDetails.IsValidIndex(0) ? Query.ItemDetails[0].TestResults.Num() : 0;
	const int32 NumItems = FMath::Min(MaxItemsToStore, Query.NumValidItems);

	EQSLocalData.Name = Query.QueryName;
	EQSLocalData.Id = Query.QueryID;
	EQSLocalData.Items.Reset(NumItems);
	EQSLocalData.Tests.Reset(NumTests);
	EQSLocalData.NumValidItems = Query.NumValidItems;

	UEnvQueryItemType* ItemCDO = Query.ItemType.GetDefaultObject();
	for (int32 ItemIdx = 0; ItemIdx < NumItems; ItemIdx++)
	{
		EQSDebug::FItemData ItemInfo;
		ItemInfo.ItemIdx = ItemIdx + FirstItemIndex;
		ItemInfo.TotalScore = Query.Items[ItemInfo.ItemIdx].Score;

		const uint8* ItemData = Query.RawData.GetData() + Query.Items[ItemInfo.ItemIdx].DataOffset;
		ItemInfo.Desc = FString::Printf(TEXT("[%d] %s"), ItemInfo.ItemIdx, *ItemCDO->GetDescription(ItemData));

		ItemInfo.TestValues.Reserve(NumTests);
		ItemInfo.TestScores.Reserve(NumTests);
		for (int32 TestIdx = 0; TestIdx < NumTests; TestIdx++)
		{
			const float ScoreN = Query.ItemDetails[ItemInfo.ItemIdx].TestResults[TestIdx];
			const float ScoreW = Query.ItemDetails[ItemInfo.ItemIdx].TestWeightedScores[TestIdx];

			ItemInfo.TestValues.Add(ScoreN);
			ItemInfo.TestScores.Add(ScoreW);
		}

		EQSLocalData.Items.Add(ItemInfo);
	}

	const int32 NumAllTests = Query.Options[Query.OptionIndex].Tests.Num();
	for (int32 TestIdx = 0; TestIdx < NumAllTests; TestIdx++)
	{
		EQSDebug::FTestData TestInfo;

		UEnvQueryTest* TestOb = Query.Options[Query.OptionIndex].Tests[TestIdx];
		TestInfo.ShortName = TestOb->GetDescriptionTitle().ToString();
		TestInfo.Detailed = TestOb->GetDescriptionDetails().ToString().Replace(TEXT("\n"), TEXT(", "));

		EQSLocalData.Tests.Add(TestInfo);
	}

	EQSLocalData.UsedOption = Query.OptionIndex;
	const int32 NumOptions = Query.Options.Num();
	for (int32 OptionIndex = 0; OptionIndex < NumOptions; ++OptionIndex)
	{
		const FString UserName = Query.Options[OptionIndex].Generator->OptionName;
		EQSLocalData.Options.Add(UserName.Len() > 0 ? UserName : Query.Options[OptionIndex].Generator->GetName());
	}
}

void  UEnvQueryDebugHelpers::BlobArrayToDebugData(const TArray<uint8>& BlobArray, EQSDebug::FQueryData& EQSLocalData, bool bUseCompression)
{
	if (!bUseCompression)
	{
		FMemoryReader ArReader(BlobArray);
		ArReader << EQSLocalData;
	}
	else
	{
		TArray<uint8> UncompressedBuffer;
		int32 UncompressedSize = 0;
		const int32 HeaderSize = sizeof(int32);
		uint8* SrcBuffer = (uint8*)BlobArray.GetData();
		FMemory::Memcpy(&UncompressedSize, SrcBuffer, HeaderSize);
		SrcBuffer += HeaderSize;
		const int32 CompressedSize = BlobArray.Num() - HeaderSize;

		UncompressedBuffer.AddZeroed(UncompressedSize);

		FCompression::UncompressMemory(NAME_Zlib, (void*)UncompressedBuffer.GetData(), UncompressedSize, SrcBuffer, CompressedSize);
		FMemoryReader ArReader(UncompressedBuffer);

		ArReader << EQSLocalData;
	}
}

#endif //ENABLE_VISUAL_LOG

#if ENABLE_VISUAL_LOG && USE_EQS_DEBUGGER

#define PRINT_TABLE_ROW(__src__, __dst__, __len__) \
	{ \
		const FString DataToPrint = __src__; \
		__dst__ += DataToPrint; \
		for (int32 i = DataToPrint.Len(); i < __len__; i++) __dst__ += FString::Printf(TEXT(" ")); \
	}

namespace
{
	static void DrawEQSItemDetails(const EQSDebug::FQueryData& EQSLocalData, int32 ItemIdx, FVisualLogLine& LogLine)
	{
		const EQSDebug::FItemData& ItemData = EQSLocalData.Items[ItemIdx];

		PRINT_TABLE_ROW(ItemData.Desc, LogLine.Line, 27);

		PRINT_TABLE_ROW(FString::Printf(TEXT(" | %.2f"), ItemData.TotalScore), LogLine.Line, 8);

		const int32 NumTests = ItemData.TestScores.Num();
		float TotalWeightedScore = 0.0f;
		for (int32 Idx = 0; Idx < NumTests; Idx++)
		{
			TotalWeightedScore += ItemData.TestScores[Idx];
		}

		for (int32 Idx = 0; Idx < NumTests; Idx++)
		{
			const float ScoreW = ItemData.TestScores[Idx];
			const float ScoreN = ItemData.TestValues[Idx];
			FString DescScoreW = FString::Printf(TEXT("%.2f"), ScoreW);
			FString DescScoreN = (ScoreN == UEnvQueryTypes::SkippedItemValue) ? TEXT("SKIP") : FString::Printf(TEXT("%.2f"), ScoreN);

			PRINT_TABLE_ROW(FString(" | ") + DescScoreW + FString(" ") + DescScoreN, LogLine.Line, 15);
		}
	}
}

void UEnvQueryDebugHelpers::LogQueryInternal(FEnvQueryInstance& Query, const FName& CategoryName, ELogVerbosity::Type Verbosity, double TimeSeconds, FVisualLogEntry* CurrentEntry)
{
	const int32 UniqueId = FVisualLogger::Get().GetUniqueId(TimeSeconds);

	EQSDebug::FQueryData EQSLocalData;
	UEnvQueryDebugHelpers::QueryToDebugData(Query, EQSLocalData);

	TArray<uint8> BlobArray;
	DebugDataToBlobArray(EQSLocalData, BlobArray);

	const FString AdditionalLogInfo = FString::Printf(TEXT("Executed EQS: \n - Name: '%s' (id=%d, option=%d),\n - All Items: %d,\n - ValidItems: %d"), *Query.QueryName, Query.QueryID, Query.OptionIndex, Query.ItemDetails.Num(), Query.NumValidItems);
	FVisualLogLine Line(CategoryName, Verbosity, AdditionalLogInfo, Query.QueryID);
	Line.TagName = *EVisLogTags::TAG_EQS;
	Line.UniqueId = UniqueId;

	CurrentEntry->AddDataBlock(EVisLogTags::TAG_EQS, BlobArray, CategoryName, Verbosity).UniqueId = UniqueId;


	if (EQSLocalData.NumValidItems > 0)
	{
		const int32 NumTests = EQSLocalData.Tests.Num();
		Line.Line += FString("\n");

		// draw test weights for best X items
		const int32 NumItems = EQSLocalData.Items.Num();

		// print sorted tests' descriptions, to be able to tie TestIdx with an actual test
		const FEnvQueryOptionInstance& Option = Query.Options[Query.OptionIndex];
		for (int32 TestIdx = 0; TestIdx < NumTests; TestIdx++)
		{
			Line.Line += FString::Printf(TEXT("%d: %s\n"), TestIdx, *Option.Tests[TestIdx]->GetDescriptionTitle().ToString());
		}

		// table header		
		{
			FString HeaderString;
			PRINT_TABLE_ROW(TEXT("Item "), HeaderString, 27);

			PRINT_TABLE_ROW(TEXT(" | Score"), HeaderString, 8);

			for (int32 TestIdx = 0; TestIdx < NumTests; TestIdx++)
			{
				PRINT_TABLE_ROW(FString::Printf(TEXT(" | Test %d  "), TestIdx), HeaderString, 15);
			}

			Line.Line += HeaderString;
		}
		Line.Line += FString("\n");

		// valid items
		for (int32 Idx = 0; Idx < NumItems; Idx++)
		{
			DrawEQSItemDetails(EQSLocalData, Idx, Line);
			Line.Line += FString("\n");
		}
	}
	CurrentEntry->LogLines.Add(Line);
}

#undef PRINT_TABLE_ROW
#endif //ENABLE_VISUAL_LOG && USE_EQS_DEBUGGER

