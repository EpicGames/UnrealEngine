// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/EnvQueryGenerator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryGenerator)

UEnvQueryGenerator::UEnvQueryGenerator(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	bAutoSortTests = true;
	bCanRunAsync = false;
}

void UEnvQueryGenerator::UpdateNodeVersion()
{
	VerNum = EnvQueryGeneratorVersion::Latest;
}

EEnvQueryResultNormalizationOption UEnvQueryGenerator::GetNormalizationOption() const
{
	return EnvQueryResultNormalizationOption;
}

void UEnvQueryGenerator::PostLoad()
{
	Super::PostLoad();
	UpdateNodeVersion();
}

