// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Templates/Greater.h"
#include "Stats/Stats.h"
#include "UObject/UObjectBaseUtility.h"
#include "EnvironmentQuery/Items/EnvQueryItemType.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_VectorBase.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_ActorBase.h"
#include "EnvironmentQuery/EnvQueryTest.h"
#include "EnvironmentQuery/EnvQueryGenerator.h"
#include "EnvironmentQuery/EnvQueryManager.h"
#include "EnvironmentQuery/Contexts/EnvQueryContext_Item.h"
#include "EnvironmentQuery/Generators/EnvQueryGenerator_Composite.h"
#include "AISystem.h"
#include "AITypes.h"

#if UE_BUILD_SHIPPING
#define eqs_ensure ensure
#else
#define eqs_ensure ensureAlways
#endif

//----------------------------------------------------------------------//
// FEnvQueryDebugData
//----------------------------------------------------------------------//

void FEnvQueryDebugData::Store(const FEnvQueryInstance& QueryInstance, const float ExecutionTime, const bool bStepDone)
{
#if USE_EQS_DEBUGGER
	const int32 GeneratorIdx = (CurrentOptionGeneratorIdx != INDEX_NONE ? CurrentOptionGeneratorIdx : OptionData[QueryInstance.OptionIndex].NumGenerators);
	const int32 StepIdx = QueryInstance.CurrentTest + GeneratorIdx;
	OptionStats[QueryInstance.OptionIndex].StepData[StepIdx].ExecutionTime += ExecutionTime;
	OptionStats[QueryInstance.OptionIndex].StepData[StepIdx].NumProcessedItems = QueryInstance.NumProcessedItems;
	if (bStepDone)
	{
		DebugItemDetails = QueryInstance.ItemDetails;
		DebugItems = QueryInstance.Items;
		RawData = QueryInstance.RawData;
	}
#endif // USE_EQS_DEBUGGER
}

void FEnvQueryDebugData::PrepareOption(const FEnvQueryInstance& QueryInstance, const TArray<UEnvQueryGenerator*>& Generators, const int32 NumTests)
{
#if USE_EQS_DEBUGGER
	const int32 NumGenerators = FMath::Max(Generators.Num(), 1);
	const int32 NumSteps = NumGenerators + NumTests;

	OptionStats.AddDefaulted(1);
	OptionStats.Last().StepData.AddZeroed(NumSteps);
	OptionStats.Last().NumRuns = 1;

	OptionData[QueryInstance.OptionIndex].NumGenerators = NumGenerators;

	// fill in generator names only when Generators array was provided (composite generator), usually it won't be
	for (int32 Idx = 0; Idx < Generators.Num(); Idx++)
	{
		check(Generators[Idx]);
		OptionData[QueryInstance.OptionIndex].GeneratorNames.Add(Generators[Idx]->GetFName());
	}

	DebugItems.Reset();
	DebugItemDetails.Reset();
	RawData.Reset();
	PerformedTestNames.Reset();
	bSingleItemResult = false;
#endif // USE_EQS_DEBUGGER
}

void FEnvQueryDebugProfileData::Add(const FEnvQueryDebugProfileData& Other)
{
	if (Other.OptionStats.Num() > OptionStats.Num())
	{
		OptionStats.AddDefaulted(Other.OptionStats.Num() - OptionStats.Num());
	}

	for (int32 OptionIdx = 0; OptionIdx < Other.OptionStats.Num(); OptionIdx++)
	{
		const FOptionStat& OtherStat = Other.OptionStats[OptionIdx];
		FOptionStat& OptionStat = OptionStats[OptionIdx];

		if (OptionStat.StepData.Num() < OtherStat.StepData.Num())
		{
			OptionStat.StepData.AddZeroed(OtherStat.StepData.Num() - OptionStat.StepData.Num());
		}

		OptionStat.NumRuns += OtherStat.NumRuns;
		for (int32 StepIdx = 0; StepIdx < OtherStat.StepData.Num(); StepIdx++)
		{
			OptionStat.StepData[StepIdx].ExecutionTime += OtherStat.StepData[StepIdx].ExecutionTime;
			OptionStat.StepData[StepIdx].NumProcessedItems += OtherStat.StepData[StepIdx].NumProcessedItems;
		}
	}

	if (Other.OptionData.Num() > OptionData.Num())
	{
		OptionData = Other.OptionData;
	}
}

//----------------------------------------------------------------------//
// FEnvQueryInstance
//----------------------------------------------------------------------//
#if USE_EQS_DEBUGGER
bool FEnvQueryInstance::bDebuggingInfoEnabled = true;
#endif // USE_EQS_DEBUGGER

bool FEnvQueryInstance::PrepareContext(UClass* ContextClass, FEnvQueryContextData& ContextData)
{
	if (ContextClass == nullptr)
	{
		return false;
	}

	if (ContextClass != UEnvQueryContext_Item::StaticClass())
	{
		const FEnvQueryInstanceContextCacheItem* CachedData = ContextCache.FindByPredicate([ContextClass](const FEnvQueryInstanceContextCacheItem& Item)
			{
				return Item.Class == ContextClass;
			});
		if (CachedData == nullptr)
		{
			UEnvQueryManager* QueryManager = UEnvQueryManager::GetCurrent(World);
			const UEnvQueryContext* ContextOb = QueryManager->PrepareLocalContext(ContextClass);

			check(ContextOb);
			ContextOb->ProvideContext(*this, ContextData);

#if STATS
			// this contraption has been added to nail down a location of a rare bug
			const uint32 AllocatedSize = GetContextAllocatedSize();
			DEC_MEMORY_STAT_BY(STAT_AI_EQS_InstanceMemory, AllocatedSize);
#endif // STATS

			ContextCache.Add({ContextClass, ContextData});

			INC_MEMORY_STAT_BY(STAT_AI_EQS_InstanceMemory, GetContextAllocatedSize());
		}
		else
		{
			ContextData = CachedData->ContextData;
		}
	}

	if (ContextData.NumValues == 0)
	{
		UE_LOGF(LogEQS, Log, "Query [%ls] is missing values for context [%ls], skipping test %d:%d [%ls]",
			*QueryName, *UEnvQueryTypes::GetShortTypeName(ContextClass).ToString(),
			OptionIndex, CurrentTest,
			CurrentTest >=0 ? *UEnvQueryTypes::GetShortTypeName(Options[OptionIndex].Tests[CurrentTest]).ToString() : TEXT("Generator")
			);

		return false;
	}

	return true;
}

bool FEnvQueryInstance::PrepareContext(UClass* Context, TArray<FEnvQuerySpatialData>& Data)
{
	if (Context == NULL)
	{
		return false;
	}

	FEnvQueryContextData ContextData;
	const bool bSuccess = PrepareContext(Context, ContextData);

	if (bSuccess && ContextData.ValueType && ContextData.ValueType->IsChildOf(UEnvQueryItemType_VectorBase::StaticClass()))
	{
		UEnvQueryItemType_VectorBase* DefTypeOb = (UEnvQueryItemType_VectorBase*)ContextData.ValueType->GetDefaultObject();
		const uint16 DefTypeValueSize = DefTypeOb->GetValueSize();
		uint8* ContextRawData = ContextData.RawData.GetData();

		Data.SetNumUninitialized(ContextData.NumValues);
		for (int32 ValueIndex = 0; ValueIndex < ContextData.NumValues; ValueIndex++)
		{
			Data[ValueIndex].Location = DefTypeOb->GetItemLocation(ContextRawData);
			Data[ValueIndex].Rotation = DefTypeOb->GetItemRotation(ContextRawData);
			ContextRawData += DefTypeValueSize;
		}
	}

	return bSuccess;
}

bool FEnvQueryInstance::PrepareContext(UClass* Context, TArray<FVector>& Data)
{
	if (Context == NULL)
	{
		return false;
	}

	FEnvQueryContextData ContextData;
	const bool bSuccess = PrepareContext(Context, ContextData);

	if (bSuccess && ContextData.ValueType && ContextData.ValueType->IsChildOf(UEnvQueryItemType_VectorBase::StaticClass()))
	{
		UEnvQueryItemType_VectorBase* DefTypeOb = (UEnvQueryItemType_VectorBase*)ContextData.ValueType->GetDefaultObject();
		const uint16 DefTypeValueSize = DefTypeOb->GetValueSize();
		uint8* ContextRawData = (uint8*)ContextData.RawData.GetData();

		Data.Reserve(ContextData.NumValues);
		for (int32 ValueIndex = 0; ValueIndex < ContextData.NumValues; ValueIndex++)
		{
			const FVector ItemLocation = DefTypeOb->GetItemLocation(ContextRawData);
			if (FAISystem::IsValidLocation(ItemLocation))
			{
				Data.Add(ItemLocation);
			}
			ContextRawData += DefTypeValueSize;
		}

		if (Data.Num() != ContextData.NumValues)
		{
			UE_LOGF(LogEQS, Warning, "FEnvQuery::PrepareContext found %d invalid vectors from context %ls", ContextData.NumValues - Data.Num(), *Context->GetPathName());
		}
	}

	return Data.Num() > 0;
}

bool FEnvQueryInstance::PrepareContext(UClass* Context, TArray<FRotator>& Data)
{
	if (Context == NULL)
	{
		return false;
	}

	FEnvQueryContextData ContextData;
	const bool bSuccess = PrepareContext(Context, ContextData);

	if (bSuccess && ContextData.ValueType && ContextData.ValueType->IsChildOf(UEnvQueryItemType_VectorBase::StaticClass()))
	{
		UEnvQueryItemType_VectorBase* DefTypeOb = (UEnvQueryItemType_VectorBase*)ContextData.ValueType->GetDefaultObject();
		const uint16 DefTypeValueSize = DefTypeOb->GetValueSize();
		uint8* ContextRawData = ContextData.RawData.GetData();

		Data.Reserve(ContextData.NumValues);
		for (int32 ValueIndex = 0; ValueIndex < ContextData.NumValues; ValueIndex++)
		{
			const FRotator ItemRotation = DefTypeOb->GetItemRotation(ContextRawData);
			if (FAISystem::IsValidRotation(ItemRotation))
			{
				Data.Add(ItemRotation);
			}
			ContextRawData += DefTypeValueSize;
		}

		if (Data.Num() != ContextData.NumValues)
		{
			UE_LOGF(LogEQS, Warning, "FEnvQuery::PrepareContext found %d invalid rotators from context %ls", ContextData.NumValues - Data.Num(), *Context->GetPathName());
		}
	}

	return Data.Num() > 0;
}

bool FEnvQueryInstance::PrepareContext(UClass* Context, TArray<AActor*>& Data)
{
	if (Context == NULL)
	{
		return false;
	}

	FEnvQueryContextData ContextData;
	const bool bSuccess = PrepareContext(Context, ContextData);

	if (bSuccess && ContextData.ValueType && ContextData.ValueType->IsChildOf(UEnvQueryItemType_ActorBase::StaticClass()))
	{
		UEnvQueryItemType_ActorBase* DefTypeOb = (UEnvQueryItemType_ActorBase*)ContextData.ValueType->GetDefaultObject();
		const uint16 DefTypeValueSize = DefTypeOb->GetValueSize();
		uint8* ContextRawData = ContextData.RawData.GetData();

		Data.Reserve(ContextData.NumValues);
		for (int32 ValueIndex = 0; ValueIndex < ContextData.NumValues; ValueIndex++)
		{
			AActor* Actor = DefTypeOb->GetActor(ContextRawData);
			if (Actor)
			{
				Data.Add(Actor);
			}
			ContextRawData += DefTypeValueSize;
		}
	}

	return Data.Num() > 0;
}

void FEnvQueryInstance::ExecuteOneStep(double TimeLimit)
{
	if (!Owner.IsValid())
	{
		MarkAsOwnerLost();
		return;
	}

	check(IsFinished() == false);

	if (!Options.IsValidIndex(OptionIndex))
	{
		NumValidItems = 0;
		FinalizeQuery();
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_AI_EQS_ExecuteOneStep);

	FEnvQueryOptionInstance& OptionItem = Options[OptionIndex];
	double StepStartTime = FPlatformTime::Seconds();

	const bool bDoingLastTest = (CurrentTest >= OptionItem.Tests.Num() - 1);
	bool bStepDone = true;
	bIsCurrentlyRunningAsync = false;
	CurrentStepTimeLimit = TimeLimit;

	if (CurrentTest < 0)
	{
		bool bRunGenerator = true;
#if USE_EQS_DEBUGGER
		int32 LastValidItems = 0;
#endif
		if (!OptionItem.Generator->IsCurrentlyRunningAsync())
		{
			SCOPE_CYCLE_COUNTER(STAT_AI_EQS_GeneratorTime);
			DEC_DWORD_STAT_BY(STAT_AI_EQS_NumItems, Items.Num());

			RawData.Reset();
			Items.Reset();
			ItemType = OptionItem.ItemType;
			bPassOnSingleResult = false;
			ValueSize = (ItemType->GetDefaultObject<UEnvQueryItemType>())->GetValueSize();

#if USE_EQS_DEBUGGER
			if (bStoreDebugInfo)
			{
				UEnvQueryGenerator_Composite* CompositeGen = Cast<UEnvQueryGenerator_Composite>(OptionItem.Generator);
				TArray<UEnvQueryGenerator*> GeneratorList;

				if (CompositeGen)
				{
					// resolve nested composites while on it
					GeneratorList.Append(CompositeGen->Generators);
					for (int32 InnerIdx = 0; InnerIdx < GeneratorList.Num(); InnerIdx++)
					{
						UEnvQueryGenerator_Composite* InnerCompositeGen = Cast<UEnvQueryGenerator_Composite>(GeneratorList[InnerIdx]);
						if (InnerCompositeGen)
						{
							GeneratorList.Append(InnerCompositeGen->Generators);
							GeneratorList.RemoveAt(InnerIdx, EAllowShrinking::No);
							InnerIdx--;
						}
					}
				}

				DebugData.PrepareOption(*this, GeneratorList, OptionItem.Tests.Num());

				// special case for composite generator: run each inner generator separately and record times
				if (GeneratorList.Num())
				{
					bRunGenerator = false;
					DebugData.CurrentOptionGeneratorIdx = 0;
					for (int32 GeneratorIdx = 0; GeneratorIdx < GeneratorList.Num() - 1; GeneratorIdx++)
					{
						{
							FScopeCycleCounterUObject GeneratorScope(GeneratorList[GeneratorIdx]);
							GeneratorList[GeneratorIdx]->GenerateItems(*this);
						}

						const double GenTime = FPlatformTime::Seconds();
						const double StepExecutionTime = GenTime - StepStartTime;
						StepStartTime += StepExecutionTime;
						TotalExecutionTime += StepExecutionTime;
						StartTime = GenTime;
						NumProcessedItems = Items.Num() - LastValidItems;
						LastValidItems = Items.Num();
						DebugData.CurrentOptionGeneratorIdx++;
						DebugData.Store(*this, FloatCastChecked<float>(StepExecutionTime, /* Precision */ 1. / 512.), false);
						NumProcessedItems = 0;
					}

					{
						FScopeCycleCounterUObject GeneratorScope(GeneratorList.Last());
						GeneratorList.Last()->GenerateItems(*this);
					}
					DebugData.CurrentOptionGeneratorIdx = INDEX_NONE;
				}
			}
#endif // USE_EQS_DEBUGGER
		}

		if (bRunGenerator)
		{
#if !UE_BUILD_SHIPPING
			const double BeforeGenTime = FPlatformTime::Seconds();
#endif // UE_BUILD_SHIPPING

			FScopeCycleCounterUObject GeneratorScope(OptionItem.Generator);
			OptionItem.Generator->GenerateItems(*this);

#if !UE_BUILD_SHIPPING
			const double GenTime = FPlatformTime::Seconds() - BeforeGenTime;
			if (GenTime >= GenerationTimeWarningSeconds)
			{				
				UE_LOGF(LogEQS, Warning, "Query %ls is over generation time warning. %f second (limit is %f second)", *QueryName, GenTime, GenerationTimeWarningSeconds);
			}
#endif // UE_BUILD_SHIPPING
		}

		bIsCurrentlyRunningAsync |= OptionItem.Generator->IsCurrentlyRunningAsync();
		bStepDone = !bIsCurrentlyRunningAsync;

		if (bStepDone)
		{
			FinalizeGeneration();
		}

#if USE_EQS_DEBUGGER
		NumProcessedItems = Items.Num() - LastValidItems;
#endif // USE_EQS_DEBUGGER
	}
	else if (OptionItem.Tests.IsValidIndex(CurrentTest))
	{
		SCOPE_CYCLE_COUNTER(STAT_AI_EQS_TestTime);

		UEnvQueryTest* TestObject = OptionItem.Tests[CurrentTest];

		if (!TestObject->IsCurrentlyRunningAsync())
		{
			// item generator uses this flag to alter the scoring behavior
			bPassOnSingleResult = (bDoingLastTest && Mode == EEnvQueryRunMode::SingleResult && TestObject->CanRunAsFinalCondition());

			if (bPassOnSingleResult && (CurrentTestStartingItem == 0))
			{
				// Since we know we're the last test that is a final condition, if we were scoring previously we should sort the tests now before we test them
				bool bSortTests = false;
				for (int32 TestIndex = 0; TestIndex < OptionItem.Tests.Num() - 1; ++TestIndex)
				{
					if (OptionItem.Tests[TestIndex]->TestPurpose != EEnvTestPurpose::Filter)
					{
						// Found one.  We should sort.
						bSortTests = true;
						break;
					}
				}

				if (bSortTests)
				{
					SortScores();
				}
			}
		}

		const int32 ItemsAlreadyProcessed = CurrentTestStartingItem;

		{
			FScopeCycleCounterUObject TestScope(TestObject);
			TestObject->RunTest(*this);
		}

		bIsCurrentlyRunningAsync |= TestObject->IsCurrentlyRunningAsync();
		bStepDone = !bIsCurrentlyRunningAsync && (CurrentTestStartingItem >= Items.Num() || bFoundSingleResult
			// or no items processed ==> this means error, unless this test is running async
			|| (ItemsAlreadyProcessed == CurrentTestStartingItem));

		if (bStepDone)
		{
			FinalizeTest();
		}
	}
	else
	{
		UE_LOGF(LogEQS, Warning, "Query [%ls] is trying to execute non existing test! [option:%d test:%d]", 
			*QueryName, OptionIndex, CurrentTest);
	}

	const double StepExecutionTime = FPlatformTime::Seconds() - StepStartTime;
	TotalExecutionTime += StepExecutionTime;

#if USE_EQS_DEBUGGER
	if (bStoreDebugInfo && !bIsCurrentlyRunningAsync)
	{
		DebugData.Store(*this, FloatCastChecked<float>(StepExecutionTime, /* Precision */ 1./512.), bStepDone);
	}
#endif // USE_EQS_DEBUGGER
	
	if (bStepDone)
	{
		CurrentTest++;
		CurrentTestStartingItem = 0;
#if USE_EQS_DEBUGGER
		NumProcessedItems = 0;
#endif // USE_EQS_DEBUGGER
	}

	// sort results or switch to next option when all tests are performed
	if (!bIsCurrentlyRunningAsync && IsFinished() == false && (OptionItem.Tests.Num() == CurrentTest || NumValidItems <= 0))
	{
		if (NumValidItems > 0)
		{
			// found items, sort and finish
			FinalizeQuery();
		}
		else
		{
			// no items here, go to next option or finish			
			if (OptionIndex + 1 >= Options.Num())
			{
				// out of options, finish processing without errors
				FinalizeQuery();
			}
			else
			{
				OptionIndex++;
				CurrentTest = -1;
			}
		}
	}
}

FString FEnvQueryInstance::GetExecutionTimeDescription() const
{
	FString Description = FString::Printf(TEXT("Total Execution Time: %.2f ms"), TotalExecutionTime * 1000.);

#if USE_EQS_DEBUGGER
	for (int32 OptionIdx = 0; OptionIdx <= OptionIndex; OptionIdx++)
	{
		const FEnvQueryOptionInstance& OptionItem = Options[OptionIdx];
		const int32 LastTestIndex = IsFinished() ? (OptionItem.Tests.Num() - 1) : CurrentTest;
		const int32 NumGenerators = DebugData.OptionData.IsValidIndex(OptionIdx) ? DebugData.OptionData[OptionIdx].NumGenerators : 1;

		for (int32 StepIdx = 0; StepIdx <= LastTestIndex; StepIdx++)
		{
			Description += (StepIdx < NumGenerators) ? TEXT("\n  generator[") : TEXT("\n    test[");
			Description += TTypeToString<int32>().ToString((StepIdx < NumGenerators) ? StepIdx : (StepIdx - NumGenerators));
			Description += TEXT("]: ");
			
			if (DebugData.OptionStats.IsValidIndex(OptionIdx) && DebugData.OptionStats[OptionIdx].StepData.IsValidIndex(StepIdx))
			{
				Description += FString::Printf(TEXT("%.2f ms (items:%d)"),
					DebugData.OptionStats[OptionIdx].StepData[StepIdx].ExecutionTime * 1000.0f,
					DebugData.OptionStats[OptionIdx].StepData[StepIdx].NumProcessedItems);
			}
			else
			{
				Description += TEXT("N/A");
			}

			Description += FString::Printf(TEXT(" (%s)"), StepIdx >= NumGenerators ?
				(OptionItem.Tests.IsValidIndex(StepIdx - NumGenerators) ? *GetNameSafe(OptionItem.Tests[StepIdx - NumGenerators]) : TEXT("unknown test!")) :
				(DebugData.OptionData.IsValidIndex(OptionIdx) && DebugData.OptionData[OptionIdx].GeneratorNames.Num() ?
					(DebugData.OptionData[OptionIdx].GeneratorNames.IsValidIndex(StepIdx) ? *DebugData.OptionData[OptionIdx].GeneratorNames[StepIdx].ToString() : TEXT("unknown generator!")) :
					*GetNameSafe(OptionItem.Generator)));
		}
	}
#else
	Description += TEXT(" (detailed data not available without USE_EQS_DEBUGGER)");
#endif // USE_EQS_DEBUGGER

	return Description;
}

#if !NO_LOGGING
void FEnvQueryInstance::Log(const FString Msg) const
{
	UE_LOGF(LogEQS, Warning, "%ls", *Msg);
}
#endif // !NO_LOGGING

void FEnvQueryInstance::ReserveItemData(const int32 NumAdditionalItems)
{
	DEC_MEMORY_STAT_BY(STAT_AI_EQS_InstanceMemory, RawData.GetAllocatedSize());

	RawData.Reserve(RawData.Num() + (NumAdditionalItems * ValueSize));

	INC_MEMORY_STAT_BY(STAT_AI_EQS_InstanceMemory, RawData.GetAllocatedSize());
}

FEnvQueryInstance::FItemIterator::FItemIterator(const UEnvQueryTest* QueryTest, FEnvQueryInstance& QueryInstance, int32 StartingItemIndex)
	: FConstItemIterator(QueryInstance, StartingItemIndex)
{
	check(QueryTest);

	CachedFilterOp = QueryTest->MultipleContextFilterOp.GetIntValue();
	CachedScoreOp = QueryTest->MultipleContextScoreOp.GetIntValue();
	bIsFiltering = (QueryTest->TestPurpose == EEnvTestPurpose::Filter) || (QueryTest->TestPurpose == EEnvTestPurpose::FilterAndScore);

	Deadline = QueryInstance.CurrentStepTimeLimit > 0.0 ? (FPlatformTime::Seconds() + QueryInstance.CurrentStepTimeLimit) : -1.0;
	InitItemScore();
}

void FEnvQueryInstance::FItemIterator::HandleFailedTestResult()
{
	ItemScore = -1.f;
	Instance.Items[CurrentItem].Discard();
#if USE_EQS_DEBUGGER
	Instance.ItemDetails[CurrentItem].FailedTestIndex = Instance.CurrentTest;
#endif
	Instance.NumValidItems--;
}

void FEnvQueryInstance::FItemIterator::StoreTestResult()
{
	CheckItemPassed();
	eqs_ensure(FMath::IsNaN(ItemScore) == false);

#if USE_EQS_DEBUGGER
	Instance.NumProcessedItems++;
#endif // USE_EQS_DEBUGGER

	if (Instance.IsInSingleItemFinalSearch())
	{
		// handle SingleResult mode
		// this also implies we're not in 'score-only' mode
		if (bPassed)
		{
			if (bForced)
			{
				// store item value in case it's using special "skipped" constant
				Instance.ItemDetails[CurrentItem].TestResults[Instance.CurrentTest] = ItemScore;
			}

			Instance.PickSingleItem(CurrentItem);
			Instance.bFoundSingleResult = true;
		}
		else if (!bPassed)
		{
			HandleFailedTestResult();
		}
	}
	else
	{
		if (!bPassed && bIsFiltering)
		{
			HandleFailedTestResult();
		}
		else if (CachedScoreOp == EEnvTestScoreOperator::AverageScore && !bForced)
		{
			if (NumPassedForItem != 0)
			{
				ItemScore /= NumPassedForItem;
			}
			else
			{
				ItemScore = 0;
			}
		}

		Instance.ItemDetails[CurrentItem].TestResults[Instance.CurrentTest] = ItemScore;
	}
}

void FEnvQueryInstance::NormalizeScores()
{
	// @note this function assumes results have been already sorted and all first NumValidItems
	// items in Items are valid (and the rest is not).
	check(NumValidItems <= Items.Num())

	float MinScore = 0.f;
	float MaxScore = -BIG_NUMBER;

	FEnvQueryItem* ItemInfo = Items.GetData();
	for (int32 ItemIndex = 0; ItemIndex < NumValidItems; ItemIndex++, ItemInfo++)
	{
		ensure(ItemInfo->IsValid());

		MinScore = FMath::Min(MinScore, ItemInfo->Score);
		MaxScore = FMath::Max(MaxScore, ItemInfo->Score);
	}

	ItemInfo = Items.GetData();
	if (MinScore == MaxScore)
	{
		const float Score = (MinScore == 0.f) ? 0.f : 1.f;
		for (int32 ItemIndex = 0; ItemIndex < NumValidItems; ItemIndex++, ItemInfo++)
		{
			ItemInfo->Score = Score;
		}
	}
	else
	{
		const float ScoreRange = MaxScore - MinScore;
		for (int32 ItemIndex = 0; ItemIndex < NumValidItems; ItemIndex++, ItemInfo++)
		{
			ItemInfo->Score = (ItemInfo->Score - MinScore) / ScoreRange;
		}
	}
}

void FEnvQueryInstance::SortScores()
{
	const int32 NumItems = Items.Num();
#if USE_EQS_DEBUGGER
	struct FSortHelperForDebugData
	{
		FEnvQueryItem	Item;
		FEnvQueryItemDetails ItemDetails;

		FSortHelperForDebugData(const FEnvQueryItem&	InItem, FEnvQueryItemDetails& InDebugDetails) : Item(InItem), ItemDetails(InDebugDetails) {}
		bool operator<(const FSortHelperForDebugData& Other) const
		{
			return Item < Other.Item;
		}
	};
	TArray<FSortHelperForDebugData> AllData;
	AllData.Reserve(NumItems);
	for (int32 Index = 0; Index < NumItems; ++Index)
	{
		AllData.Add(FSortHelperForDebugData(Items[Index], ItemDetails[Index]));
	}
	AllData.Sort(TGreater<FSortHelperForDebugData>());

	for (int32 Index = 0; Index < NumItems; ++Index)
	{
		Items[Index] = AllData[Index].Item;
		ItemDetails[Index] = AllData[Index].ItemDetails;
	}
#else
	Items.Sort(TGreater<FEnvQueryItem>());
#endif
}

void FEnvQueryInstance::StripRedundantData()
{
#if USE_EQS_DEBUGGER
	if (bStoreDebugInfo)
	{
		DebugData = FEnvQueryDebugData();
	}
#endif // USE_EQS_DEBUGGER
	Items.SetNum(NumValidItems);
}

void FEnvQueryInstance::PickRandomItemOfScoreAtLeast(float MinScore)
{
	// find first valid item with score worse than best range
	int32 NumBestItems = NumValidItems;
	for (int32 ItemIndex = 1; ItemIndex < NumValidItems; ItemIndex++)
	{
		if (Items[ItemIndex].Score < MinScore)
		{
			NumBestItems = ItemIndex;
			break;
		}
	}

	// pick only one, discard others
	PickSingleItem(UAISystem::GetRandomStream().RandHelper(NumBestItems));
}

void FEnvQueryInstance::PickSingleItem(int32 ItemIndex)
{
	check(Items.Num() > 0);

	if (Items.IsValidIndex(ItemIndex) == false)
	{
		UE_LOGF(LogEQS, Warning
			, "Query [%ls] tried to pick item %d as best item, but this index is out of scope (num items: %d). Falling back to item 0."
			, *QueryName, ItemIndex, Items.Num());
		ItemIndex = 0;
	}

	FEnvQueryItem BestItem;
	// Copy the score from the actual item rather than just putting "1".  That way, it will correctly show cases where
	// the final filtering test was skipped by an item (and therefore not failed, i.e. passed).
	BestItem.Score = Items[ItemIndex].Score;
	BestItem.DataOffset = Items[ItemIndex].DataOffset;

	DEC_MEMORY_STAT_BY(STAT_AI_EQS_InstanceMemory, Items.GetAllocatedSize());

#if USE_EQS_DEBUGGER
	if (bStoreDebugInfo)
	{
		Items.Swap(0, ItemIndex);
		ItemDetails.Swap(0, ItemIndex);

		DebugData.bSingleItemResult = true;

		// do not limit valid items amount for debugger purposes!
		// bFoundSingleResult can be used to determine if more than one item is valid

		// normalize all scores for debugging
		//NormalizeScores();
	}
	else
	{
#endif
		Items.Empty(1);
		Items.Add(BestItem);
		NumValidItems = 1;
#if USE_EQS_DEBUGGER
	}
#endif

	INC_MEMORY_STAT_BY(STAT_AI_EQS_InstanceMemory, Items.GetAllocatedSize());

}

void FEnvQueryInstance::FinalizeQuery()
{
	if (NumValidItems > 0)
	{
		EEnvQueryResultNormalizationOption NormalizationOption = EEnvQueryResultNormalizationOption::Default;
		if (const UEnvQueryGenerator* Generator = Options[OptionIndex].Generator)
		{
			NormalizationOption = Generator->GetNormalizationOption();
		}

		if (Mode == EEnvQueryRunMode::SingleResult)
		{
			// if last test was not pure condition: sort and pick one of best items
			if (bFoundSingleResult == false && bPassOnSingleResult == false)
			{
				SortScores();

				switch (NormalizationOption)
				{
				case EEnvQueryResultNormalizationOption::Default: // falling through on purpose
				case EEnvQueryResultNormalizationOption::Unaltered:
					break;
				case EEnvQueryResultNormalizationOption::Normalized:
					NormalizeScores();
					break;
				}

				PickSingleItem(0);
			}
		}
		else if (Mode == EEnvQueryRunMode::RandomBest5Pct || Mode == EEnvQueryRunMode::RandomBest25Pct)
		{
			SortScores();

			switch (NormalizationOption)
			{
			case EEnvQueryResultNormalizationOption::Default: // falling through on purpose
			case EEnvQueryResultNormalizationOption::Unaltered:
				break;
			case EEnvQueryResultNormalizationOption::Normalized:
				NormalizeScores();
				break;
			}

			const float ScoreRangePct = (Mode == EEnvQueryRunMode::RandomBest5Pct) ? 0.95f : 0.75f;
			PickRandomItemOfScoreAtLeast(Items[0].Score * ScoreRangePct);
		}
		else
		{
			SortScores();

			DEC_MEMORY_STAT_BY(STAT_AI_EQS_InstanceMemory, Items.GetAllocatedSize());

			// remove failed ones from Items
			Items.SetNum(NumValidItems);

			INC_MEMORY_STAT_BY(STAT_AI_EQS_InstanceMemory, Items.GetAllocatedSize());

			// normalizing after scoring and reducing number of elements to not 
			// do anything for discarded items
			switch (NormalizationOption)
			{
			case EEnvQueryResultNormalizationOption::Unaltered:
				break;
			case EEnvQueryResultNormalizationOption::Default: // falling through on purpose
			case EEnvQueryResultNormalizationOption::Normalized:
				NormalizeScores();
				break;
			}
		}

		MarkAsFinishedWithoutIssues();
	}
	else
	{
		Items.Reset();
		MarkAsFailed();
	}
}

void FEnvQueryInstance::FinalizeGeneration()
{
	FEnvQueryOptionInstance& OptionInstance = Options[OptionIndex];
	const int32 NumTests = OptionInstance.Tests.Num();

	DEC_MEMORY_STAT_BY(STAT_AI_EQS_InstanceMemory, ItemDetails.GetAllocatedSize());
	INC_DWORD_STAT_BY(STAT_AI_EQS_NumItems, Items.Num());

	NumValidItems = Items.Num();
	ItemDetails.Reset();
	bFoundSingleResult = false;

	if (NumValidItems > 0)
	{
		ItemDetails.Reserve(NumValidItems);
		for (int32 ItemIndex = 0; ItemIndex < NumValidItems; ItemIndex++)
		{
			ItemDetails.Add(FEnvQueryItemDetails(NumTests, ItemIndex));
		}
	}

	INC_MEMORY_STAT_BY(STAT_AI_EQS_InstanceMemory, ItemDetails.GetAllocatedSize());

	ItemTypeVectorCDO = (ItemType && ItemType->IsChildOf(UEnvQueryItemType_VectorBase::StaticClass())) ?
		ItemType->GetDefaultObject<UEnvQueryItemType_VectorBase>() : NULL;

	ItemTypeActorCDO = (ItemType && ItemType->IsChildOf(UEnvQueryItemType_ActorBase::StaticClass())) ?
		ItemType->GetDefaultObject<UEnvQueryItemType_ActorBase>() : NULL;
}

void FEnvQueryInstance::FinalizeTest()
{
	FEnvQueryOptionInstance& OptionInstance = Options[OptionIndex];
	const int32 NumTests = OptionInstance.Tests.Num();

	UEnvQueryTest* TestOb = OptionInstance.Tests[CurrentTest];
#if USE_EQS_DEBUGGER
	if (bStoreDebugInfo)
	{
		DebugData.PerformedTestNames.Add(UEnvQueryTypes::GetShortTypeName(TestOb).ToString());
	}
#endif

	// if it's not the last and final test
	if (IsInSingleItemFinalSearch() == false)
	{
		// do regular normalization
		TestOb->NormalizeItemScores(*this);
	}
	else
	{
#if USE_EQS_DEBUGGER
		if (bStoreDebugInfo == false)
#endif // USE_EQS_DEBUGGER
			ItemDetails.Reset();	// mind the "if (bStoreDebugInfo == false)" above
	}
}

#if STATS
uint32 FEnvQueryInstance::GetAllocatedSize() const
{
	SIZE_T MemSize = sizeof(*this) + Items.GetAllocatedSize() + RawData.GetAllocatedSize();
	MemSize += GetContextAllocatedSize();
	MemSize += NamedParams.GetAllocatedSize();
	MemSize += ItemDetails.GetAllocatedSize();
	MemSize += Options.GetAllocatedSize();

	for (int32 OptionCount = 0; OptionCount < Options.Num(); OptionCount++)
	{
		MemSize += Options[OptionCount].GetAllocatedSize();
	}

	return IntCastChecked<uint32>(MemSize);
}

uint32 FEnvQueryInstance::GetContextAllocatedSize() const
{
	SIZE_T MemSize = ContextCache.GetAllocatedSize();
	for (const FEnvQueryInstanceContextCacheItem& CacheItem : ContextCache)
	{
		MemSize += CacheItem.ContextData.GetAllocatedSize();
	}

	return IntCastChecked<uint32>(MemSize);
}
#endif // STATS

FBox FEnvQueryInstance::GetBoundingBox() const
{
	const TArray<FEnvQueryItem>& QueryItems = 
#if USE_EQS_DEBUGGER
	DebugData.DebugItems.Num() > 0 ? DebugData.DebugItems : 
#endif // USE_EQS_DEBUGGER
		Items;

	const TArray<uint8>& QueryRawData = 
#if USE_EQS_DEBUGGER
		DebugData.RawData.Num() > 0 ? DebugData.RawData : 
#endif // USE_EQS_DEBUGGER
		RawData;

	FBox BBox(ForceInit);

	if (ItemType->IsChildOf(UEnvQueryItemType_VectorBase::StaticClass()))
	{
		UEnvQueryItemType_VectorBase* DefTypeOb = ItemType->GetDefaultObject<UEnvQueryItemType_VectorBase>();

		for (int32 Index = 0; Index < QueryItems.Num(); ++Index)
		{		
			BBox += DefTypeOb->GetItemLocation(QueryRawData.GetData() + QueryItems[Index].DataOffset);
		}
	}

	return BBox;
}

#undef eqs_ensure
