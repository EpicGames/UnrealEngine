// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/EnvQueryInstanceBlueprintWrapper.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_ActorBase.h"
#include "EnvironmentQuery/EnvQueryManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryInstanceBlueprintWrapper)

UEnvQueryInstanceBlueprintWrapper::UEnvQueryInstanceBlueprintWrapper(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, QueryID(INDEX_NONE)
{

}

void UEnvQueryInstanceBlueprintWrapper::OnQueryFinished(TSharedPtr<FEnvQueryResult> Result)
{
	check(Result.IsValid());
	QueryResult = Result;
	ItemType = Result->ItemType;
	OptionIndex = Result->OptionIndex;

	OnQueryFinishedEvent.Broadcast(this, Result->GetRawStatus());

	// remove our reference to the query instance
	QueryInstance = nullptr;

	// unregister self, no longer shielded from GC
	UEnvQueryManager* EnvQueryManager = Cast<UEnvQueryManager>(GetOuter());
	if (ensure(EnvQueryManager))
	{
		EnvQueryManager->UnregisterActiveWrapper(*this);
	}
}

float UEnvQueryInstanceBlueprintWrapper::GetItemScore(int32 ItemIndex) const
{
	return QueryResult.IsValid() ? QueryResult->GetItemScore(ItemIndex) : -1.f;
}

bool UEnvQueryInstanceBlueprintWrapper::GetQueryResultsAsActors(TArray<AActor*>& ResultActors) const
{
	ResultActors.Empty();

	if (QueryResult.IsValid() && 
		(QueryResult->GetRawStatus() == EEnvQueryStatus::Success) &&
		(ItemType.Get() != nullptr) &&
		ItemType->IsChildOf(UEnvQueryItemType_ActorBase::StaticClass()))
	{
		if (RunMode != EEnvQueryRunMode::AllMatching)
		{
			ResultActors.Add(QueryResult->GetItemAsActor(0));
		}
		else
		{
			QueryResult->GetAllAsActors(ResultActors);
		}

		return (ResultActors.Num() > 0);
	}

	return false;
}

TArray<AActor*> UEnvQueryInstanceBlueprintWrapper::GetResultsAsActors() const
{
	TArray<AActor*> Results;

	if (QueryResult.IsValid() && 
		(ItemType.Get() != nullptr) &&
		ItemType->IsChildOf(UEnvQueryItemType_ActorBase::StaticClass()))
	{
		if (RunMode != EEnvQueryRunMode::AllMatching)
		{
			Results.Add(QueryResult->GetItemAsActor(0));
		}
		else
		{
			QueryResult->GetAllAsActors(Results);
		}
	}

	return Results;
}

bool UEnvQueryInstanceBlueprintWrapper::GetQueryResultsAsLocations(TArray<FVector>& ResultLocations) const
{
	ResultLocations.Empty();

	if (QueryResult.IsValid() &&
		(QueryResult->GetRawStatus() == EEnvQueryStatus::Success))
	{
		if (RunMode != EEnvQueryRunMode::AllMatching)
		{
			ResultLocations.Add(QueryResult->GetItemAsLocation(0));
		}
		else
		{
			QueryResult->GetAllAsLocations(ResultLocations);
		}

		return (ResultLocations.Num() > 0);
	}

	return false;
}

TArray<FVector> UEnvQueryInstanceBlueprintWrapper::GetResultsAsLocations() const
{
	TArray<FVector> Results;

	if (QueryResult.IsValid())
	{
		if (RunMode != EEnvQueryRunMode::AllMatching)
		{
			Results.Add(QueryResult->GetItemAsLocation(0));
		}
		else
		{
			QueryResult->GetAllAsLocations(Results);
		}
	}

	return Results;
}

void UEnvQueryInstanceBlueprintWrapper::RunQuery(const EEnvQueryRunMode::Type InRunMode, FEnvQueryRequest& QueryRequest)
{
	RunMode = InRunMode;
	QueryID = QueryRequest.Execute(RunMode, this, &UEnvQueryInstanceBlueprintWrapper::OnQueryFinished);
	if (QueryID != INDEX_NONE)
	{
		// register self as a wrapper needing shielding from GC
		UEnvQueryManager* EnvQueryManager = Cast<UEnvQueryManager>(GetOuter());
		if (ensure(EnvQueryManager))
		{
			EnvQueryManager->RegisterActiveWrapper(*this);
			QueryInstance = EnvQueryManager->FindQueryInstance(QueryID);
		}
	}
}

void UEnvQueryInstanceBlueprintWrapper::SetNamedParam(FName ParamName, float Value)
{
	FEnvQueryInstance* InstancePtr = QueryInstance.Get();
	if (InstancePtr != nullptr)
	{
		InstancePtr->NamedParams.Add(ParamName, Value);
	}
}

void UEnvQueryInstanceBlueprintWrapper::SetInstigator(const UObject* Object)
{
#if !UE_BUILD_SHIPPING
	Instigator = Object;
#endif // !UE_BUILD_SHIPPING
}

bool UEnvQueryInstanceBlueprintWrapper::IsSupportedForNetworking() const
{
	// this object can't be replicated, but there are dragons in the land of blueprint...
	FEnvQueryInstance* InstancePtr = QueryInstance.Get();
	FString InstigatorName = "not available in shipping";
#if !UE_BUILD_SHIPPING
	InstigatorName = *GetNameSafe(Instigator.Get());
#endif // !UE_BUILD_SHIPPING

	UE_LOGF(LogEQS, Warning, "%ls can't be replicated over network!\n- Query: %ls\n- Querier:%ls\n- Instigator:%ls",
		*GetName(),
		InstancePtr ? *InstancePtr->QueryName : TEXT("instance not found"),
		InstancePtr ? *GetNameSafe(InstancePtr->Owner.Get()) : TEXT("instance not found"),
		*InstigatorName);

	return false;
}

