// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/EnvQueryManager.h"
#include "UObject/UObjectIterator.h"
#include "EngineGlobals.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Controller.h"
#include "AISystem.h"
#include "VisualLogger/VisualLogger.h"
#include "EnvironmentQuery/EnvQuery.h"
#include "EnvironmentQuery/EnvQueryTest.h"
#include "EnvironmentQuery/EnvQueryGenerator.h"
#include "EnvironmentQuery/EnvQueryOption.h"
#include "EnvironmentQuery/EQSTestingPawn.h"
#include "EnvironmentQuery/EnvQueryDebugHelpers.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Float.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Int.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Bool.h"
#include "Engine/Engine.h"
#include "UObject/UObjectHash.h"
#include "UObject/Package.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "ProfilingDebugging/CsvProfiler.h"


#if WITH_EDITOR
#include "Editor/EditorEngine.h"
#include "EngineUtils.h"

extern UNREALED_API UEditorEngine* GEditor;
#endif // WITH_EDITOR
#include "Misc/TimeGuard.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryManager)

DEFINE_LOG_CATEGORY(LogEQS);

DEFINE_STAT(STAT_AI_EQS_Tick);
DEFINE_STAT(STAT_AI_EQS_TickWork);
DEFINE_STAT(STAT_AI_EQS_TickNotifies);
DEFINE_STAT(STAT_AI_EQS_TickQueryRemovals);
DEFINE_STAT(STAT_AI_EQS_LoadTime);
DEFINE_STAT(STAT_AI_EQS_ExecuteOneStep);
DEFINE_STAT(STAT_AI_EQS_GeneratorTime);
DEFINE_STAT(STAT_AI_EQS_TestTime);
DEFINE_STAT(STAT_AI_EQS_NumInstances);
DEFINE_STAT(STAT_AI_EQS_NumItems);
DEFINE_STAT(STAT_AI_EQS_InstanceMemory);
DEFINE_STAT(STAT_AI_EQS_AvgInstanceResponseTime);
DEFINE_STAT(STAT_AI_EQS_Debug_StoreQuery);
DEFINE_STAT(STAT_AI_EQS_Debug_StoreTickTime);
DEFINE_STAT(STAT_AI_EQS_Debug_StoreStats);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	bool UEnvQueryManager::bAllowEQSTimeSlicing = true;
#endif

#if USE_EQS_DEBUGGER
	TMap<FName, FEQSDebugger::FStatsInfo> UEnvQueryManager::DebuggerStats;
#endif


//////////////////////////////////////////////////////////////////////////
// FEnvQueryRequest

FEnvQueryRequest& FEnvQueryRequest::SetNamedParams(const TArray<FEnvNamedValue>& Params)
{
	for (int32 ParamIndex = 0; ParamIndex < Params.Num(); ParamIndex++)
	{
		NamedParams.Add(Params[ParamIndex].ParamName, Params[ParamIndex].Value);
	}

	return *this;
}

FEnvQueryRequest& FEnvQueryRequest::SetDynamicParam(const FAIDynamicParam& Param, const UBlackboardComponent* BlackboardComponent)
{
	checkf(BlackboardComponent || (Param.BBKey.IsSet() == false), TEXT("BBKey.IsSet but no BlackboardComponent provided"));
	ensureMsgf(Param.bAllowBBKey || !Param.BBKey.IsSet(), TEXT("Dynamic Param %s for query template %s doesn't allow BBKey but has a bbkey set."), *Param.ParamName.ToString(), *GetNameSafe(QueryTemplate));

	// check if given param requires runtime resolve, like reading from BB
	if (Param.bAllowBBKey && Param.BBKey.IsSet() && BlackboardComponent)
	{
		// grab info from BB
		switch (Param.ParamType)
		{
		case EAIParamType::Float:
		{
			const float Value = BlackboardComponent->GetValue<UBlackboardKeyType_Float>(Param.BBKey.GetSelectedKeyID());
			SetFloatParam(Param.ParamName, Value);
		}
		break;
		case EAIParamType::Int:
		{
			const int32 Value = BlackboardComponent->GetValue<UBlackboardKeyType_Int>(Param.BBKey.GetSelectedKeyID());
			SetIntParam(Param.ParamName, Value);
		}
		break;
		case EAIParamType::Bool:
		{
			const bool Value = BlackboardComponent->GetValue<UBlackboardKeyType_Bool>(Param.BBKey.GetSelectedKeyID());
			SetBoolParam(Param.ParamName, Value);
		}
		break;
		default:
			checkNoEntry();
			break;
		}
	}
	else
	{
		switch (Param.ParamType)
		{
		case EAIParamType::Float:
		{
			SetFloatParam(Param.ParamName, Param.Value);
		}
		break;
		case EAIParamType::Int:
		{
			SetIntParam(Param.ParamName, static_cast<int32>(Param.Value));
		}
		break;
		case EAIParamType::Bool:
		{
			bool Result = Param.Value > 0.;
			SetBoolParam(Param.ParamName, Result);
		}
		break;
		default:
			checkNoEntry();
			break;
		}
	}

	return *this;
}

int32 FEnvQueryRequest::Execute(EEnvQueryRunMode::Type RunMode, FQueryFinishedSignature const& FinishDelegate)
{
	if (Owner == NULL)
	{
		Owner = FinishDelegate.GetUObject();
		if (Owner == NULL)
		{
			UE_LOGF(LogEQS, Warning, "Unknown owner of request: %ls", *GetNameSafe(QueryTemplate));
			return INDEX_NONE;
		}
	}

	if (World == NULL)
	{
		World = GEngine->GetWorldFromContextObject(Owner, EGetWorldErrorMode::ReturnNull);
		if (World == NULL)
		{
			UE_LOGF(LogEQS, Warning, "Unable to access world with owner: %ls", *GetNameSafe(Owner));
			return INDEX_NONE;
		}
	}

	UEnvQueryManager* EnvQueryManager = UEnvQueryManager::GetCurrent(World);
	if (EnvQueryManager == NULL)
	{
		UE_LOGF(LogEQS, Warning, "Missing EQS manager!");
		return INDEX_NONE;
	}

	return EnvQueryManager->RunQuery(*this, RunMode, FinishDelegate);
}


//////////////////////////////////////////////////////////////////////////
// UEnvQueryManager

TArray<TSubclassOf<UEnvQueryItemType> > UEnvQueryManager::RegisteredItemTypes;

UEnvQueryManager::UEnvQueryManager(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NextQueryID = 0;
	MaxAllowedTestingTime = 0.01f;
	bTestQueriesUsingBreadth = true;
	NumRunningQueriesAbortedSinceLastUpdate = 0;

	QueryCountWarningThreshold = 0;
	QueryCountWarningInterval = 30.0;
#if !(UE_BUILD_SHIPPING)
	LastQueryCountWarningThresholdTime = -FLT_MAX;
#endif

#if USE_EQS_DEBUGGER
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		UEnvQueryManager::DebuggerStats.Empty();
	}
#endif
}

void UEnvQueryManager::PostLoad()
{
	Super::PostLoad();
	MarkAsGarbage();
}

void UEnvQueryManager::PostInitProperties()
{
	Super::PostInitProperties();
#if WITH_EDITOR
	if (GEditor)
	{
		OnBlueprintCompiledHandle = GEditor->OnBlueprintCompiled().AddUObject(this, &UEnvQueryManager::OnBlueprintCompiled);
	}
#endif // WITH_EDITOR
}

void UEnvQueryManager::FinishDestroy()
{
	FCoreUObjectDelegates::PreLoadMap.RemoveAll(this);
	Super::FinishDestroy();
}

UEnvQueryManager* UEnvQueryManager::GetCurrent(UWorld* World)
{
	UAISystem* AISys = UAISystem::GetCurrentSafe(World);
	return AISys ? AISys->GetEnvironmentQueryManager() : nullptr;
}

UEnvQueryManager* UEnvQueryManager::GetCurrent(const UObject* WorldContextObject)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	UAISystem* AISys = UAISystem::GetCurrentSafe(World);
	return AISys ? AISys->GetEnvironmentQueryManager() : nullptr;
}

#if USE_EQS_DEBUGGER
void UEnvQueryManager::NotifyAssetUpdate(UEnvQuery* Query)
{
#if WITH_EDITOR
	if (GEditor == NULL)
	{
		return;
	}

	for (const FWorldContext& WorldContext : GEditor->GetWorldContexts())
	{
		if (UWorld* World = WorldContext.World())
		{
			UEnvQueryManager* EQS = UEnvQueryManager::GetCurrent(World);
			if (EQS)
			{
				EQS->InstanceCache.Reset();
			}

			// was as follows, but got broken with changes to actor iterator (FActorIteratorBase::SpawnedActorArray)
			// for (TActorIterator<AEQSTestingPawn> It(World); It; ++It)
			for (TActorIterator<AActor> It(World); It; ++It)
			{
				AEQSTestingPawn* EQSPawn = Cast<AEQSTestingPawn>(*It);
				if (EQSPawn == NULL)
				{
					continue;
				}

				if (EQSPawn->QueryTemplate == Query || Query == NULL)
				{
					EQSPawn->RunEQSQuery();
				}
			}
		}
	}
#endif //WITH_EDITOR
}
#endif // USE_EQS_DEBUGGER

TStatId UEnvQueryManager::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UEnvQueryManager, STATGROUP_Tickables);
}

int32 UEnvQueryManager::RunQuery(const FEnvQueryRequest& Request, EEnvQueryRunMode::Type RunMode, FQueryFinishedSignature const& FinishDelegate)
{
	TSharedPtr<FEnvQueryInstance> QueryInstance = PrepareQueryInstance(Request, RunMode);
	return RunQuery(QueryInstance, FinishDelegate);
}

int32 UEnvQueryManager::RunQuery(const TSharedPtr<FEnvQueryInstance>& QueryInstance, FQueryFinishedSignature const& FinishDelegate)
{
	if (QueryInstance.IsValid() == false)
	{
		return INDEX_NONE;
	}

	QueryInstance->FinishDelegate = FinishDelegate;
	RunningQueries.Add(QueryInstance);

	UE_VLOG_UELOG(QueryInstance->Owner.Get(), LogEQS, Verbose, TEXT("%hs: Query: %s - Owner: %s"),
		__FUNCTION__,
		*QueryInstance->QueryName,
		*GetNameSafe(QueryInstance->Owner.Get()));

	return QueryInstance->QueryID;
}

TSharedPtr<FEnvQueryResult> UEnvQueryManager::RunInstantQuery(const FEnvQueryRequest& Request, EEnvQueryRunMode::Type RunMode)
{
	TSharedPtr<FEnvQueryInstance> QueryInstance = PrepareQueryInstance(Request, RunMode);
	if (!QueryInstance.IsValid())
	{
		return NULL;
	}

	RunInstantQuery(QueryInstance);

	return QueryInstance;
}

void UEnvQueryManager::RunInstantQuery(const TSharedPtr<FEnvQueryInstance>& QueryInstance)
{
	if (! ensure(QueryInstance.IsValid()))
	{
		return;
	}

	UE_VLOG_UELOG(QueryInstance->Owner.Get(), LogEQS, Verbose, TEXT("%hs: Query: %s - Owner: %s"),
		__FUNCTION__,
		*QueryInstance->QueryName,
		*GetNameSafe(QueryInstance->Owner.Get()));

	{
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(EnvQueryManager);

		RegisterExternalQuery(QueryInstance);
		while (QueryInstance->IsFinished() == false)
		{
			QueryInstance->ExecuteOneStep(UEnvQueryTypes::UnlimitedStepTime);
		}

		UnregisterExternalQuery(QueryInstance);
	}

	UE_VLOG_EQS(*QueryInstance.Get(), LogEQS, All);

#if USE_EQS_DEBUGGER
	EQSDebugger.StoreQuery(QueryInstance);
#endif // USE_EQS_DEBUGGER
}

void UEnvQueryManager::RemoveAllQueriesByQuerier(const UObject& Querier, bool bExecuteFinishDelegate)
{
	for (int32 QueryIndex = RunningQueries.Num() - 1; QueryIndex >= 0; --QueryIndex)
	{
		const TSharedPtr<FEnvQueryInstance>& QueryInstance = RunningQueries[QueryIndex];
		if (QueryInstance.IsValid() == false || QueryInstance->Owner.IsValid() == false || QueryInstance->Owner.Get() == &Querier)
		{
			if (QueryInstance->IsFinished() == false)
			{
				QueryInstance->MarkAsAborted();

				UE_VLOG_UELOG(QueryInstance->Owner.Get(), LogEQS, Verbose, TEXT("%hs: Query: %s - Owner: %s"),
					__FUNCTION__,
					*QueryInstance->QueryName,
					*GetNameSafe(QueryInstance->Owner.Get()));

				if (bExecuteFinishDelegate)
				{
					QueryInstance->FinishDelegate.ExecuteIfBound(QueryInstance);
				}

				// We will remove the aborted query from the RunningQueries array on the next EQS update
				++NumRunningQueriesAbortedSinceLastUpdate;
			}
		}
	}
}

TSharedPtr<FEnvQueryInstance> UEnvQueryManager::PrepareQueryInstance(const FEnvQueryRequest& Request, EEnvQueryRunMode::Type RunMode)
{
	TSharedPtr<FEnvQueryInstance> QueryInstance = CreateQueryInstance(Request.QueryTemplate, RunMode);
	if (!QueryInstance.IsValid())
	{
		UE_VLOG_UELOG(Request.Owner.Get(), LogEQS, Warning, TEXT("Error creating query instance for QueryTemplate: %s - Owner: %s"),
			Request.QueryTemplate != nullptr ? *Request.QueryTemplate->QueryName.ToString() : TEXT("NONE"),
			*GetNameSafe(Request.Owner));

		return nullptr;
	}

	QueryInstance->World = GetWorldFast();
	QueryInstance->Owner = Request.Owner;
	QueryInstance->StartTime = FPlatformTime::Seconds();
#if !UE_BUILD_SHIPPING
	QueryInstance->GenerationTimeWarningSeconds = GenerationTimeWarningSeconds;
#endif // UE_BUILD_SHIPPING

	DEC_MEMORY_STAT_BY(STAT_AI_EQS_InstanceMemory, QueryInstance->NamedParams.GetAllocatedSize());

	// @TODO: interface for providing default named params (like custom ranges in AI)
	QueryInstance->NamedParams = Request.NamedParams;

	INC_MEMORY_STAT_BY(STAT_AI_EQS_InstanceMemory, QueryInstance->NamedParams.GetAllocatedSize());

	QueryInstance->QueryID = NextQueryID++;

	return QueryInstance;
}

bool UEnvQueryManager::AbortQuery(int32 RequestID)
{
	for (int32 QueryIndex = 0; QueryIndex < RunningQueries.Num(); QueryIndex++)
	{
		TSharedPtr<FEnvQueryInstance>& QueryInstance = RunningQueries[QueryIndex];
		if (QueryInstance->QueryID == RequestID &&
			QueryInstance->IsFinished() == false)
		{
			UE_VLOG_UELOG(QueryInstance->Owner.Get(), LogEQS, Verbose, TEXT("%hs: Query: %s - Owner: %s"),
				__FUNCTION__,
				*QueryInstance->QueryName,
				*GetNameSafe(QueryInstance->Owner.Get()));

			QueryInstance->MarkAsAborted();
			QueryInstance->FinishDelegate.ExecuteIfBound(QueryInstance);
			
			// We will remove the aborted query from the RunningQueries array on the next EQS update
			++NumRunningQueriesAbortedSinceLastUpdate;

			return true;
		}
	}

	return false;
}

void UEnvQueryManager::Tick(float DeltaTime)
{
	SCOPE_TIME_GUARD_MS(TEXT("UEnvQueryManager::Tick"), 10);
	SCOPE_CYCLE_COUNTER(STAT_AI_EQS_Tick);
	SET_DWORD_STAT(STAT_AI_EQS_NumInstances, RunningQueries.Num());

	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(EnvQueryManager);

#if !(UE_BUILD_SHIPPING)
	CheckQueryCount();
#endif

	double TimeLeft = MaxAllowedTestingTime;
	int32 QueriesFinishedDuringUpdate = 0;

	{
		SCOPE_CYCLE_COUNTER(STAT_AI_EQS_TickWork);
		
		const int32 NumRunningQueries = RunningQueries.Num();
		int32 NumQueriesRunningAsync = 0;
		int32 Index = 0;

		while ((TimeLeft > 0.0) 
			&& (Index < NumRunningQueries) 
			// make sure we account for queries that have finished (been aborted)
			// before UEnvQueryManager::Tick has been called
			&& (QueriesFinishedDuringUpdate + NumRunningQueriesAbortedSinceLastUpdate + NumQueriesRunningAsync < NumRunningQueries))
		{
			const double StepStartTime = FPlatformTime::Seconds();
			double ResultHandlingDuration = 0.;
#if USE_EQS_DEBUGGER
			bool bWorkHasBeenDone = false;
#endif // USE_EQS_DEBUGGER

			TSharedPtr<FEnvQueryInstance> QueryInstance = RunningQueries[Index];
			FEnvQueryInstance* QueryInstancePtr = QueryInstance.Get();
			if (QueryInstancePtr == nullptr || QueryInstancePtr->IsFinished())
			{
				// If this query is already finished, skip it.
				++Index;
			}
			else
			{
#if STATS
				FScopeCycleCounterUObject OwnerScopeCounter(QueryInstance->Owner.Get());
				FScopeCycleCounter QueryScopeCounter(QueryInstance->StatId);
#endif // STATS

				QueryInstancePtr->ExecuteOneStep(TimeLeft);

#if USE_EQS_DEBUGGER
				bWorkHasBeenDone = true;
#endif // USE_EQS_DEBUGGER
				if (QueryInstancePtr->IsFinished())
				{
					// Always log that we executed total execution time at the end of the query.
					if (QueryInstancePtr->TotalExecutionTime > ExecutionTimeWarningSeconds)
					{
						UE_VLOG_UELOG(QueryInstance->Owner.Get(), LogEQS, Warning, TEXT("Query %s (Owner: %s) has finished in %.2f ms, exceeding the configured limit of %.2f ms. Execution details: %s"), 
							*QueryInstancePtr->QueryName, *GetNameSafe(QueryInstancePtr->Owner.Get()), 
							1000.f * QueryInstancePtr->TotalExecutionTime, 1000.f * ExecutionTimeWarningSeconds, 
							*QueryInstancePtr->GetExecutionTimeDescription());

						QueryInstancePtr->bHasLoggedTimeLimitWarning = true;
					}

					// Now, handle the response to the query finishing, but calculate the time from that to remove from
					// the time spent for time-slicing purposes, because that's NOT the EQS manager doing work.
					{
						SCOPE_CYCLE_COUNTER(STAT_AI_EQS_TickNotifies);
						const double ResultHandlingStartTime = FPlatformTime::Seconds();
	
						UE_VLOG_EQS(*QueryInstancePtr, LogEQS, All);

#if USE_EQS_DEBUGGER
						EQSDebugger.StoreStats(*QueryInstancePtr);
						EQSDebugger.StoreQuery(QueryInstance);
#endif // USE_EQS_DEBUGGER

						UE_VLOG_UELOG(QueryInstance->Owner.Get(), LogEQS, Verbose, TEXT("%hs: Finished Query: %s - Owner: %s"),
							__FUNCTION__,
							*QueryInstance->QueryName,
							*GetNameSafe(QueryInstance->Owner.Get()));

						QueryInstancePtr->FinishDelegate.ExecuteIfBound(QueryInstance);

						ResultHandlingDuration = FPlatformTime::Seconds() - ResultHandlingStartTime;

						// Always log if FinishDelegate took too long to handle the result
						if (ResultHandlingDuration > HandlingResultTimeWarningSeconds)
						{
							FName FunctionName("Unavailable");
	#if USE_DELEGATE_TRYGETBOUNDFUNCTIONNAME
							FunctionName = QueryInstancePtr->FinishDelegate.TryGetBoundFunctionName();
	#endif // USE_DELEGATE_TRYGETBOUNDFUNCTIONNAME

							UE_VLOG_UELOG(QueryInstance->Owner.Get(), LogEQS, Warning,
								TEXT("FinishDelegate for EQS query %s took %f seconds and is over handling time limit warning of %f. Delegate info: object = %s method = %s"),
								*QueryInstancePtr->QueryName, ResultHandlingDuration, HandlingResultTimeWarningSeconds,
								*GetNameSafe(QueryInstancePtr->FinishDelegate.GetUObject()),
								*FunctionName.ToString());
						}
					}

					++QueriesFinishedDuringUpdate;
					++Index;
				}

				// If current Query is waiting on results to be processed asynchronously, check for results next frame.
				else if (QueryInstancePtr->IsCurrentlyRunningAsync())
				{
					++NumQueriesRunningAsync;
					++Index;
				}
				// If we're testing queries using breadth, move on to the next query.
				// If we're testing queries using depth, we only move on to the next query when we finish the current one.
				else if (bTestQueriesUsingBreadth)
				{
					++Index;
				}

				if (QueryInstancePtr->TotalExecutionTime > ExecutionTimeWarningSeconds && !QueryInstancePtr->bHasLoggedTimeLimitWarning)
				{
					UE_VLOG_UELOG(QueryInstance->Owner.Get(), LogEQS, Warning, TEXT("Query %s (Owner: %s) has taken %.2f ms so far, exceeding the configured limit of %.2f ms. Execution details: %s"), 
						*QueryInstancePtr->QueryName, *GetNameSafe(QueryInstancePtr->Owner.Get()), 
						1000.f * QueryInstancePtr->TotalExecutionTime, 1000.f * ExecutionTimeWarningSeconds, 
						*QueryInstancePtr->GetExecutionTimeDescription());

					QueryInstancePtr->bHasLoggedTimeLimitWarning = true;
				}
			}

			// Start over at the beginning if we are testing using breadth and we've reached the end of the list
			if (bTestQueriesUsingBreadth && (Index == NumRunningQueries))
			{
				Index = 0;
			}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (bAllowEQSTimeSlicing) // if Time slicing is enabled...
#endif
			{	// Don't include the querier handling as part of the total time spent by EQS for time-slicing purposes.
				const double StepProcessingTime = ((FPlatformTime::Seconds() - StepStartTime) - ResultHandlingDuration);
				TimeLeft -= StepProcessingTime;

#if USE_EQS_DEBUGGER
				// we want to do any kind of logging only if any work has been done
				if (QueryInstancePtr && bWorkHasBeenDone)
				{
					EQSDebugger.StoreTickTime(*QueryInstancePtr, StepProcessingTime, MaxAllowedTestingTime);
				}
#endif // USE_EQS_DEBUGGER
			}
		}
	}

	{
		const int32 NumQueriesFinished = QueriesFinishedDuringUpdate + NumRunningQueriesAbortedSinceLastUpdate;
		double FinishedQueriesTotalTime = 0.;

		if (NumQueriesFinished > 0)
		{
			SCOPE_CYCLE_COUNTER(STAT_AI_EQS_TickQueryRemovals);

			// When using breadth testing we don't know when a particular query will finish,
			// or if we have queries that were aborted since the last update we don't know which ones were aborted,
			// so we have to go through all the queries.
			// When doing depth without any queries aborted since the last update we know how many to remove.
			// Or if we have finished all the queries.  In that case we don't need to check if the queries are finished)
			if ((NumQueriesFinished != RunningQueries.Num()) && (bTestQueriesUsingBreadth || (NumRunningQueriesAbortedSinceLastUpdate > 0)))
			{
				for (int32 Index = RunningQueries.Num() - 1, FinishedQueriesCounter = NumQueriesFinished; Index >= 0 && FinishedQueriesCounter > 0; --Index)
				{
					TSharedPtr<FEnvQueryInstance>& QueryInstance = RunningQueries[Index];
					if (!QueryInstance.IsValid())
					{
						RunningQueries.RemoveAt(Index, EAllowShrinking::No);
						continue;
					}

					if (QueryInstance->IsFinished())
					{
						FinishedQueriesTotalTime += (FPlatformTime::Seconds() - QueryInstance->StartTime);
						RunningQueries.RemoveAt(Index, EAllowShrinking::No);
						--FinishedQueriesCounter;
					}
				}
			}
			else // queries tested using depth without any aborted queries since our last update, or we're removing all queries
			{
				for (int32 Index = 0; Index < NumQueriesFinished; ++Index)
				{
					TSharedPtr<FEnvQueryInstance>& QueryInstance = RunningQueries[Index];
					ensure(QueryInstance->IsFinished());

					FinishedQueriesTotalTime += (FPlatformTime::Seconds() - QueryInstance->StartTime);
				}

				RunningQueries.RemoveAt(0, NumQueriesFinished, EAllowShrinking::No);
			}
		}

		// Reset the running queries aborted since last update counter
		NumRunningQueriesAbortedSinceLastUpdate = 0;

		const double InstanceAverageResponseTime = (NumQueriesFinished > 0) ? (1000. * FinishedQueriesTotalTime / NumQueriesFinished) : 0.;
		SET_FLOAT_STAT(STAT_AI_EQS_AvgInstanceResponseTime, InstanceAverageResponseTime);
	}
}

#if !(UE_BUILD_SHIPPING)
void UEnvQueryManager::CheckQueryCount() const
{
	if ((QueryCountWarningThreshold > 0) && (RunningQueries.Num() >= QueryCountWarningThreshold))
	{
		const double CurrentTime = FPlatformTime::Seconds();

		if ((LastQueryCountWarningThresholdTime < 0.0) || ((LastQueryCountWarningThresholdTime + QueryCountWarningInterval) < CurrentTime))
		{
			LogQueryInfo(true /* bDisplayThresholdWarning */);

			LastQueryCountWarningThresholdTime = CurrentTime;
		}
	}
}

void UEnvQueryManager::LogQueryInfo(bool bDisplayThresholdWarning) const
{
	if (bDisplayThresholdWarning)
	{
		UE_VLOG_UELOG(this, LogEQS, Warning, TEXT("The number of EQS queries (%d) has reached the warning threshold (%d).  Logging queries."), RunningQueries.Num(), QueryCountWarningThreshold);
	}
	else
	{
		UE_VLOG_UELOG(this, LogEQS, Warning, TEXT("The number of EQS queries is (%d).  Logging queries."), RunningQueries.Num());
	}

	for (const TSharedPtr<FEnvQueryInstance>& RunningQuery : RunningQueries)
	{
		if (RunningQuery.IsValid())
		{
			UE_VLOG_UELOG(this, LogEQS, Warning, TEXT("Query: %s - Owner: %s - %s"),
				*RunningQuery->QueryName,
				RunningQuery->Owner.IsValid() ? *RunningQuery->Owner->GetName() : TEXT("Invalid"),
				*RunningQuery->GetExecutionTimeDescription());
		}
		else
		{
			UE_VLOG_UELOG(this, LogEQS, Warning, TEXT("Invalid query found in list!"));
		}
	}
}
#endif

void UEnvQueryManager::PrintActiveQueryInfo() const
{
#if !(UE_BUILD_SHIPPING)
	LogQueryInfo(false /* bDisplayThresholdWarning */);
#endif
}

void UEnvQueryManager::OnWorldCleanup()
{
	if (RunningQueries.Num() > 0)
	{
		// @todo investigate if this is even needed. We should be fine with just removing all queries
		TArray<TSharedPtr<FEnvQueryInstance> > RunningQueriesCopy = RunningQueries;
		RunningQueries.Reset();

		for (int32 Index = 0; Index < RunningQueriesCopy.Num(); Index++)
		{
			TSharedPtr<FEnvQueryInstance>& QueryInstance = RunningQueriesCopy[Index];
			if (QueryInstance->IsFinished() == false)
			{
				UE_VLOG_UELOG(QueryInstance->Owner.Get(), LogEQS, Verbose, TEXT("Query failed due to world cleanup: Query: %s - Owner: %s"),
					*QueryInstance->QueryName,
					*GetNameSafe(QueryInstance->Owner.Get()));

				QueryInstance->MarkAsFailed();
				QueryInstance->FinishDelegate.ExecuteIfBound(QueryInstance);
			}
		}
	}

	GCShieldedWrappers.Reset();
}

void UEnvQueryManager::RegisterExternalQuery(const TSharedPtr<FEnvQueryInstance>& QueryInstance)
{
	if (QueryInstance.IsValid())
	{
		ExternalQueries.Add(QueryInstance->QueryID, QueryInstance);
	}
}

void UEnvQueryManager::UnregisterExternalQuery(const TSharedPtr<FEnvQueryInstance>& QueryInstance)
{
	if (QueryInstance.IsValid())
	{
		ExternalQueries.Remove(QueryInstance->QueryID);
	}
}

namespace EnvQueryTestSort
{
	struct FAllMatching
	{
		FORCEINLINE_DEBUGGABLE bool operator()(const UEnvQueryTest& TestA, const UEnvQueryTest& TestB) const
		{
			// conditions go first
			const bool bConditionA = (TestA.TestPurpose != EEnvTestPurpose::Score); // Is Test A Filtering?
			const bool bConditionB = (TestB.TestPurpose != EEnvTestPurpose::Score); // Is Test B Filtering?
			if (bConditionA != bConditionB)
			{
				return bConditionA;
			}

			// cheaper tests go first
			return (TestA.Cost < TestB.Cost);
		}
	};
}

UEnvQuery* UEnvQueryManager::FindQueryTemplate(const FString& QueryName) const
{
	for (int32 InstanceIndex = 0; InstanceIndex < InstanceCache.Num(); InstanceIndex++)
	{
		const UEnvQuery* QueryTemplate = InstanceCache[InstanceIndex].Template;

		if (QueryTemplate != NULL && QueryTemplate->GetName() == QueryName)
		{
			return const_cast<UEnvQuery*>(QueryTemplate);
		}
	}

	for (FThreadSafeObjectIterator It(UEnvQuery::StaticClass()); It; ++It)
	{
		if (It->GetName() == QueryName)
		{
			return Cast<UEnvQuery>(*It);
		}
	}

	return NULL;
}

TSharedPtr<FEnvQueryInstance> UEnvQueryManager::CreateQueryInstance(const UEnvQuery* Template, EEnvQueryRunMode::Type RunMode)
{
	if (Template == nullptr || Template->Options.Num() == 0)
	{
		UE_CVLOG_UELOG(Template != nullptr && Template->Options.Num() == 0, this, LogEQS, Warning, TEXT("Query [%s] doesn't have any valid options!"), *Template->GetName());
		return nullptr;
	}

	const FName TemplateFullName = FName(Template->GetFullName());
	
	// try to find entry in cache
	FEnvQueryInstance* InstanceTemplate = NULL;
	for (int32 InstanceIndex = 0; InstanceIndex < InstanceCache.Num(); InstanceIndex++)
	{
		if (InstanceCache[InstanceIndex].AssetName == TemplateFullName &&
			InstanceCache[InstanceIndex].Instance.Mode == RunMode)
		{
			InstanceTemplate = &InstanceCache[InstanceIndex].Instance;
			break;
		}
	}

	// and create one if can't be found
	if (InstanceTemplate == NULL)
	{
		SCOPE_CYCLE_COUNTER(STAT_AI_EQS_LoadTime);
		static const UEnum* RunModeEnum = StaticEnum<EEnvQueryRunMode::Type>();

		// duplicate template in manager's world for BP based nodes
		const FString NewInstanceName = RunModeEnum
			? FString::Printf(TEXT("%s_%s"), *MakeUniqueObjectName(this, Template->GetClass(), Template->GetFName()).ToString(), *RunModeEnum->GetNameStringByValue(RunMode))
			: FString::Printf(TEXT("%s_%d"), *MakeUniqueObjectName(this, Template->GetClass(), Template->GetFName()).ToString(), uint8(RunMode));
		UEnvQuery* LocalTemplate = (UEnvQuery*)StaticDuplicateObject(Template, this, *NewInstanceName);

		{
			// memory stat tracking: temporary variable will exist only inside this section
			FEnvQueryInstanceCache NewCacheEntry;
			NewCacheEntry.AssetName = TemplateFullName;
			NewCacheEntry.Template = LocalTemplate;
			NewCacheEntry.Instance.UniqueName = LocalTemplate->GetFName();
			NewCacheEntry.Instance.QueryName = LocalTemplate->GetQueryName().ToString();
			NewCacheEntry.Instance.Mode = RunMode;
			STAT(NewCacheEntry.Instance.StatId = FDynamicStats::CreateStatId<FStatGroup_STATGROUP_AI_EQS>(NewCacheEntry.Instance.UniqueName));

			const int32 Idx = InstanceCache.Add(NewCacheEntry);
			InstanceTemplate = &InstanceCache[Idx].Instance;
		}

		// NOTE: We must iterate over this from 0->Num because we are copying the options from the template into the
		// instance, and order matters!  Since we also may need to remove invalid or null options, we must decrement
		// the iteration pointer when doing so to avoid problems.
		int32 SourceOptionIndex = 0;
		for (int32 OptionIndex = 0; OptionIndex < LocalTemplate->Options.Num(); ++OptionIndex, ++SourceOptionIndex)
		{
			UEnvQueryOption* MyOption = LocalTemplate->Options[OptionIndex];
			if (MyOption == nullptr ||
				MyOption->Generator == nullptr ||
				MyOption->Generator->IsValidGenerator() == false)
			{
				if (MyOption == nullptr)
				{
					UE_VLOG_UELOG(this, LogEQS, Error, TEXT("Trying to spawn a query with broken Template (null option): %s, option %d"),
						*GetNameSafe(LocalTemplate), OptionIndex);
				} 
				else if (MyOption->Generator == nullptr)
				{
					UE_VLOG_UELOG(this, LogEQS, Error, TEXT("Trying to spawn a query with broken Template (generator:MISSING): %s, option %d"),
						*GetNameSafe(LocalTemplate), OptionIndex);
				}
				else
				{
					UE_VLOG_UELOG(this, LogEQS, Error, TEXT("Trying to spawn a query with broken Template (generator:%s itemType:%s): %s, option %d"),
						MyOption->Generator->IsValidGenerator() ? TEXT("ok") : TEXT("invalid"),
						MyOption->Generator->ItemType ? TEXT("ok") : TEXT("MISSING"),
						*GetNameSafe(LocalTemplate), OptionIndex);
				}

				LocalTemplate->Options.RemoveAt(OptionIndex, EAllowShrinking::No);
				--OptionIndex; // See note at top of for loop.  We cannot iterate backwards here.
				continue;
			}

			UEnvQueryOption* LocalOption = (UEnvQueryOption*)StaticDuplicateObject(MyOption, this);
			UEnvQueryGenerator* LocalGenerator = (UEnvQueryGenerator*)StaticDuplicateObject(MyOption->Generator, this);
			LocalTemplate->Options[OptionIndex] = LocalOption;
			LocalOption->Generator = LocalGenerator;

			if (MyOption->Tests.Num() == 0)
			{
				// no further work required here
				CreateOptionInstance(LocalOption, SourceOptionIndex, TArray<UEnvQueryTest*>(), *InstanceTemplate);
				continue;
			}

			// check if TestOrder property is set correctly in asset, try to recreate it if not
			// don't use editor graph, so any disabled tests in the middle will make it look weird
			if (MyOption->Tests.Num() > 1 && MyOption->Tests.Last() && MyOption->Tests.Last()->TestOrder == 0)
			{
				for (int32 TestIndex = 0; TestIndex < MyOption->Tests.Num(); TestIndex++)
				{
					if (MyOption->Tests[TestIndex])
					{
						MyOption->Tests[TestIndex]->TestOrder = TestIndex;
					}
				}
			}

			const bool bOptionSingleResultSearch = (RunMode == EEnvQueryRunMode::SingleResult) && LocalGenerator->bAutoSortTests;
			EEnvTestCost::Type HighestFilterCost(EEnvTestCost::Low);
			UEnvQueryTest* MostExpensiveFilter = nullptr;
			int32 MostExpensiveFilterIndex = INDEX_NONE;
			TArray<UEnvQueryTest*> SortedTests = MyOption->Tests;
			TSubclassOf<UEnvQueryItemType> GeneratedType = MyOption->Generator->ItemType;
			for (int32 TestIndex = SortedTests.Num() - 1; TestIndex >= 0; TestIndex--)
			{
				UEnvQueryTest* TestOb = SortedTests[TestIndex];
				if (TestOb == NULL || !TestOb->IsSupportedItem(GeneratedType))
				{
					UE_VLOG_UELOG(this, LogEQS, Warning, TEXT("Query [%s] can't use test [%s] in option %d [%s], removing it"),
						*GetNameSafe(LocalTemplate), *GetNameSafe(TestOb), OptionIndex, *MyOption->Generator->OptionName);

					SortedTests.RemoveAt(TestIndex, EAllowShrinking::No);
					if (MostExpensiveFilterIndex != INDEX_NONE)
					{
						--MostExpensiveFilterIndex;
					}
				}
				else if (bOptionSingleResultSearch
					&& TestOb->TestPurpose == EEnvTestPurpose::Filter
					&& (HighestFilterCost < TestOb->Cost || MostExpensiveFilterIndex == INDEX_NONE))
				{
					HighestFilterCost = TestOb->Cost;
					MostExpensiveFilterIndex = TestIndex;
				}
			}

			LocalOption->Tests.Reset(SortedTests.Num());
			// MostExpensiveFilterIndex can be INDEX_NONE if there are no filtering 
			// tests done (i.e. just scoring tests).
			if (bOptionSingleResultSearch && MostExpensiveFilterIndex != INDEX_NONE)
			{
				// we're going to remove the most expensive test so that 
				// it can be added last after everything else gets sorted
				MostExpensiveFilter = (UEnvQueryTest*)StaticDuplicateObject(SortedTests[MostExpensiveFilterIndex], this);
				for (int32 TestIdx = 0; TestIdx < SortedTests.Num(); TestIdx++)
				{
					if (TestIdx == MostExpensiveFilterIndex)
					{
						continue;
					}
					UEnvQueryTest* LocalTest = (UEnvQueryTest*)StaticDuplicateObject(SortedTests[TestIdx], this);
					LocalOption->Tests.Add(LocalTest);
				}
			}
			else
			{				
				for (int32 TestIdx = 0; TestIdx < SortedTests.Num(); TestIdx++)
				{
					UEnvQueryTest* LocalTest = (UEnvQueryTest*)StaticDuplicateObject(SortedTests[TestIdx], this);
					LocalOption->Tests.Add(LocalTest);
				}
			}

			if (SortedTests.Num() && LocalGenerator->bAutoSortTests)
			{
				LocalOption->Tests.StableSort(EnvQueryTestSort::FAllMatching());
				if (bOptionSingleResultSearch && MostExpensiveFilter)
				{
					// the only difference between running for a single result is that 
					// we want to perform a single most expensive test as the last test
					// to accept the first one that passes the test.
					// so here we're adding that final, most expensive test
					LocalOption->Tests.Add(MostExpensiveFilter);
				}
			}

			CreateOptionInstance(LocalOption, SourceOptionIndex, LocalOption->Tests, *InstanceTemplate);
		}
	}

	if (InstanceTemplate->Options.Num() == 0)
	{
		return nullptr;
	}

	// create new instance
	TSharedPtr<FEnvQueryInstance> NewInstance(new FEnvQueryInstance(*InstanceTemplate));
	return NewInstance;
}

void UEnvQueryManager::CreateOptionInstance(UEnvQueryOption* OptionTemplate, int32 SourceOptionIndex, const TArray<UEnvQueryTest*>& SortedTests, FEnvQueryInstance& Instance)
{
	FEnvQueryOptionInstance OptionInstance;
	OptionInstance.Generator = OptionTemplate->Generator;
	OptionInstance.ItemType = OptionTemplate->Generator->ItemType;
	OptionInstance.SourceOptionIndex = SourceOptionIndex;
	OptionInstance.Tests = SortedTests;

	DEC_MEMORY_STAT_BY(STAT_AI_EQS_InstanceMemory, Instance.Options.GetAllocatedSize());

	const int32 AddedIdx = Instance.Options.Add(OptionInstance);

#if USE_EQS_DEBUGGER
	Instance.DebugData.OptionData.AddDefaulted(1);
	FEnvQueryDebugProfileData::FOptionData& OptionData = Instance.DebugData.OptionData.Last();
	
	OptionData.OptionIdx = SourceOptionIndex;
	for (int32 TestIndex = 0; TestIndex < SortedTests.Num(); TestIndex++)
	{
		UEnvQueryTest* TestOb = SortedTests[TestIndex];
		OptionData.TestIndices.Add(TestOb->TestOrder);
	}
#endif

	INC_MEMORY_STAT_BY(STAT_AI_EQS_InstanceMemory, Instance.Options.GetAllocatedSize() + Instance.Options[AddedIdx].GetAllocatedSize());
}

UEnvQueryContext* UEnvQueryManager::PrepareLocalContext(TSubclassOf<UEnvQueryContext> ContextClass)
{
	UEnvQueryContext* LocalContext = LocalContextMap.FindRef(ContextClass->GetFName());
	if (LocalContext == NULL)
	{
		LocalContext = (UEnvQueryContext*)StaticDuplicateObject(ContextClass.GetDefaultObject(), this);
		LocalContexts.Add(LocalContext);
		LocalContextMap.Add(ContextClass->GetFName(), LocalContext);
	}

	return LocalContext;
}

float UEnvQueryManager::FindNamedParam(int32 QueryId, FName ParamName) const
{
	float ParamValue = 0.0f;

	const TWeakPtr<FEnvQueryInstance>* QueryInstancePtr = ExternalQueries.Find(QueryId);
	if (QueryInstancePtr)
	{
		TSharedPtr<FEnvQueryInstance> QueryInstance = (*QueryInstancePtr).Pin();
		if (QueryInstance.IsValid())
		{
			ParamValue = QueryInstance->NamedParams.FindRef(ParamName);
		}
	}
	else
	{
		for (int32 QueryIndex = 0; QueryIndex < RunningQueries.Num(); QueryIndex++)
		{
			const TSharedPtr<FEnvQueryInstance>& QueryInstance = RunningQueries[QueryIndex];
			if (QueryInstance->QueryID == QueryId)
			{
				ParamValue = QueryInstance->NamedParams.FindRef(ParamName);
				break;
			}
		}
	}

	return ParamValue;
}

//----------------------------------------------------------------------//
// BP functions and related functionality 
//----------------------------------------------------------------------//
UEnvQueryInstanceBlueprintWrapper* UEnvQueryManager::RunEQSQuery(UObject* WorldContextObject, UEnvQuery* QueryTemplate, UObject* Querier, TEnumAsByte<EEnvQueryRunMode::Type> RunMode, TSubclassOf<UEnvQueryInstanceBlueprintWrapper> WrapperClass)
{ 
	if (QueryTemplate == nullptr || Querier == nullptr)
	{
		return nullptr;
	}

	UEnvQueryManager* EQSManager = GetCurrent(WorldContextObject);
	UEnvQueryInstanceBlueprintWrapper* QueryInstanceWrapper = nullptr;

	if (EQSManager)
	{
		bool bValidQuerier = true;

		// convert controller-owners to pawns, unless specifically configured not to do so
		if (GET_AI_CONFIG_VAR(bAllowControllersAsEQSQuerier) == false && Cast<AController>(Querier))
		{
			AController* Controller = Cast<AController>(Querier);
			if (Controller->GetPawn())
			{
				Querier = Controller->GetPawn();
			}
			else
			{
				UE_VLOG(Controller, LogEQS, Error, TEXT("Trying to run EQS query while not having a pawn! Aborting."));
				bValidQuerier = false;
			}
		}

		if (bValidQuerier)
		{
			QueryInstanceWrapper = NewObject<UEnvQueryInstanceBlueprintWrapper>(EQSManager, (UClass*)(WrapperClass) ? (UClass*)WrapperClass : UEnvQueryInstanceBlueprintWrapper::StaticClass());
			check(QueryInstanceWrapper);

			FEnvQueryRequest QueryRequest(QueryTemplate, Querier);
			// @todo named params still missing support
			//QueryRequest.SetNamedParams(QueryParams);
			QueryInstanceWrapper->SetInstigator(WorldContextObject);
			QueryInstanceWrapper->RunQuery(RunMode, QueryRequest);
		}
	}
	
	return QueryInstanceWrapper;
}

void UEnvQueryManager::RegisterActiveWrapper(UEnvQueryInstanceBlueprintWrapper& Wrapper)
{
	GCShieldedWrappers.AddUnique(&Wrapper);
}

void UEnvQueryManager::UnregisterActiveWrapper(UEnvQueryInstanceBlueprintWrapper& Wrapper)
{
	GCShieldedWrappers.RemoveSingleSwap(&Wrapper, EAllowShrinking::No);
}

TSharedPtr<FEnvQueryInstance> UEnvQueryManager::FindQueryInstance(const int32 QueryID)
{
	if (QueryID != INDEX_NONE)
	{
		// going from the back since it's most probably there
		for (int32 QueryIndex = RunningQueries.Num() - 1; QueryIndex >= 0; --QueryIndex)
		{
			if (RunningQueries[QueryIndex]->QueryID == QueryID)
			{
				return RunningQueries[QueryIndex];
			}
		}
	}

	return nullptr;
}

#if WITH_EDITOR
void UEnvQueryManager::OnBlueprintCompiled()
{
	LocalContextMap.Reset();
}
#endif // WITH_EDITOR

//----------------------------------------------------------------------//
// Exec functions (i.e. console commands)
//----------------------------------------------------------------------//
void UEnvQueryManager::SetAllowTimeSlicing(bool bAllowTimeSlicing)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	bAllowEQSTimeSlicing = bAllowTimeSlicing;

	UE_LOGF(LogEQS, Log, "Set allow time slicing to %ls.",
			bAllowEQSTimeSlicing ? TEXT("true") : TEXT("false"));
#else
	UE_LOGF(LogEQS, Log, "Time slicing cannot be disabled in Test or Shipping builds.  SetAllowTimeSlicing does nothing.");
#endif
}

bool UEnvQueryManager::Exec_Dev(UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar)
{
#if USE_EQS_DEBUGGER
	if (FParse::Command(&Cmd, TEXT("DumpEnvQueryStats")))
	{
		const FString FileName = FPaths::CreateTempFilename(*FPaths::ProjectLogDir(), TEXT("EnvQueryStats"), TEXT(".ue_eqs"));

		FEQSDebugger::SaveStats(FileName);
		return true;
	}
#endif

	return false;
}


void UEnvQueryManager::Configure(const FEnvQueryManagerConfig& NewConfig)
{
	UE_VLOG_UELOG(this, LogEQS, Log, TEXT("Applying new FEnvQueryManagerConfig: %s"), *NewConfig.ToString());

	MaxAllowedTestingTime = NewConfig.MaxAllowedTestingTime;
	bTestQueriesUsingBreadth = NewConfig.bTestQueriesUsingBreadth;
	QueryCountWarningThreshold = NewConfig.QueryCountWarningThreshold;
	QueryCountWarningInterval = NewConfig.QueryCountWarningInterval;
	ExecutionTimeWarningSeconds = NewConfig.ExecutionTimeWarningSeconds;
	HandlingResultTimeWarningSeconds = NewConfig.HandlingResultTimeWarningSeconds;
	GenerationTimeWarningSeconds = NewConfig.GenerationTimeWarningSeconds;	
}

//----------------------------------------------------------------------//
// FEQSDebugger
//----------------------------------------------------------------------//
#if USE_EQS_DEBUGGER

void FEQSDebugger::StoreStats(const FEnvQueryInstance& QueryInstance)
{
	SCOPE_CYCLE_COUNTER(STAT_AI_EQS_Debug_StoreStats);

	FStatsInfo& UpdateInfo = UEnvQueryManager::DebuggerStats.FindOrAdd(FName(*QueryInstance.QueryName));

	const FEnvQueryDebugProfileData& QueryStats = QueryInstance.DebugData;
	const double ExecutionTime = QueryInstance.TotalExecutionTime;
	
	if (ExecutionTime > UpdateInfo.MostExpensiveDuration)
	{
		UpdateInfo.MostExpensiveDuration = ExecutionTime;
		UpdateInfo.MostExpensive = QueryStats;
	}

	// better solution for counting average?
	if (UpdateInfo.TotalAvgCount >= 100000)
	{
		UpdateInfo.TotalAvgData = QueryStats;
		UpdateInfo.TotalAvgDuration = ExecutionTime;
		UpdateInfo.TotalAvgCount = 1;
	}
	else
	{
		UpdateInfo.TotalAvgData.Add(QueryStats);
		UpdateInfo.TotalAvgDuration += ExecutionTime;
		UpdateInfo.TotalAvgCount++;
	}
}

void FEQSDebugger::StoreQuery(const TSharedPtr<FEnvQueryInstance>& QueryInstance)
{
	SCOPE_CYCLE_COUNTER(STAT_AI_EQS_Debug_StoreQuery);

	for (TMap<FObjectKey, TArray<FEnvQueryInfo>>::TIterator It(StoredQueries); It; ++It)
	{
		if (It.Key().ResolveObjectPtr() == nullptr)
		{
			It.RemoveCurrent();
		}
	}

	if (!QueryInstance.IsValid() || QueryInstance->World == nullptr)
	{
		return;
	}

	TArray<FEnvQueryInfo>& AllQueries = StoredQueries.FindOrAdd(QueryInstance->Owner.Get());
	int32 QueryIdx = INDEX_NONE;

	for (int32 Idx = 0; Idx < AllQueries.Num(); Idx++)
	{
		const FEnvQueryInfo& TestInfo = AllQueries[Idx];
		if (TestInfo.Instance.IsValid() && QueryInstance->QueryName == TestInfo.Instance->QueryName)
		{
			QueryIdx = Idx;
			break;
		}
	}

	if (QueryIdx == INDEX_NONE)
	{
		QueryIdx = AllQueries.AddDefaulted(1);
	}

	FEnvQueryInfo& UpdateInfo = AllQueries[QueryIdx];
	UpdateInfo.Instance = QueryInstance;
	UpdateInfo.Timestamp = QueryInstance->World->GetTimeSeconds();
}

void FEQSDebugger::StoreTickTime(const FEnvQueryInstance& QueryInstance, double TickTime, double MaxTickTime)
{
#if USE_EQS_TICKLOADDATA
	SCOPE_CYCLE_COUNTER(STAT_AI_EQS_Debug_StoreTickTime);

	const int32 NumRecordedTicks = 0x4000;

	FStatsInfo& UpdateInfo = UEnvQueryManager::DebuggerStats.FindOrAdd(FName(*QueryInstance.QueryName));
	if (UpdateInfo.TickPct.Num() != NumRecordedTicks)
	{
		UpdateInfo.TickPct.Reset();
		UpdateInfo.TickPct.AddZeroed(NumRecordedTicks);
	}

	if (UpdateInfo.LastTickFrame != GFrameCounter)
	{
		UpdateInfo.LastTickFrame = GFrameCounter;
		UpdateInfo.LastTickTime = 0.;
	}

	const uint16 TickIdx = GFrameCounter & (NumRecordedTicks - 1);
	UpdateInfo.FirstTickEntry = (TickIdx < UpdateInfo.FirstTickEntry) ? TickIdx : UpdateInfo.FirstTickEntry;
	UpdateInfo.LastTickEntry = (TickIdx > UpdateInfo.LastTickEntry) ? TickIdx : UpdateInfo.LastTickEntry;

	UpdateInfo.LastTickTime += TickTime;
	UpdateInfo.TickPct[TickIdx] = static_cast<uint8>(FMath::Min(255, FMath::TruncToInt(255 * UpdateInfo.LastTickTime / MaxTickTime)));
#endif // USE_EQS_TICKLOADDATA
}

const TArray<FEQSDebugger::FEnvQueryInfo>& FEQSDebugger::GetAllQueriesForOwner(const UObject* Owner)
{
	TArray<FEnvQueryInfo>& AllQueries = StoredQueries.FindOrAdd(Owner);
	return AllQueries;
}

FArchive& operator<<(FArchive& Ar, FEQSDebugger::FStatsInfo& Data)
{
	int32 VersionNum = 1;
	Ar << VersionNum;

	Ar << Data.MostExpensive;
	Ar << Data.MostExpensiveDuration;
	Ar << Data.TotalAvgData;
	Ar << Data.TotalAvgDuration;
	Ar << Data.TotalAvgCount;
	Ar << Data.TickPct;
	Ar << Data.FirstTickEntry;
	Ar << Data.LastTickEntry;
	return Ar;
}

void FEQSDebugger::SaveStats(const FString& FileName)
{
	FArchive* FileWriter = IFileManager::Get().CreateFileWriter(*FileName);
	if (FileWriter)
	{
		int32 NumRecords = UEnvQueryManager::DebuggerStats.Num();
		(*FileWriter) << NumRecords;

		for (auto It : UEnvQueryManager::DebuggerStats)
		{
			FString QueryName = It.Key.ToString();
			(*FileWriter) << QueryName;
			(*FileWriter) << It.Value;
		}

		FileWriter->Close();
		delete FileWriter;

		UE_LOGF(LogEQS, Display, "EQS debugger data saved! File: %ls", *FileName);
	}
}

void FEQSDebugger::LoadStats(const FString& FileName)
{
	FArchive* FileReader = IFileManager::Get().CreateFileReader(*FileName);
	if (FileReader)
	{
		UEnvQueryManager::DebuggerStats.Reset();

		int32 NumRecords = UEnvQueryManager::DebuggerStats.Num();
		(*FileReader) << NumRecords;

		for (int32 Idx = 0; Idx < NumRecords; Idx++)
		{
			FString QueryName;
			FEQSDebugger::FStatsInfo Stats;

			(*FileReader) << QueryName;
			(*FileReader) << Stats;

			UEnvQueryManager::DebuggerStats.Add(*QueryName, Stats);
		}

		FileReader->Close();
		delete FileReader;
	}
}

#endif // USE_EQS_DEBUGGER

//----------------------------------------------------------------------//
// FEnvQueryManagerConfig
//----------------------------------------------------------------------//
FString FEnvQueryManagerConfig::ToString() const
{
	return FString::Printf(TEXT("MaxAllowedTestingTime=%f bTestQueriesUsingBreadth=%d QueryCountWarningThreshold=%d QueryCountWarningInterval=%f ExecutionTimeWarningSeconds=%f HandlingResultTimeWarningSeconds=%f GenerationTimeWarningSeconds=%f"), MaxAllowedTestingTime, bTestQueriesUsingBreadth, QueryCountWarningThreshold, QueryCountWarningInterval, ExecutionTimeWarningSeconds, HandlingResultTimeWarningSeconds, GenerationTimeWarningSeconds);
}

