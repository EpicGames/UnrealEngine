// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/EnvQueryNode.h"
#include "UObject/UnrealType.h"
#include "DataProviders/AIDataProvider.h"
#include "EnvironmentQuery/EnvQueryManager.h"
#include "DataProviders/AIDataProvider_QueryParams.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryNode)

UEnvQueryNode::UEnvQueryNode(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

void UEnvQueryNode::UpdateNodeVersion()
{
	VerNum = 0;
}

FText UEnvQueryNode::GetDescriptionTitle() const
{
	return UEnvQueryTypes::GetShortTypeName(this);
}

FText UEnvQueryNode::GetDescriptionDetails() const
{
	return FText::GetEmpty();
}

#if WITH_EDITOR
void UEnvQueryNode::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName NAME_DataBinding = GET_MEMBER_NAME_CHECKED(FAIDataProviderValue, DataBinding);
	
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (const FStructProperty* TestStruct = CastField<FStructProperty>(PropertyChangedEvent.MemberProperty))
	{
		// Validate that DataField is still a valid value for DataBinding
		if (TestStruct->Struct != nullptr
			&& TestStruct->Struct->IsChildOf(FAIDataProviderValue::StaticStruct()))
		{
			if (FAIDataProviderValue* AIDataProviderValue = TestStruct->ContainerPtrToValuePtr<FAIDataProviderValue>(this))
			{
				TArray<FName> MatchingProperties;
				AIDataProviderValue->GetMatchingProperties(MatchingProperties);

				const FName NameValue = AIDataProviderValue->DataField;

				// If current field name is not in the list of matching names then default on the first match
				if (MatchingProperties.Num() && !MatchingProperties.Contains(NameValue))
				{
					AIDataProviderValue->DataField = MatchingProperties[0];
				}
				// If current field name is set but there are no matching names then reset the binding to the default since it's invalid.
				// This could happen when copy /pasting FAIDataProviderValue based properties of different types.
				else if (MatchingProperties.IsEmpty() && !NameValue.IsNone())
				{
					AIDataProviderValue->DataField = FName();
					AIDataProviderValue->DataBinding = nullptr;
				}
			}
		}

		// populate ParamName value
		const FName PropertyName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : FName();
		if (PropertyName == NAME_DataBinding)
		{
			const FString TypeDesc = TestStruct->GetCPPType(nullptr, CPPF_None);

#define GET_STRUCT_NAME_CHECKED(StructName) ((void)sizeof(StructName), TEXT(#StructName))

			if (TypeDesc.Contains(GET_STRUCT_NAME_CHECKED(FAIDataProviderIntValue))
				|| TypeDesc.Contains(GET_STRUCT_NAME_CHECKED(FAIDataProviderFloatValue))
				|| TypeDesc.Contains(GET_STRUCT_NAME_CHECKED(FAIDataProviderBoolValue)))
			{
				if (const FAIDataProviderTypedValue* PropertyValue = TestStruct->ContainerPtrToValuePtr<FAIDataProviderTypedValue>(this))
				{
					UAIDataProvider_QueryParams* QueryParamProvider = Cast<UAIDataProvider_QueryParams>(PropertyValue->DataBinding);
					if (QueryParamProvider && QueryParamProvider->ParamName.IsNone())
					{
						FString Stripped;
						const FString NodeName = GetFName().GetPlainNameString();
						if (NodeName.Split(TEXT("_"), nullptr, &Stripped))
						{
							QueryParamProvider->ParamName = *FString::Printf(TEXT("%s.%s"), *Stripped, *PropertyChangedEvent.MemberProperty->GetName());
						}
						else
						{
							QueryParamProvider->ParamName = PropertyChangedEvent.MemberProperty->GetFName();
						}
					}
				}
			}
#undef GET_STRUCT_NAME_CHECKED
		}
	}

#if USE_EQS_DEBUGGER
	UEnvQueryManager::NotifyAssetUpdate(nullptr);
#endif // USE_EQS_DEBUGGER
}
#endif //WITH_EDITOR && USE_EQS_DEBUGGER

