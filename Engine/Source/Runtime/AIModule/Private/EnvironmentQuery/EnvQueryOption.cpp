// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/EnvQueryOption.h"
#include "EnvironmentQuery/EnvQueryGenerator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryOption)

#define LOCTEXT_NAMESPACE "EnvQueryGenerator"

UEnvQueryOption::UEnvQueryOption(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

FText UEnvQueryOption::GetDescriptionTitle() const
{
	return Generator ? Generator->GetDescriptionTitle() : FText::GetEmpty();
}

FText UEnvQueryOption::GetDescriptionDetails() const
{
	FText DetailsDesc = FText::GetEmpty();
	if (Generator)
	{
		FText OptionDetails = Generator->GetDescriptionDetails();
		DetailsDesc = Generator->bAutoSortTests ? OptionDetails :
			FText::Format(FText::FromString("{0}\n{1}"), OptionDetails, LOCTEXT("NoSortMode", "TEST SORTING DISABLED"));
	}

	return DetailsDesc;
}

#undef LOCTEXT_NAMESPACE

