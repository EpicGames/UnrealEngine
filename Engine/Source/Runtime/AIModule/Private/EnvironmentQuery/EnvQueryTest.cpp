// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/EnvQueryTest.h"
#include "EnvironmentQuery/Contexts/EnvQueryContext_Item.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_ActorBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryTest)

#define LOCTEXT_NAMESPACE "EnvQueryGenerator"

UEnvQueryTest::UEnvQueryTest(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	TestPurpose = EEnvTestPurpose::FilterAndScore;
	Cost = EEnvTestCost::Low;
	FilterType = EEnvTestFilterType::Range;
	ScoringEquation = EEnvTestScoreEquation::Linear;
	ClampMinType = EEnvQueryTestClamping::None;
	ClampMaxType = EEnvQueryTestClamping::None;
	BoolValue.DefaultValue = true;
	ScoringFactor.DefaultValue = 1.0f;
	NormalizationType = EEQSNormalizationType::Absolute;

	bWorkOnFloatValues = true;
}

void UEnvQueryTest::NormalizeItemScores(FEnvQueryInstance& QueryInstance)
{
	UObject* QueryOwner = QueryInstance.Owner.Get();
	if (QueryOwner == nullptr || !IsScoring())
	{
		return;
	}

	ScoringFactor.BindData(QueryOwner, QueryInstance.QueryID);
	const float ScoringFactorValue = ScoringFactor.GetValue();

	float MinScore = (NormalizationType == EEQSNormalizationType::Absolute) ? 0 : BIG_NUMBER;
	float MaxScore = -BIG_NUMBER;

	if (ClampMinType == EEnvQueryTestClamping::FilterThreshold)
	{
		FloatValueMin.BindData(QueryOwner, QueryInstance.QueryID);
		MinScore = FloatValueMin.GetValue();
	}
	else if (ClampMinType == EEnvQueryTestClamping::SpecifiedValue)
	{
		ScoreClampMin.BindData(QueryOwner, QueryInstance.QueryID);
		MinScore = ScoreClampMin.GetValue();
	}

	if (ClampMaxType == EEnvQueryTestClamping::FilterThreshold)
	{
		FloatValueMax.BindData(QueryOwner, QueryInstance.QueryID);
		MaxScore = FloatValueMax.GetValue();
	}
	else if (ClampMaxType == EEnvQueryTestClamping::SpecifiedValue)
	{
		ScoreClampMax.BindData(QueryOwner, QueryInstance.QueryID);
		MaxScore = ScoreClampMax.GetValue();
	}

	FEnvQueryItemDetails* DetailInfo = QueryInstance.ItemDetails.GetData();
	if ((ClampMinType == EEnvQueryTestClamping::None) ||
		(ClampMaxType == EEnvQueryTestClamping::None)
	   )
	{
		for (int32 ItemIndex = 0; ItemIndex < QueryInstance.Items.Num(); ItemIndex++, DetailInfo++)
		{
			if (!QueryInstance.Items[ItemIndex].IsValid())
			{
				continue;
			}

			float TestValue = DetailInfo->TestResults[QueryInstance.CurrentTest];
			if (TestValue != UEnvQueryTypes::SkippedItemValue)
			{
				if (ClampMinType == EEnvQueryTestClamping::None)
				{
					MinScore = FMath::Min(MinScore, TestValue);
				}

				if (ClampMaxType == EEnvQueryTestClamping::None)
				{
					MaxScore = FMath::Max(MaxScore, TestValue);
				}
			}
		}
	}

	DetailInfo = QueryInstance.ItemDetails.GetData();

	if (MinScore != MaxScore)
	{
		if (bDefineReferenceValue)
		{
			ReferenceValue.BindData(QueryOwner, QueryInstance.QueryID);
		}
		const float TargetScore = bDefineReferenceValue ? ReferenceValue.GetValue() : MaxScore;
		const float ValueSpan = FMath::Max(FMath::Abs(TargetScore - MinScore), FMath::Abs(TargetScore - MaxScore));
		const float AbsoluteWeight = FMath::Abs(ScoringFactorValue);

		for (int32 ItemIndex = 0; ItemIndex < QueryInstance.ItemDetails.Num(); ItemIndex++, DetailInfo++)
		{
			if (QueryInstance.Items[ItemIndex].IsValid() == false)
			{
				continue;
			}

			float WeightedScore = 0.0f;

			float& TestValue = DetailInfo->TestResults[QueryInstance.CurrentTest];
			if (TestValue != UEnvQueryTypes::SkippedItemValue)
			{
				const float ClampedScore = FMath::Clamp(TestValue, MinScore, MaxScore);
				const float NormalizedScore = (ScoringFactorValue >= 0) 
					? (1.f - (FMath::Abs(TargetScore - ClampedScore) / ValueSpan))
					: (FMath::Abs(TargetScore - ClampedScore) / ValueSpan);

				switch (ScoringEquation)
				{
					case EEnvTestScoreEquation::Linear:
						WeightedScore = AbsoluteWeight * NormalizedScore;
						break;

					case EEnvTestScoreEquation::InverseLinear:
					{
						// For now, we're avoiding having a separate flag for flipping the direction of the curve
						// because we don't have usage cases yet and want to avoid too complex UI.  If we decide
						// to add that flag later, we'll need to remove this option, since it should just be "mirror
						// curve" plus "Linear".
						float InverseNormalizedScore = (1.0f - NormalizedScore);
						WeightedScore = AbsoluteWeight * InverseNormalizedScore;
						break;
					}

					case EEnvTestScoreEquation::Square:
						WeightedScore = AbsoluteWeight * (NormalizedScore * NormalizedScore);
						break;

					case EEnvTestScoreEquation::SquareRoot:
						WeightedScore = AbsoluteWeight * FMath::Sqrt(NormalizedScore);
						break;

					case EEnvTestScoreEquation::Constant:
						// I know, it's not "constant".  It's "Constant, or zero".  The tooltip should explain that.
						WeightedScore = (NormalizedScore > 0) ? AbsoluteWeight : 0.0f;
						break;
						
					default:
						break;
				}
			}
			else
			{
				// Do NOT clear TestValue to 0, because the SkippedItemValue is used to display "SKIP" when debugging.
				// TestValue = 0.0f;
				WeightedScore = 0.0f;
			}

#if USE_EQS_DEBUGGER
			DetailInfo->TestWeightedScores[QueryInstance.CurrentTest] = WeightedScore;
#endif
			QueryInstance.Items[ItemIndex].Score += WeightedScore;
		}
	}
}

bool UEnvQueryTest::IsContextPerItem(TSubclassOf<UEnvQueryContext> CheckContext) const
{
	return CheckContext == UEnvQueryContext_Item::StaticClass();
}

FVector UEnvQueryTest::GetItemLocation(FEnvQueryInstance& QueryInstance, int32 ItemIndex) const
{
	return QueryInstance.ItemTypeVectorCDO ?
		QueryInstance.ItemTypeVectorCDO->GetItemLocation(QueryInstance.RawData.GetData() + QueryInstance.Items[ItemIndex].DataOffset) :
		FVector::ZeroVector;
}

FRotator UEnvQueryTest::GetItemRotation(FEnvQueryInstance& QueryInstance, int32 ItemIndex) const
{
	return QueryInstance.ItemTypeVectorCDO ?
		QueryInstance.ItemTypeVectorCDO->GetItemRotation(QueryInstance.RawData.GetData() + QueryInstance.Items[ItemIndex].DataOffset) :
		FRotator::ZeroRotator;
}

AActor* UEnvQueryTest::GetItemActor(FEnvQueryInstance& QueryInstance, int32 ItemIndex) const
{
	return QueryInstance.ItemTypeActorCDO ?
		QueryInstance.ItemTypeActorCDO->GetActor(QueryInstance.RawData.GetData() + QueryInstance.Items[ItemIndex].DataOffset) :
		NULL;
}

void UEnvQueryTest::PostLoad()
{
	Super::PostLoad();

	if (VerNum < EnvQueryTestVersion::ReferenceValueFix && bDefineReferenceValue)
	{
		// if someone was using the ReferenceValue the fix will reverse their score. We can fix it by flipping the 
		// ScoringFactor. Unfortunately we won't be able to to that if the ScoringFactor is actually bound to some 
		// provider - in that case all we can do is to spit out a warning. 

		UWorld* World = GetWorld();
		if (World == nullptr)
		{
			// we want this logic to run only for assets, not the instantiated queries that should pull the following change in automatically
			if (ScoringFactor.IsDynamic() == false)
			{
				ScoringFactor.DefaultValue *= -1;
				UE_LOG(LogEQS, Verbose, TEXT("%s using a ScoringFactor. Due to a bug-fix the effect will be flipped so the\
					value\'s sign has been flipped for this EQS Test asset instance.")
					, *GetPathName());
			}
			else if (ScoringEquation == EEnvTestScoreEquation::Linear)
			{
				// we can still address this by flipping the ScoringEquation to 
				ScoringEquation = EEnvTestScoreEquation::InverseLinear;
				UE_LOGF(LogEQS, Verbose, "%ls using a AIDataProvider for ScoringFactor. Due to a bug-fix the effect will be flipped so ScoringEquation has been automatically flipped from Linear to InverseLinear for this EQS Test instance."
					, *GetPathName());
			}
			else if (ScoringEquation == EEnvTestScoreEquation::InverseLinear)
			{
				// we can still address this by flipping the ScoringEquation to 
				ScoringEquation = EEnvTestScoreEquation::Linear;
				UE_LOGF(LogEQS, Verbose, "%ls using a AIDataProvider for ScoringFactor. Due to a bug-fix the effect will be flipped so ScoringEquation has been automatically flipped from InverseLinear to Linear for this EQS Test instance."
					, *GetPathName());
			}
			else
			{
				UE_LOGF(LogEQS, Warning, "%ls using a AIDataProvider for ScoringFactor. Due to a bug-fix the effect will be flipped and an automated change could not be performed. Please address manually."
					, *GetPathName());
			}

			if (MarkPackageDirty() == false)
			{
				UE_LOGF(LogEQS, Warning, "%ls unable to mark package dirty - please resave manually or use a commandlet."
					, *GetPathName());
			}
		}
	}

	UpdateNodeVersion();
#if WITH_EDITOR
	UpdatePreviewData();
#endif // WITH_EDITOR
}

void UEnvQueryTest::UpdateNodeVersion()
{
	VerNum = EnvQueryTestVersion::Latest;
}

FText UEnvQueryTest::DescribeFloatTestParams() const
{
	FText FilterDesc;
	if (IsFiltering())
	{
		switch (FilterType)
		{
		case EEnvTestFilterType::Minimum:
			FilterDesc = FText::Format(LOCTEXT("FilterAtLeast", "at least {0}"), FText::FromString(FloatValueMin.ToString()));
			break;

		case EEnvTestFilterType::Maximum:
			FilterDesc = FText::Format(LOCTEXT("FilterUpTo", "up to {0}"), FText::FromString(FloatValueMax.ToString()));
			break;

		case EEnvTestFilterType::Range:
			FilterDesc = FText::Format(LOCTEXT("FilterBetween", "between {0} and {1}"),
				FText::FromString(FloatValueMin.ToString()), FText::FromString(FloatValueMax.ToString()));
			break;

		default:
			break;
		}
	}

	FNumberFormattingOptions NumberFormattingOptions;
	NumberFormattingOptions.MaximumFractionalDigits = 2;

	FText ScoreDesc;
	if (!IsScoring())
	{
		ScoreDesc = LOCTEXT("DontScore", "don't score");
	}
	else if (ScoringEquation == EEnvTestScoreEquation::Constant)
	{
		FText FactorDesc = ScoringFactor.IsDynamic() ?
			FText::FromString(ScoringFactor.ToString()) :
			FText::Format(FText::FromString("x{0}"), FText::AsNumber(FMath::Abs(ScoringFactor.DefaultValue), &NumberFormattingOptions));

		ScoreDesc = FText::Format(FText::FromString("{0} [{1}]"), LOCTEXT("ScoreConstant", "constant score"), FactorDesc);
	}
	else if (ScoringFactor.IsDynamic())
	{
		ScoreDesc = FText::Format(FText::FromString("{0}: {1}"), LOCTEXT("ScoreFactor", "score factor"), FText::FromString(ScoringFactor.ToString()));
	}
	else
	{
		const bool bScalesUp = (ScoringEquation == EEnvTestScoreEquation::InverseLinear) 
			? (ScoringFactor.DefaultValue < 0) 
			: (ScoringFactor.DefaultValue > 0);
		const FText ScoreSignDesc = bScalesUp ? LOCTEXT("Greater", "greater") : LOCTEXT("Lesser", "lesser");
		const FText ScoreValueDesc = FText::AsNumber(FMath::Abs(ScoringFactor.DefaultValue), &NumberFormattingOptions);
		ScoreDesc = FText::Format(FText::FromString("{0} {1} [x{2}]"), LOCTEXT("ScorePrefer", "prefer"), ScoreSignDesc, ScoreValueDesc);
	}

	return FilterDesc.IsEmpty() ? ScoreDesc : FText::Format(FText::FromString("{0}, {1}"), FilterDesc, ScoreDesc);
}

FText UEnvQueryTest::DescribeBoolTestParams(const FString& ConditionDesc) const
{
	FText FilterDesc;
	if (IsFiltering() && FilterType == EEnvTestFilterType::Match)
	{
		FilterDesc = BoolValue.IsDynamic() ?
			FText::Format(FText::FromString("{0} {1}: {2}"), LOCTEXT("FilterRequire", "require"), FText::FromString(ConditionDesc), FText::FromString(BoolValue.ToString())) :
			FText::Format(FText::FromString("{0} {1}{2}"), LOCTEXT("FilterRequire", "require"), BoolValue.DefaultValue ? FText::GetEmpty() : LOCTEXT("NotWithSpace", "not "), FText::FromString(ConditionDesc));
	}

	FNumberFormattingOptions NumberFormattingOptions;
	NumberFormattingOptions.MaximumFractionalDigits = 2;

	FText ScoreDesc;
	if (!IsScoring())
	{
		ScoreDesc = LOCTEXT("DontScore", "don't score");
	}
	else if (ScoringEquation == EEnvTestScoreEquation::Constant)
	{
		FText FactorDesc = ScoringFactor.IsDynamic() ?
			FText::FromString(ScoringFactor.ToString()) :
			FText::Format(FText::FromString("x{0}"), FText::AsNumber(FMath::Abs(ScoringFactor.DefaultValue), &NumberFormattingOptions));

		ScoreDesc = FText::Format(FText::FromString("{0} [{1}]"), LOCTEXT("ScoreConstant", "constant score"), FactorDesc);
	}
	else if (ScoringFactor.IsDynamic())
	{
		ScoreDesc = FText::Format(FText::FromString("{0}: {1}"), LOCTEXT("ScoreFactor", "score factor"), FText::FromString(ScoringFactor.ToString()));
	}
	else
	{
		FText ScoreSignDesc = (ScoringFactor.DefaultValue > 0) ? FText::GetEmpty() : LOCTEXT("NotWithSpace", "not ");
		FText ScoreValueDesc = FText::AsNumber(FMath::Abs(ScoringFactor.DefaultValue), &NumberFormattingOptions);
		ScoreDesc = FText::Format(FText::FromString("{0} {1}{2} [x{3}]"), LOCTEXT("ScorePrefer", "prefer"), ScoreSignDesc, FText::FromString(ConditionDesc), ScoreValueDesc);
	}

	return FilterDesc.IsEmpty() ? ScoreDesc : FText::Format(FText::FromString("{0}, {1}"), FilterDesc, ScoreDesc);
}

void UEnvQueryTest::SetWorkOnFloatValues(bool bWorkOnFloats)
{
	bWorkOnFloatValues = bWorkOnFloats;

	// Make sure FilterType is set to a valid value.
	if (bWorkOnFloats)
	{
		if (FilterType == EEnvTestFilterType::Match)
		{
			FilterType = EEnvTestFilterType::Range;
		}
	}
	else
	{
		if (FilterType != EEnvTestFilterType::Match)
		{
			FilterType = EEnvTestFilterType::Match;
		}

		// Scoring MUST be Constant for boolean tests.
		ScoringEquation = EEnvTestScoreEquation::Constant;
	}

	UpdatePreviewData();
}

#if WITH_EDITOR
void UEnvQueryTest::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.MemberProperty)
	{
		const FName MemberPropName = PropertyChangedEvent.MemberProperty->GetFName();

		if (MemberPropName == GET_MEMBER_NAME_CHECKED(UEnvQueryTest, TestPurpose) 
			|| MemberPropName == GET_MEMBER_NAME_CHECKED(UEnvQueryTest, FilterType)
			|| MemberPropName == GET_MEMBER_NAME_CHECKED(UEnvQueryTest, ClampMaxType)
			|| MemberPropName == GET_MEMBER_NAME_CHECKED(UEnvQueryTest, ClampMinType) 
			|| MemberPropName == GET_MEMBER_NAME_CHECKED(UEnvQueryTest, ScoringEquation) 
			|| MemberPropName == GET_MEMBER_NAME_CHECKED(UEnvQueryTest, ScoringFactor))
		{
			UpdatePreviewData();
		}
	}
}
#endif

void UEnvQueryTest::UpdatePreviewData()
{
#if WITH_EDITORONLY_DATA && WITH_EDITOR
	PreviewData.Samples.Reset(FEnvQueryTestScoringPreview::DefaultSamplesCount);

	switch (ScoringEquation)
	{
	case EEnvTestScoreEquation::Linear:
	{
		for (int32 SampleIndex = 0; SampleIndex < FEnvQueryTestScoringPreview::DefaultSamplesCount; ++SampleIndex)
		{
			PreviewData.Samples.Add(static_cast<float>(SampleIndex) / (FEnvQueryTestScoringPreview::DefaultSamplesCount - 1));
		}
	}
		break;
	case EEnvTestScoreEquation::Square:
	{
		for (int32 SampleIndex = 0; SampleIndex < FEnvQueryTestScoringPreview::DefaultSamplesCount; ++SampleIndex)
		{
			PreviewData.Samples.Add(FMath::Square(static_cast<float>(SampleIndex) / (FEnvQueryTestScoringPreview::DefaultSamplesCount - 1)));
		}
	}
		break;
	case EEnvTestScoreEquation::InverseLinear:
	{
		for (int32 SampleIndex = 0; SampleIndex < FEnvQueryTestScoringPreview::DefaultSamplesCount; ++SampleIndex)
		{
			PreviewData.Samples.Add(1.f - static_cast<float>(SampleIndex) / (FEnvQueryTestScoringPreview::DefaultSamplesCount - 1));
		}
	}
		break;
	case EEnvTestScoreEquation::SquareRoot:
	{
		for (int32 SampleIndex = 0; SampleIndex < FEnvQueryTestScoringPreview::DefaultSamplesCount; ++SampleIndex)
		{
			PreviewData.Samples.Add(FMath::Sqrt(static_cast<float>(SampleIndex) / (FEnvQueryTestScoringPreview::DefaultSamplesCount - 1)));
		}
	}
		break;
	case EEnvTestScoreEquation::Constant:
	{
		for (int32 SampleIndex = 0; SampleIndex < FEnvQueryTestScoringPreview::DefaultSamplesCount; ++SampleIndex)
		{
			PreviewData.Samples.Add(0.5f);
		}
	}
		break;
	default:
		checkNoEntry();
		break;
	}

	const bool bInversed = ScoringFactor.GetValue() < 0.0f;
	if (bInversed)
	{
		for (float& Sample : PreviewData.Samples)
		{
			Sample = 1.f - Sample;
		}
	}


	PreviewData.bShowClampMin = (ClampMinType != EEnvQueryTestClamping::None);
	PreviewData.bShowClampMax = (ClampMaxType != EEnvQueryTestClamping::None);
	const bool bCanFilter = (TestPurpose != EEnvTestPurpose::Score);
	PreviewData.bShowFilterHigh = ((FilterType == EEnvTestFilterType::Maximum) || (FilterType == EEnvTestFilterType::Range)) && bCanFilter;
	PreviewData.bShowFilterLow = ((FilterType == EEnvTestFilterType::Minimum) || (FilterType == EEnvTestFilterType::Range)) && bCanFilter;

	PreviewData.FilterLow = 0.2f;
	PreviewData.FilterHigh = 0.8f;
	PreviewData.ClampMin = (ClampMinType == EEnvQueryTestClamping::FilterThreshold) ? PreviewData.FilterLow : 0.3f;
	PreviewData.ClampMax = (ClampMaxType == EEnvQueryTestClamping::FilterThreshold) ? PreviewData.FilterHigh : 0.7f;

	if (PreviewData.bShowClampMin)
	{
		const int32 FixedIdx = FMath::RoundToInt(PreviewData.ClampMin * (PreviewData.Samples.Num() - 1));
		for (int32 Idx = 0; Idx < FixedIdx; Idx++)
		{
			PreviewData.Samples[Idx] = PreviewData.Samples[FixedIdx];
		}
	}

	if (PreviewData.bShowClampMax)
	{
		const int32 FixedIdx = FMath::RoundToInt(PreviewData.ClampMax * (PreviewData.Samples.Num() - 1));
		for (int32 Idx = FixedIdx + 1; Idx < PreviewData.Samples.Num(); Idx++)
		{
			PreviewData.Samples[Idx] = PreviewData.Samples[FixedIdx];
		}
	}
#endif
}

#undef LOCTEXT_NAMESPACE

