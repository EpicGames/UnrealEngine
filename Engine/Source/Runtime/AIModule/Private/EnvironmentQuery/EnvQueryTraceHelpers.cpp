// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/EnvQueryTraceHelpers.h"
#include "Engine/HitResult.h"
#include "NavigationData.h"
#include "NavFilters/NavigationQueryFilter.h"
#include "Algo/RemoveIf.h"

template<>
void FEQSHelpers::FBatchTrace::DoSingleSourceMultiDestinations<EEnvTraceShape::Line>(const FVector& Source, TArray<FNavLocation>& Points)
{
	FVector HitPos(FVector::ZeroVector);
	for (int32 Idx = Points.Num() - 1; Idx >= 0; Idx--)
	{
		const bool bHit = RunLineTrace(Source, Points[Idx].Location, HitPos);
		if (bHit)
		{
			Points[Idx] = FNavLocation(HitPos);
		}
		else if (TraceMode == ETraceMode::Discard)
		{
			Points.RemoveAt(Idx, EAllowShrinking::No);
		}
	}
}

template<>
void FEQSHelpers::FBatchTrace::DoSingleSourceMultiDestinations<EEnvTraceShape::Box>(const FVector& Source, TArray<FNavLocation>& Points)
{
	FVector HitPos(FVector::ZeroVector);
	for (int32 Idx = Points.Num() - 1; Idx >= 0; Idx--)
	{
		const bool bHit = RunBoxTrace(Source, Points[Idx].Location, HitPos);
		if (bHit)
		{
			Points[Idx] = FNavLocation(HitPos);
		}
		else if (TraceMode == ETraceMode::Discard)
		{
			Points.RemoveAt(Idx, EAllowShrinking::No);
		}
	}
}

template<>
void FEQSHelpers::FBatchTrace::DoSingleSourceMultiDestinations<EEnvTraceShape::Sphere>(const FVector& Source, TArray<FNavLocation>& Points)
{
	FVector HitPos(FVector::ZeroVector);
	for (int32 Idx = Points.Num() - 1; Idx >= 0; Idx--)
	{
		const bool bHit = RunSphereTrace(Source, Points[Idx].Location, HitPos);
		if (bHit)
		{
			Points[Idx] = FNavLocation(HitPos);
		}
		else if (TraceMode == ETraceMode::Discard)
		{
			Points.RemoveAt(Idx, EAllowShrinking::No);
		}
	}
}

template<>
void FEQSHelpers::FBatchTrace::DoSingleSourceMultiDestinations<EEnvTraceShape::Capsule>(const FVector& Source, TArray<FNavLocation>& Points)
{
	FVector HitPos(FVector::ZeroVector);
	for (int32 Idx = Points.Num() - 1; Idx >= 0; Idx--)
	{
		const bool bHit = RunCapsuleTrace(Source, Points[Idx].Location, HitPos);
		if (bHit)
		{
			Points[Idx] = FNavLocation(HitPos);
		}
		else if (TraceMode == ETraceMode::Discard)
		{
			Points.RemoveAt(Idx, EAllowShrinking::No);
		}
	}
}

template<>
void FEQSHelpers::FBatchTrace::DoMultiSourceMultiDestinations2D<EEnvTraceShape::Line>(const TArray<FRayStartEnd>& Rays, TArray<FNavLocation>& OutPoints)
{
	FVector HitPos(FVector::ZeroVector);

	for (int32 RayIndex = 0; RayIndex < Rays.Num(); ++RayIndex)
	{
		const FVector StartLocation = Rays[RayIndex].RayStart;
		// adjusted end to make this a 2D trace
		FVector EndLocation = Rays[RayIndex].RayEnd;
		EndLocation.Z = StartLocation.Z;
		const bool bHit = RunLineTrace(StartLocation, Rays[RayIndex].RayEnd, HitPos);
		if (bHit || TraceMode != ETraceMode::Discard)
		{
			OutPoints.Add(FNavLocation(bHit ? HitPos : EndLocation));
		}
	}
}

template<>
void FEQSHelpers::FBatchTrace::DoMultiSourceMultiDestinations2D<EEnvTraceShape::Box>(const TArray<FRayStartEnd>& Rays, TArray<FNavLocation>& OutPoints)
{
	FVector HitPos(FVector::ZeroVector);

	for (int32 RayIndex = 0; RayIndex < Rays.Num(); ++RayIndex)
	{
		const FVector StartLocation = Rays[RayIndex].RayStart;
		// adjusted end to make this a 2D trace
		FVector EndLocation = Rays[RayIndex].RayEnd;
		EndLocation.Z = StartLocation.Z;
		const bool bHit = RunBoxTrace(StartLocation, Rays[RayIndex].RayEnd, HitPos);
		if (bHit || TraceMode != ETraceMode::Discard)
		{
			OutPoints.Add(FNavLocation(bHit ? HitPos : EndLocation));
		}
	}
}

template<>
void FEQSHelpers::FBatchTrace::DoMultiSourceMultiDestinations2D<EEnvTraceShape::Sphere>(const TArray<FRayStartEnd>& Rays, TArray<FNavLocation>& OutPoints)
{
	FVector HitPos(FVector::ZeroVector);

	for (int32 RayIndex = 0; RayIndex < Rays.Num(); ++RayIndex)
	{
		const FVector StartLocation = Rays[RayIndex].RayStart;
		// adjusted end to make this a 2D trace
		FVector EndLocation = Rays[RayIndex].RayEnd;
		EndLocation.Z = StartLocation.Z;
		const bool bHit = RunSphereTrace(StartLocation, Rays[RayIndex].RayEnd, HitPos);
		if (bHit || TraceMode != ETraceMode::Discard)
		{
			OutPoints.Add(FNavLocation(bHit ? HitPos : EndLocation));
		}
	}
}

template<>
void FEQSHelpers::FBatchTrace::DoMultiSourceMultiDestinations2D<EEnvTraceShape::Capsule>(const TArray<FRayStartEnd>& Rays, TArray<FNavLocation>& OutPoints)
{
	FVector HitPos(FVector::ZeroVector);

	for (int32 RayIndex = 0; RayIndex < Rays.Num(); ++RayIndex)
	{
		const FVector StartLocation = Rays[RayIndex].RayStart;
		// adjusted end to make this a 2D trace
		FVector EndLocation = Rays[RayIndex].RayEnd;
		EndLocation.Z = StartLocation.Z;
		const bool bHit = RunCapsuleTrace(StartLocation, Rays[RayIndex].RayEnd, HitPos);
		if (bHit || TraceMode != ETraceMode::Discard)
		{
			OutPoints.Add(FNavLocation(bHit ? HitPos : EndLocation));
		}
	}
}

template<>
void FEQSHelpers::FBatchTrace::DoProject<EEnvTraceShape::Line>(TArray<FNavLocation>& Points, float StartOffsetZ, float EndOffsetZ, float HitOffsetZ)
{
	FVector HitPos(FVector::ZeroVector);
	for (int32 Idx = Points.Num() - 1; Idx >= 0; Idx--)
	{
		const bool bHit = RunLineTrace(Points[Idx].Location + FVector(0, 0, StartOffsetZ), Points[Idx].Location + FVector(0, 0, EndOffsetZ), HitPos);

		if (bHit)
		{
			Points[Idx] = FNavLocation(HitPos + FVector(0, 0, HitOffsetZ));
		}
		else if (TraceMode == ETraceMode::Discard)
		{
			Points.RemoveAt(Idx, EAllowShrinking::No);
		}

		if (TraceHits.IsValidIndex(Idx))
		{
			TraceHits[Idx] = bHit ? 1 : 0;
		}
	}
}

template<>
void FEQSHelpers::FBatchTrace::DoProject<EEnvTraceShape::Box>(TArray<FNavLocation>& Points, float StartOffsetZ, float EndOffsetZ, float HitOffsetZ)
{
	FVector HitPos(FVector::ZeroVector);
	for (int32 Idx = Points.Num() - 1; Idx >= 0; Idx--)
	{
		const bool bHit = RunBoxTrace(Points[Idx].Location + FVector(0, 0, StartOffsetZ), Points[Idx].Location + FVector(0, 0, EndOffsetZ), HitPos);
		if (bHit)
		{
			Points[Idx] = FNavLocation(HitPos + FVector(0, 0, HitOffsetZ));
		}
		else if (TraceMode == ETraceMode::Discard)
		{
			Points.RemoveAt(Idx, EAllowShrinking::No);
		}

		if (TraceHits.IsValidIndex(Idx))
		{
			TraceHits[Idx] = bHit ? 1 : 0;
		}
	}
}

template<>
void FEQSHelpers::FBatchTrace::DoProject<EEnvTraceShape::Sphere>(TArray<FNavLocation>& Points, float StartOffsetZ, float EndOffsetZ, float HitOffsetZ)
{
	FVector HitPos(FVector::ZeroVector);
	for (int32 Idx = Points.Num() - 1; Idx >= 0; Idx--)
	{
		const bool bHit = RunSphereTrace(Points[Idx].Location + FVector(0, 0, StartOffsetZ), Points[Idx].Location + FVector(0, 0, EndOffsetZ), HitPos);
		if (bHit)
		{
			Points[Idx] = FNavLocation(HitPos + FVector(0, 0, HitOffsetZ));
		}
		else if (TraceMode == ETraceMode::Discard)
		{
			Points.RemoveAt(Idx, EAllowShrinking::No);
		}

		if (TraceHits.IsValidIndex(Idx))
		{
			TraceHits[Idx] = bHit ? 1 : 0;
		}
	}
}

template<>
void FEQSHelpers::FBatchTrace::DoProject<EEnvTraceShape::Capsule>(TArray<FNavLocation>& Points, float StartOffsetZ, float EndOffsetZ, float HitOffsetZ)
{
	FVector HitPos(FVector::ZeroVector);
	for (int32 Idx = Points.Num() - 1; Idx >= 0; Idx--)
	{
		const bool bHit = RunCapsuleTrace(Points[Idx].Location + FVector(0, 0, StartOffsetZ), Points[Idx].Location + FVector(0, 0, EndOffsetZ), HitPos);
		if (bHit)
		{
			Points[Idx] = FNavLocation(HitPos + FVector(0, 0, HitOffsetZ));
		}
		else if (TraceMode == ETraceMode::Discard)
		{
			Points.RemoveAt(Idx, EAllowShrinking::No);
		}

		if (TraceHits.IsValidIndex(Idx))
		{
			TraceHits[Idx] = bHit ? 1 : 0;
		}
	}
}

void FEQSHelpers::RunRaycastsOnNavHitOnlyWalls(const ANavigationData& NavData, const UObject& Querier, const FEnvTraceData& TraceData, const FVector& SourcePt, TArray<FNavLocation>& Points, const TArray<AActor*>& IgnoredActors, const ETraceMode TraceMode)
{
	FSharedConstNavQueryFilter NavigationFilter = UNavigationQueryFilter::GetQueryFilter(NavData, &Querier, TraceData.NavigationFilter);

	TArray<FNavigationRaycastWork> RaycastWorkload;
	RaycastWorkload.Reserve(Points.Num());

	for (const auto& ItemLocation : Points)
	{
		RaycastWorkload.Add(FNavigationRaycastWork(SourcePt, ItemLocation.Location));
	}

	NavData.BatchRaycast(RaycastWorkload, NavigationFilter);

	// now we accept all the traces that didn't hit anything
	// while for the ones that did we make another, geometry pass
	// to tell if the navmesh trace hit a wall or a ledge
	// @todo handle "keep" here
	Points.Reset();
	TArray<FRayStartEnd> GeometryTraceCandidates;
	for (int32 ResultIndex = 0; ResultIndex < RaycastWorkload.Num(); ResultIndex++)
	{
		if (RaycastWorkload[ResultIndex].bDidHit)
		{
			if (GeometryTraceCandidates.GetSlack() == 0)
			{
				// reserve lazily
				GeometryTraceCandidates.Reserve(RaycastWorkload.Num());
			}
			GeometryTraceCandidates.Add(FRayStartEnd(RaycastWorkload[ResultIndex].HitLocation.Location, RaycastWorkload[ResultIndex].RayEnd));
		}
		else
		{
			Points.Add(RaycastWorkload[ResultIndex].HitLocation);
		}
	}

	if (GeometryTraceCandidates.Num() > 0)
	{
		check(NavData.GetWorld());

		FVector TraceExtent(TraceData.ExtentX, TraceData.ExtentY, TraceData.ExtentZ);

		FCollisionQueryParams TraceParams(SCENE_QUERY_STAT(EnvQueryTrace), TraceData.bTraceComplex);
		TraceParams.AddIgnoredActors(IgnoredActors);

		FBatchTrace TraceHelper(NavData.GetWorld(), TraceData, TraceParams, TraceExtent, TraceMode);

		switch (TraceData.TraceShape)
		{
		case EEnvTraceShape::Line:
			TraceHelper.DoMultiSourceMultiDestinations2D<EEnvTraceShape::Line>(GeometryTraceCandidates, Points);
			break;

		case EEnvTraceShape::Sphere:
			TraceHelper.DoMultiSourceMultiDestinations2D<EEnvTraceShape::Sphere>(GeometryTraceCandidates, Points);
			break;

		case EEnvTraceShape::Capsule:
			TraceHelper.DoMultiSourceMultiDestinations2D<EEnvTraceShape::Capsule>(GeometryTraceCandidates, Points);
			break;

		case EEnvTraceShape::Box:
			TraceHelper.DoMultiSourceMultiDestinations2D<EEnvTraceShape::Box>(GeometryTraceCandidates, Points);
			break;

		default:
			break;
		}
	}
}

bool FEQSHelpers::FBatchTrace::RunLineTrace(const FVector& StartPos, const FVector& EndPos, FVector& HitPos) const
{
	FHitResult OutHit;
	const bool bHit = World->LineTraceSingleByChannel(OutHit, StartPos, EndPos, Channel, QueryParams, ResponseParams);
	HitPos = OutHit.Location;
	return bHit;
}

bool FEQSHelpers::FBatchTrace::RunSphereTrace(const FVector& StartPos, const FVector& EndPos, FVector& HitPos) const
{
	FHitResult OutHit;
	const bool bHit = World->SweepSingleByChannel(OutHit, StartPos, EndPos, FQuat::Identity, Channel, FCollisionShape::MakeSphere(FloatCastChecked<float>(Extent.X, UE::LWC::DefaultFloatPrecision)), QueryParams, ResponseParams);
	HitPos = OutHit.Location;
	return bHit;
}

bool FEQSHelpers::FBatchTrace::RunCapsuleTrace(const FVector& StartPos, const FVector& EndPos, FVector& HitPos) const
{
	FHitResult OutHit;
	const bool bHit = World->SweepSingleByChannel(OutHit, StartPos, EndPos, FQuat::Identity, Channel, FCollisionShape::MakeCapsule(FloatCastChecked<float>(Extent.X, UE::LWC::DefaultFloatPrecision), FloatCastChecked<float>(Extent.Z, 1./16.)), QueryParams, ResponseParams);
	HitPos = OutHit.Location;
	return bHit;
}

bool FEQSHelpers::FBatchTrace::RunBoxTrace(const FVector& StartPos, const FVector& EndPos, FVector& HitPos) const
{
	FHitResult OutHit;
	const bool bHit = World->SweepSingleByChannel(OutHit, StartPos, EndPos, FQuat((EndPos - StartPos).Rotation()), Channel, FCollisionShape::MakeBox(Extent), QueryParams, ResponseParams);
	HitPos = OutHit.Location;
	return bHit;
}

void FEQSHelpers::RunNavRaycasts(const ANavigationData& NavData, const UObject& Querier, const FEnvTraceData& TraceData, const FVector& SourcePt, TArray<FNavLocation>& Points, const ETraceMode TraceMode /*= ETraceMode::Keep*/)
{
	FSharedConstNavQueryFilter NavigationFilter = UNavigationQueryFilter::GetQueryFilter(NavData, &Querier, TraceData.NavigationFilter);

	TArray<FNavigationRaycastWork> RaycastWorkload;
	RaycastWorkload.Reserve(Points.Num());

	for (const auto& ItemLocation : Points)
	{
		RaycastWorkload.Add(FNavigationRaycastWork(SourcePt, ItemLocation.Location));
	}

	NavData.BatchRaycast(RaycastWorkload, NavigationFilter);

	for (int32 Idx = 0; Idx < Points.Num(); Idx++)
	{
		Points[Idx] = RaycastWorkload[Idx].HitLocation;
	}

	if (TraceMode == ETraceMode::Discard)
	{
		for (int32 Idx = Points.Num() - 1; Idx >= 0; --Idx)
		{
			if (!RaycastWorkload[Idx].bDidHit)
			{
				Points.RemoveAt(Idx, EAllowShrinking::No);
			}
		}
	}
}

void FEQSHelpers::RunNavProjection(const ANavigationData& NavData, const UObject& Querier, const FEnvTraceData& TraceData, TArray<FNavLocation>& Points, const ETraceMode TraceMode /*= ETraceMode::Discard*/)
{
	FSharedConstNavQueryFilter NavigationFilter = UNavigationQueryFilter::GetQueryFilter(NavData, &Querier, TraceData.NavigationFilter);
	TArray<FNavigationProjectionWork> Workload;
	Workload.Reserve(Points.Num());

	if (TraceData.ProjectDown == TraceData.ProjectUp)
	{
		for (const auto& Point : Points)
		{
			Workload.Add(FNavigationProjectionWork(Point.Location));
		}
	}
	else
	{
		const FVector VerticalOffset = FVector(0, 0, (TraceData.ProjectUp - TraceData.ProjectDown) / 2);
		for (const auto& Point : Points)
		{
			Workload.Add(FNavigationProjectionWork(Point.Location + VerticalOffset));
		}
	}

	const FVector ProjectionExtent(TraceData.ExtentX, TraceData.ExtentX, (TraceData.ProjectDown + TraceData.ProjectUp) / 2);
	NavData.BatchProjectPoints(Workload, ProjectionExtent, NavigationFilter);

	for (int32 Idx = Workload.Num() - 1; Idx >= 0; Idx--)
	{
		if (Workload[Idx].bResult)
		{
			Points[Idx] = Workload[Idx].OutLocation;
			Points[Idx].Location.Z += TraceData.PostProjectionVerticalOffset;
		}
	}

	if (TraceMode == ETraceMode::Discard)
	{
		const FNavLocation* PointsBegin = Points.GetData();
		int32 NewNum = Algo::StableRemoveIf(Points, [&Workload, PointsBegin](FNavLocation& Point)
		{
			return !Workload[IntCastChecked<int32>(&Point - PointsBegin)].bResult;
		});
		Points.SetNum(NewNum, EAllowShrinking::No);
	}
}

void FEQSHelpers::RunPhysRaycasts(UWorld* World, const FEnvTraceData& TraceData, const FVector& SourcePt, TArray<FNavLocation>& Points, const TArray<AActor*>& IgnoredActors, const ETraceMode TraceMode)
{
	FVector TraceExtent(TraceData.ExtentX, TraceData.ExtentY, TraceData.ExtentZ);

	FCollisionQueryParams TraceParams(SCENE_QUERY_STAT(EnvQueryTrace), TraceData.bTraceComplex);
	TraceParams.AddIgnoredActors(IgnoredActors);

	FBatchTrace BatchOb(World, TraceData, TraceParams, TraceExtent, TraceMode);

	switch (TraceData.TraceShape)
	{
	case EEnvTraceShape::Line:
		BatchOb.DoSingleSourceMultiDestinations<EEnvTraceShape::Line>(SourcePt, Points);
		break;

	case EEnvTraceShape::Sphere:
		BatchOb.DoSingleSourceMultiDestinations<EEnvTraceShape::Sphere>(SourcePt, Points);
		break;

	case EEnvTraceShape::Capsule:
		BatchOb.DoSingleSourceMultiDestinations<EEnvTraceShape::Capsule>(SourcePt, Points);
		break;

	case EEnvTraceShape::Box:
		BatchOb.DoSingleSourceMultiDestinations<EEnvTraceShape::Box>(SourcePt, Points);
		break;

	default:
		break;
	}
}

void FEQSHelpers::RunPhysProjection(UWorld* World, const FEnvTraceData& TraceData, TArray<FNavLocation>& Points, const ETraceMode TraceMode)
{
	FVector TraceExtent(TraceData.ExtentX, TraceData.ExtentY, TraceData.ExtentZ);

	FCollisionQueryParams TraceParams(SCENE_QUERY_STAT(EnvQueryTrace), TraceData.bTraceComplex);

	FBatchTrace BatchOb(World, TraceData, TraceParams, TraceExtent, TraceMode);

	switch (TraceData.TraceShape)
	{
	case EEnvTraceShape::Line:
		BatchOb.DoProject<EEnvTraceShape::Line>(Points, TraceData.ProjectUp, -TraceData.ProjectDown, TraceData.PostProjectionVerticalOffset);
		break;

	case EEnvTraceShape::Sphere:
		BatchOb.DoProject<EEnvTraceShape::Sphere>(Points, TraceData.ProjectUp, -TraceData.ProjectDown, TraceData.PostProjectionVerticalOffset);
		break;

	case EEnvTraceShape::Capsule:
		BatchOb.DoProject<EEnvTraceShape::Capsule>(Points, TraceData.ProjectUp, -TraceData.ProjectDown, TraceData.PostProjectionVerticalOffset);
		break;

	case EEnvTraceShape::Box:
		BatchOb.DoProject<EEnvTraceShape::Box>(Points, TraceData.ProjectUp, -TraceData.ProjectDown, TraceData.PostProjectionVerticalOffset);
		break;

	default:
		break;
	}
}

void FEQSHelpers::RunPhysProjection(UWorld* World, const FEnvTraceData& TraceData, TArray<FNavLocation>& Points, TArray<uint8>& TraceHits)
{
	FVector TraceExtent(TraceData.ExtentX, TraceData.ExtentY, TraceData.ExtentZ);

	FCollisionQueryParams TraceParams(SCENE_QUERY_STAT(EnvQueryTrace), TraceData.bTraceComplex);

	FBatchTrace BatchOb(World, TraceData, TraceParams, TraceExtent, ETraceMode::Keep);
	BatchOb.TraceHits.AddZeroed(Points.Num());

	switch (TraceData.TraceShape)
	{
	case EEnvTraceShape::Line:
		BatchOb.DoProject<EEnvTraceShape::Line>(Points, TraceData.ProjectUp, -TraceData.ProjectDown, TraceData.PostProjectionVerticalOffset);
		break;

	case EEnvTraceShape::Sphere:
		BatchOb.DoProject<EEnvTraceShape::Sphere>(Points, TraceData.ProjectUp, -TraceData.ProjectDown, TraceData.PostProjectionVerticalOffset);
		break;

	case EEnvTraceShape::Capsule:
		BatchOb.DoProject<EEnvTraceShape::Capsule>(Points, TraceData.ProjectUp, -TraceData.ProjectDown, TraceData.PostProjectionVerticalOffset);
		break;

	case EEnvTraceShape::Box:
		BatchOb.DoProject<EEnvTraceShape::Box>(Points, TraceData.ProjectUp, -TraceData.ProjectDown, TraceData.PostProjectionVerticalOffset);
		break;

	default:
		break;
	}

	TraceHits.Append(BatchOb.TraceHits);
}

//----------------------------------------------------------------------//
// DEPRECATED
//----------------------------------------------------------------------//
void FEQSHelpers::RunNavRaycasts(const ANavigationData& NavData, const FEnvTraceData& TraceData, const FVector& SourcePt, TArray<FNavLocation>& Points, ETraceMode TraceMode)
{
	RunNavRaycasts(NavData, NavData, TraceData, SourcePt, Points, TraceMode);
}

void FEQSHelpers::RunNavProjection(const ANavigationData& NavData, const FEnvTraceData& TraceData, TArray<FNavLocation>& Points, ETraceMode TraceMode)
{
	RunNavProjection(NavData, NavData, TraceData, Points, TraceMode);
}
