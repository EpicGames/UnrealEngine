// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/CollisionProfile.h"
#include "Engine/EngineTypes.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "AI/Navigation/NavigationTypes.h"
#include "CollisionQueryParams.h"
#include "WorldCollision.h"
#include "Engine/World.h"

class ANavigationData;
class Error;

namespace FEQSHelpers
{
	enum class ETraceMode : uint8
	{
		Keep,
		Discard,
	};

	struct FBatchTrace
	{
		UWorld* World;
		const FVector Extent;
		const FCollisionQueryParams QueryParams;
		FCollisionResponseParams ResponseParams;
		enum ECollisionChannel Channel;
		ETraceMode TraceMode;
		TArray<uint8> TraceHits;

		FBatchTrace(UWorld* InWorld, enum ECollisionChannel InChannel, const FCollisionQueryParams& InParams,
			const FVector& InExtent, ETraceMode InTraceMode)
			: World(InWorld), Extent(InExtent), QueryParams(InParams), Channel(InChannel), TraceMode(InTraceMode)
		{

		}

		FBatchTrace(UWorld* InWorld, const FEnvTraceData& TraceData, const FCollisionQueryParams& InParams,
			const FVector& InExtent, ETraceMode InTraceMode)
			: World(InWorld), Extent(InExtent), QueryParams(InParams), TraceMode(InTraceMode)
		{
			if (TraceData.TraceMode == EEnvQueryTrace::GeometryByProfile)
			{
				UCollisionProfile::GetChannelAndResponseParams(TraceData.TraceProfileName, Channel, ResponseParams);
			}
			else
			{
				Channel = UEngineTypes::ConvertToCollisionChannel(TraceData.TraceChannel);
			}
		}
		
		bool RunLineTrace(const FVector& StartPos, const FVector& EndPos, FVector& HitPos) const;

		bool RunSphereTrace(const FVector& StartPos, const FVector& EndPos, FVector& HitPos) const;

		bool RunCapsuleTrace(const FVector& StartPos, const FVector& EndPos, FVector& HitPos) const;

		bool RunBoxTrace(const FVector& StartPos, const FVector& EndPos, FVector& HitPos) const;

		template<EEnvTraceShape::Type TraceType>
		void DoSingleSourceMultiDestinations(const FVector& Source, TArray<FNavLocation>& Points)
		{
			UE_LOGF(LogEQS, Error, "FBatchTrace::DoSingleSourceMultiDestinations called with unhandled trace type: %d", int32(TraceType));
		}

		/** note that this function works slightly different in terms of discarding items. 
		 *	"Accepted" items get added to the OutPoints array*/
		template<EEnvTraceShape::Type TraceType>
		void DoMultiSourceMultiDestinations2D(const TArray<FRayStartEnd>& Rays, TArray<FNavLocation>& OutPoints)
		{
			UE_LOGF(LogEQS, Error, "FBatchTrace::DoMultiSourceMultiDestinations2D called with unhandled trace type: %d", int32(TraceType));
		}

		template<EEnvTraceShape::Type TraceType>
		void DoProject(TArray<FNavLocation>& Points, float StartOffsetZ, float EndOffsetZ, float HitOffsetZ)
		{
			UE_LOGF(LogEQS, Error, "FBatchTrace::DoSingleSourceMultiDestinations called with unhandled trace type: %d", int32(TraceType));
		}
	};

	AIMODULE_API void RunNavRaycasts(const ANavigationData& NavData, const UObject& Querier, const FEnvTraceData& TraceData, const FVector& SourcePt, TArray<FNavLocation>& Points, const ETraceMode TraceMode = ETraceMode::Keep);
	AIMODULE_API void RunNavProjection(const ANavigationData& NavData, const UObject& Querier, const FEnvTraceData& TraceData, TArray<FNavLocation>& Points, const ETraceMode TraceMode = ETraceMode::Discard);
	AIMODULE_API void RunPhysRaycasts(UWorld* World, const FEnvTraceData& TraceData, const FVector& SourcePt, TArray<FNavLocation>& Points, const TArray<AActor*>& IgnoredActors, const ETraceMode TraceMode = ETraceMode::Keep);
	AIMODULE_API void RunPhysProjection(UWorld* World, const FEnvTraceData& TraceData, TArray<FNavLocation>& Points, const ETraceMode TraceMode = ETraceMode::Discard);
	AIMODULE_API void RunPhysProjection(UWorld* World, const FEnvTraceData& TraceData, TArray<FNavLocation>& Points, TArray<uint8>& TraceHits);
	/** Does initial raycast on navmesh, just like, RunNavRaycasts but
	 *	once it hits navmesh edge it does a geometry trace to determine whether
	 *	it hit a wall or a ledge (empty space). */
	AIMODULE_API void RunRaycastsOnNavHitOnlyWalls(const ANavigationData& NavData, const UObject& Querier, const FEnvTraceData& TraceData, const FVector& SourcePt, TArray<FNavLocation>& Points, const TArray<AActor*>& IgnoredActors, const ETraceMode TraceMode = ETraceMode::Keep);

	// deprecated

	UE_DEPRECATED_FORGAME(4.12, "This function is now deprecated, please use version with Querier argument instead.")
	void RunNavRaycasts(const ANavigationData& NavData, const FEnvTraceData& TraceData, const FVector& SourcePt, TArray<FNavLocation>& Points, ETraceMode TraceMode = ETraceMode::Keep);

	UE_DEPRECATED_FORGAME(4.12, "This function is now deprecated, please use version with Querier argument instead.")
	void RunNavProjection(const ANavigationData& NavData, const FEnvTraceData& TraceData, TArray<FNavLocation>& Points, ETraceMode TraceMode = ETraceMode::Discard);
}
