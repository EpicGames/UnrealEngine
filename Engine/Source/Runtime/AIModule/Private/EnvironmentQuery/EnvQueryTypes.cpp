// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/EnvQueryTypes.h"
#include "UObject/Package.h"
#include "AI/Navigation/NavAgentInterface.h"
#include "NavFilters/NavigationQueryFilter.h"
#include "NavigationSystem.h"
#include "NavMesh/RecastNavMesh.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_VectorBase.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_ActorBase.h"
#include "BehaviorTree/BTNode.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Object.h"
#include "VisualLogger/VisualLogger.h"
#include "DataProviders/AIDataProvider_QueryParams.h"
#include "EnvironmentQuery/EnvQuery.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "EnvironmentQuery/EnvQueryManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryTypes)

#define LOCTEXT_NAMESPACE "EnvQueryGenerator"

const float UEnvQueryTypes::SkippedItemValue = -MAX_flt;
float UEnvQueryTypes::UnlimitedStepTime = -1.f;

//----------------------------------------------------------------------//
// FEnvQueryResult
//----------------------------------------------------------------------//
AActor* FEnvQueryResult::GetItemAsActor(int32 Index) const
{
	if (Items.IsValidIndex(Index) &&
		(ItemType.Get() != nullptr) &&
		ItemType->IsChildOf(UEnvQueryItemType_ActorBase::StaticClass()))
	{
		UEnvQueryItemType_ActorBase* DefTypeOb = ItemType->GetDefaultObject<UEnvQueryItemType_ActorBase>();
		check(DefTypeOb != nullptr);
		return DefTypeOb->GetActor(RawData.GetData() + Items[Index].DataOffset);
	}

	return nullptr;
}

FVector FEnvQueryResult::GetItemAsLocation(int32 Index) const
{
	if (Items.IsValidIndex(Index) &&
		(ItemType.Get() != nullptr) &&
		ItemType->IsChildOf(UEnvQueryItemType_VectorBase::StaticClass()))
	{
		UEnvQueryItemType_VectorBase* DefTypeOb = ItemType->GetDefaultObject<UEnvQueryItemType_VectorBase>();
		check(DefTypeOb != nullptr);
		return DefTypeOb->GetItemLocation(RawData.GetData() + Items[Index].DataOffset);
	}

	return FVector::ZeroVector;
}

void FEnvQueryResult::GetAllAsActors(TArray<AActor*>& OutActors) const
{
	if ((ItemType.Get() != nullptr) &&
		ItemType->IsChildOf(UEnvQueryItemType_ActorBase::StaticClass()) &&
		(Items.Num() > 0))
	{
		UEnvQueryItemType_ActorBase* DefTypeOb = ItemType->GetDefaultObject<UEnvQueryItemType_ActorBase>();
		check(DefTypeOb != nullptr);
		
		OutActors.Reserve(OutActors.Num() + Items.Num());

		for (const FEnvQueryItem& Item : Items)
		{
			OutActors.Add(DefTypeOb->GetActor(RawData.GetData() + Item.DataOffset));
		}
	}
}

void FEnvQueryResult::GetAllAsLocations(TArray<FVector>& OutLocations) const
{
	if ((ItemType.Get() != nullptr) &&
		ItemType->IsChildOf(UEnvQueryItemType_VectorBase::StaticClass()) &&
		(Items.Num() > 0))
	{
		UEnvQueryItemType_VectorBase* DefTypeOb = ItemType->GetDefaultObject<UEnvQueryItemType_VectorBase>();
		check(DefTypeOb != nullptr);

		OutLocations.Reserve(OutLocations.Num() + Items.Num());

		for (const FEnvQueryItem& Item : Items)
		{
			OutLocations.Add(DefTypeOb->GetItemLocation(RawData.GetData() + Item.DataOffset));
		}
	}
}

//----------------------------------------------------------------------//
// UEnvQueryTypes
//----------------------------------------------------------------------//
FText UEnvQueryTypes::GetShortTypeName(const UObject* Ob)
{
	if (Ob == nullptr)
	{
		return LOCTEXT("Unknown", "unknown");
	}

	const UClass* ObClass = Ob->IsA(UClass::StaticClass()) ? (const UClass*)Ob : Ob->GetClass();
	check(ObClass != nullptr);
	if (ObClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
	{
		return FText::FromString(ObClass->GetName().LeftChop(2));
	}

	FString TypeDesc = ObClass->GetName();
	const int32 ShortNameIdx = TypeDesc.Find(TEXT("_"), ESearchCase::CaseSensitive);
	if (ShortNameIdx != INDEX_NONE)
	{
		TypeDesc.MidInline(ShortNameIdx + 1, MAX_int32, EAllowShrinking::No);
	}

	return FText::FromString(TypeDesc);
}

FText UEnvQueryTypes::DescribeContext(TSubclassOf<UEnvQueryContext> ContextClass)
{
	return GetShortTypeName(ContextClass);
}

FText FEnvDirection::ToText() const
{
	if (DirMode == EEnvDirection::TwoPoints)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("LineFrom"), UEnvQueryTypes::DescribeContext(LineFrom));
		Args.Add(TEXT("LineTo"), UEnvQueryTypes::DescribeContext(LineTo));

		return FText::Format(LOCTEXT("DescribeLineFromAndTo", "[From: {LineFrom} To: {LineTo}]"), Args);
	}
	else
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("Rotation"), UEnvQueryTypes::DescribeContext(Rotation));

		return FText::Format(LOCTEXT("DescribeRotation", "[{Rotation} rotation]"), Args);
	}
}

FEnvTraceData::FEnvTraceData() :
	VersionNum(0), 
	ProjectDown(1024.0f), ProjectUp(1024.0f), ExtentX(10.0f), ExtentY(10.0f), ExtentZ(10.0f),
	PostProjectionVerticalOffset(0.0f),	TraceChannel(TraceTypeQuery1), SerializedChannel(ECC_WorldStatic), TraceProfileName(NAME_None),
	TraceShape(EEnvTraceShape::Line), TraceMode(EEnvQueryTrace::None),
	bTraceComplex(false), bOnlyBlockingHits(true),
	bCanTraceOnNavMesh(true), bCanTraceOnGeometry(true), bCanDisableTrace(true), bCanProjectDown(false)
{
}

FText FEnvTraceData::ToText(FEnvTraceData::EDescriptionMode DescMode) const
{
	FText Desc;

	if (TraceMode == EEnvQueryTrace::GeometryByChannel || TraceMode == EEnvQueryTrace::GeometryByProfile)
	{
		FNumberFormattingOptions NumberFormatOptions;
		NumberFormatOptions.MaximumFractionalDigits = 2;

		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("ExtentX"), FText::AsNumber(ExtentX, &NumberFormatOptions));
			Args.Add(TEXT("ExtentY"), FText::AsNumber(ExtentY, &NumberFormatOptions));
			Args.Add(TEXT("ExtentZ"), FText::AsNumber(ExtentZ, &NumberFormatOptions));

			Desc = (TraceShape == EEnvTraceShape::Line) ? LOCTEXT("Line", "line") :
				(TraceShape == EEnvTraceShape::Sphere) ? FText::Format(LOCTEXT("SphereWithRadius", "sphere (radius: {ExtentX})"), Args) :
				(TraceShape == EEnvTraceShape::Capsule) ? FText::Format(LOCTEXT("CasuleWithRadiusHalfHeight", "capsule (radius: {ExtentX}, half height: {ExtentZ})"), Args) :
				(TraceShape == EEnvTraceShape::Box) ? FText::Format(LOCTEXT("BoxWithExtents", "box (extent: {ExtentX} {ExtentY} {ExtentZ)"), Args) :
				LOCTEXT("Unknown", "unknown");
		}

		if (DescMode == FEnvTraceData::Brief)
		{
			static UEnum* ChannelEnum = StaticEnum<ETraceTypeQuery>();
			check(ChannelEnum != nullptr);

			FFormatNamedArguments Args;
			Args.Add(TEXT("ExtentDescription"), Desc);
			Args.Add(TEXT("ProjectionTraceDesc"), bCanProjectDown ? LOCTEXT("Projection", "projection") : LOCTEXT("Trace", "trace"));
			Args.Add(TEXT("Channel"), ChannelEnum->GetDisplayNameTextByValue(TraceChannel));

			Desc = FText::Format(LOCTEXT("GeometryBriefDescription", "{ExtentDescription} {ProjectionTraceDesc} on {Channel}"), Args);
		}
		else
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("Description"), Desc);

			if (bTraceComplex)
			{
				Desc = FText::Format(LOCTEXT("DescWithComplexCollision", "{Description}, complex collision"), Args);
			}

			if (!bOnlyBlockingHits)
			{
				Desc = FText::Format(LOCTEXT("DescWithNonBlocking", "{Description}, accept non blocking"), Args);
			}
		}
	}
	else if (TraceMode == EEnvQueryTrace::Navigation)
	{
		Desc = LOCTEXT("Navmesh", "navmesh");
		if (bCanProjectDown)
		{
			FNumberFormattingOptions NumberFormatOptions;
			NumberFormatOptions.MaximumFractionalDigits = 0;

			FFormatNamedArguments Args;
			Args.Add(TEXT("Description"), Desc);
			Args.Add(TEXT("Direction"), (ProjectDown == ProjectUp) ? LOCTEXT("Height", "height") : (ProjectDown > ProjectUp) ? LOCTEXT("Down", "down") : LOCTEXT("Up", "up"));
			Args.Add(TEXT("ProjectionAmount"), FText::AsNumber(FMath::Max(ProjectDown, ProjectUp), &NumberFormatOptions));

			Desc = FText::Format(LOCTEXT("DescriptionWithProjection", "projection ({Direction}: {ProjectionAmount}"), Args);

			if (ExtentX > 1.0f)
			{
				NumberFormatOptions.MaximumFractionalDigits = 2;

				FFormatNamedArguments RadiusArgs;
				RadiusArgs.Add(TEXT("Description"), Desc);
				RadiusArgs.Add(TEXT("Radius"), FText::AsNumber(ExtentX, &NumberFormatOptions));

				Desc = FText::Format(LOCTEXT("DescriptionWithRadius", "{Description}, radius {Radius}"), RadiusArgs);
			}

			FFormatNamedArguments EndingArgs;
			EndingArgs.Add(TEXT("Description"), Desc);
			Desc = FText::Format(LOCTEXT("DescriptionWithEnding", "{Description})"), EndingArgs);
		}
		else
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("Description"), Desc);

			Desc = FText::Format(LOCTEXT("DescriptionWithTrace", "{Description} trace"), Args);
			if (NavigationFilter)
			{
				FFormatNamedArguments NavFilterArgs;
				NavFilterArgs.Add(TEXT("Description"), Desc);
				NavFilterArgs.Add(TEXT("NavigationFilter"), FText::FromString(NavigationFilter->GetName()));
				Desc = FText::Format(LOCTEXT("DescriptionWithNavigationFilter", "{Description} (filter: {NavigationFilter})"), NavFilterArgs);
			}
		}
	}
	else if (TraceMode == EEnvQueryTrace::NavigationOverLedges)
	{
		Desc = LOCTEXT("NavigationOverLedges", "Navmesh trace going over navmesh ledges");
	}

	return Desc;
}

void FEnvTraceData::SetGeometryOnly()
{
	TraceMode = EEnvQueryTrace::GeometryByChannel;
	bCanTraceOnGeometry = true;
	bCanTraceOnNavMesh = false;
	bCanDisableTrace = false;
}

void FEnvTraceData::SetNavmeshOnly()
{
	TraceMode = EEnvQueryTrace::Navigation;
	bCanTraceOnGeometry = false;
	bCanTraceOnNavMesh = true;
	bCanDisableTrace = false;
}

void FEnvTraceData::OnPostLoad()
{
	if (VersionNum == 0)
	{
		// update trace channels
		SerializedChannel = UEngineTypes::ConvertToCollisionChannel(TraceChannel.GetValue());
	}

	TraceChannel = UEngineTypes::ConvertToTraceType(SerializedChannel);
	VersionNum = 1;
}


//----------------------------------------------------------------------//
// namespace FEQSHelpers
//----------------------------------------------------------------------//
namespace FEQSHelpers
{
	const ANavigationData* FindNavigationDataForQuery(FEnvQueryInstance& QueryInstance, TSubclassOf<UEnvQueryContext> NavDataOverrideContext)
	{
		const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(QueryInstance.World);
		if (NavSys == nullptr)
		{
			return nullptr;
		}

		if (NavDataOverrideContext)
		{
			TArray<AActor*> NavDataCandidateActors;
			if (QueryInstance.PrepareContext(NavDataOverrideContext, NavDataCandidateActors))
			{
				for (const AActor* NavDataCandidateActor : NavDataCandidateActors)
				{
					if (const ANavigationData* NavDataCandidate = Cast<ANavigationData>(NavDataCandidateActor))
					{
						// NavigationData found, we can stop here
						return NavDataCandidate;
					}
					if (NavSys)
					{
						// Check if this actor use a NavData that could be used
						if (const INavAgentInterface* NavAgent = Cast<INavAgentInterface>(QueryInstance.Owner.Get()))
						{
							const FNavAgentProperties& NavAgentProps = NavAgent->GetNavAgentPropertiesRef();
							if (NavAgentProps.IsValid() || NavAgentProps.PreferredNavData.IsValid())	// test that to avoid GetNavDataForProps warning
							{
								if (const ANavigationData* AgentNavData = NavSys->GetNavDataForProps(NavAgent->GetNavAgentPropertiesRef()))
								{
									return AgentNavData;
								}
							}
						}
					}
				}
			}
		}

		// try to match navigation agent for querier
		if (const INavAgentInterface* NavAgent = QueryInstance.Owner.IsValid() ? Cast<INavAgentInterface>(QueryInstance.Owner.Get()) : nullptr)
		{
			const FNavAgentProperties& NavAgentProps = NavAgent->GetNavAgentPropertiesRef();
			if (NavAgentProps.IsValid() || NavAgentProps.PreferredNavData.IsValid())
			{
				return NavSys->GetNavDataForProps(NavAgentProps, NavAgent->GetNavAgentLocation());
			}
		}

		return NavSys->GetDefaultNavDataInstance();
	}
}

//----------------------------------------------------------------------//
// FEnvQueryInstance
//----------------------------------------------------------------------//
FEnvQueryInstance::FEnvQueryInstance() 
	: World(nullptr)
	, CurrentTest(INDEX_NONE)
	, CurrentTestStartingItem(INDEX_NONE)
	, NumValidItems(0)
	, ValueSize(0)
#if USE_EQS_DEBUGGER
	, NumProcessedItems(0)
	, bStoreDebugInfo(bDebuggingInfoEnabled)
#endif // USE_EQS_DEBUGGER
	, bFoundSingleResult(false)
	, bPassOnSingleResult(false)
	, bHasLoggedTimeLimitWarning(false)
	, bIsCurrentlyRunningAsync(false)
	, StartTime(0)
	, TotalExecutionTime(0)
	, CurrentStepTimeLimit(0)
	, Mode(EEnvQueryRunMode::SingleResult)
	, ItemTypeVectorCDO(nullptr)
	, ItemTypeActorCDO(nullptr)
{
	IncStats();
}

FEnvQueryInstance::FEnvQueryInstance(const FEnvQueryInstance& Other) 
{ 
	*this = Other; 
	IncStats(); 
}

FEnvQueryInstance::~FEnvQueryInstance() 
{ 
	DecStats(); 
}

//----------------------------------------------------------------------//
// FEQSConfigurableQueryParam
//----------------------------------------------------------------------//
void FAIDynamicParam::ConfigureBBKey(UObject &QueryOwner)
{
	BBKey.AllowNoneAsValue(true);

	switch (ParamType)
	{
	case EAIParamType::Float:
	case EAIParamType::Int:
		BBKey.AddFloatFilter(&QueryOwner, ParamName);
		BBKey.AddIntFilter(&QueryOwner, ParamName);
		break;
	case EAIParamType::Bool:
		BBKey.AddBoolFilter(&QueryOwner, ParamName);
		break;
	default:
		checkNoEntry();
		break;
	}
};

void FAIDynamicParam::GenerateConfigurableParamsFromNamedValues(UObject &QueryOwner, TArray<FAIDynamicParam>& OutQueryConfig, TArray<FEnvNamedValue>& InQueryParams)
{
	for (const FEnvNamedValue& Param : InQueryParams)
	{
		FAIDynamicParam& NewParam = OutQueryConfig[OutQueryConfig.Add(FAIDynamicParam())];
		NewParam.ParamName = Param.ParamName;
		NewParam.ParamType = Param.ParamType;
		NewParam.Value = Param.Value;
		NewParam.ConfigureBBKey(QueryOwner);
	}
}

//----------------------------------------------------------------------//
// FEQSQueryExecutionParams
//----------------------------------------------------------------------//
FEQSParametrizedQueryExecutionRequest::FEQSParametrizedQueryExecutionRequest()
	: QueryTemplate(nullptr)
	, RunMode(EEnvQueryRunMode::SingleResult)
	, bUseBBKeyForQueryTemplate(false)
	, bInitialized(false)
{
	//EQSQueryBlackboardKey.AddObjectFilter(this, GET_MEMBER_NAME_CHECKED(FEQSQueryExecutionParams, EQSQueryBlackboardKey), UEnvQuery::StaticClass());
}

void FEQSParametrizedQueryExecutionRequest::InitForOwnerAndBlackboard(UObject& Owner, UBlackboardData* BBAsset)
{
	if (bInitialized == true)
	{
		return;
	}

	bInitialized = true;

	EQSQueryBlackboardKey.AddObjectFilter(&Owner, NAME_None, UEnvQuery::StaticClass());

	if ((bUseBBKeyForQueryTemplate || QueryConfig.Num() > 0) && BBAsset)
	{
		for (FAIDynamicParam& RuntimeParam : QueryConfig)
		{
			if (RuntimeParam.BBKey.NeedsResolving())
			{
				RuntimeParam.BBKey.ResolveSelectedKey(*BBAsset);
			}
		}

		if (bUseBBKeyForQueryTemplate)
		{
			EQSQueryBlackboardKey.ResolveSelectedKey(*BBAsset);
		}
	}
}

int32 FEQSParametrizedQueryExecutionRequest::Execute(UObject& QueryOwner, const UBlackboardComponent* BlackboardComponent, FQueryFinishedSignature& QueryFinishedDelegate)
{
	if (bUseBBKeyForQueryTemplate)
	{
		check(BlackboardComponent);

		// set QueryTemplate to contents of BB key
		UObject* QueryTemplateObject = BlackboardComponent->GetValue<UBlackboardKeyType_Object>(EQSQueryBlackboardKey.GetSelectedKeyID());
		QueryTemplate = Cast<UEnvQuery>(QueryTemplateObject);

		UE_CVLOG(QueryTemplate == nullptr, &QueryOwner, LogBehaviorTree, Warning, TEXT("Trying to run EQS query configured to use BB key, but indicated key (%s) doesn't contain EQS template pointer")
			, *EQSQueryBlackboardKey.SelectedKeyName.ToString());
	}

	if (QueryTemplate != nullptr)
	{
		FEnvQueryRequest QueryRequest(QueryTemplate, &QueryOwner);
		if (QueryConfig.Num() > 0)
		{
			// resolve 
			for (FAIDynamicParam& RuntimeParam : QueryConfig)
			{
				QueryRequest.SetDynamicParam(RuntimeParam, BlackboardComponent);
			}
		}

		return QueryRequest.Execute(RunMode, QueryFinishedDelegate);
	}

	return INDEX_NONE;
}

#if WITH_EDITOR
void FEQSParametrizedQueryExecutionRequest::PostEditChangeProperty(UObject& Owner, FPropertyChangedEvent& PropertyChangedEvent)
{
	check(PropertyChangedEvent.MemberProperty);
	check(PropertyChangedEvent.Property);

	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FEQSParametrizedQueryExecutionRequest, QueryTemplate))
	{
		if (QueryTemplate)
		{
			QueryTemplate->CollectQueryParams(Owner, QueryConfig);
		}
		else
		{
			QueryConfig.Reset();
		}
	}
	else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FAIDynamicParam, bAllowBBKey))
	{
		const int32 ChangedIndex = PropertyChangedEvent.GetArrayIndex(GET_MEMBER_NAME_CHECKED(FEQSParametrizedQueryExecutionRequest, QueryConfig).ToString());
		if (QueryConfig.IsValidIndex(ChangedIndex))
		{
			if (!QueryConfig[ChangedIndex].bAllowBBKey)
			{
				QueryConfig[ChangedIndex].BBKey.InvalidateResolvedKey();
			}
		}
	}
}
#endif // WITH_EDITOR

FArchive& operator<<(FArchive& Ar, FEnvQueryDebugProfileData::FStep& Data)
{
	Ar << Data.ExecutionTime;
	Ar << Data.NumProcessedItems;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FEnvQueryDebugProfileData::FOptionStat& Data)
{
	Ar << Data.StepData;
	Ar << Data.NumRuns;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FEnvQueryDebugProfileData::FOptionData& Data)
{
	Ar << Data.NumGenerators;
	Ar << Data.GeneratorNames;
	Ar << Data.OptionIdx;
	Ar << Data.TestIndices;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FEnvQueryDebugProfileData& Data)
{
	int32 VersionNum = 1;
	Ar << VersionNum;

	Ar << Data.OptionStats;
	Ar << Data.OptionData;
	return Ar;
}

//----------------------------------------------------------------------//
// DEPRECATED
//----------------------------------------------------------------------//
#if WITH_RECAST

namespace FEQSHelpers
{
	const ARecastNavMesh* FindNavMeshForQuery(FEnvQueryInstance& QueryInstance)
	{
		return Cast<const ARecastNavMesh>(FEQSHelpers::FindNavigationDataForQuery(QueryInstance));
	}
}

#endif // WITH_RECAST

#undef LOCTEXT_NAMESPACE
