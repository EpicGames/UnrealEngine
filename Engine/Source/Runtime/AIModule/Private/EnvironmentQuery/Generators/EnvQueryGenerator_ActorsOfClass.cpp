// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Generators/EnvQueryGenerator_ActorsOfClass.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "EngineUtils.h"
#include "EnvironmentQuery/Contexts/EnvQueryContext_Querier.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryGenerator_ActorsOfClass)

#define LOCTEXT_NAMESPACE "EnvQueryGenerator"

UEnvQueryGenerator_ActorsOfClass::UEnvQueryGenerator_ActorsOfClass(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer)
{
	ItemType = UEnvQueryItemType_Actor::StaticClass();

	SearchedActorClass = AActor::StaticClass();
	GenerateOnlyActorsInRadius.DefaultValue = true;
	SearchRadius.DefaultValue = 500.0f;
	SearchCenter = UEnvQueryContext_Querier::StaticClass();
}

void UEnvQueryGenerator_ActorsOfClass::GenerateItems(FEnvQueryInstance& QueryInstance) const
{
	if (SearchedActorClass == nullptr)
	{
		return;
	}

	UObject* QueryOwner = QueryInstance.Owner.Get();
	if (QueryOwner == nullptr)
	{
		return;
	}
	
	UWorld* World = GEngine->GetWorldFromContextObject(QueryOwner, EGetWorldErrorMode::LogAndReturnNull);
	if (World == nullptr)
	{
		return;
	}

	GenerateOnlyActorsInRadius.BindData(QueryOwner, QueryInstance.QueryID);
	bool bUseRadius = GenerateOnlyActorsInRadius.GetValue();

	TArray<AActor*> MatchingActors;
	if (bUseRadius)
	{
		TArray<FVector> ContextLocations;
		QueryInstance.PrepareContext(SearchCenter, ContextLocations);

		SearchRadius.BindData(QueryOwner, QueryInstance.QueryID);
		const float RadiusValue = SearchRadius.GetValue();
		const float RadiusSq = FMath::Square(RadiusValue);

		for (TActorIterator<AActor> ItActor = TActorIterator<AActor>(World, SearchedActorClass); ItActor; ++ItActor)
		{
			for (int32 ContextIndex = 0; ContextIndex < ContextLocations.Num(); ++ContextIndex)
			{
				if (FVector::DistSquared(ContextLocations[ContextIndex], ItActor->GetActorLocation()) < RadiusSq)
				{
					MatchingActors.Add(*ItActor);
					break;
				}
			}
		}
	}
	else
	{	// If radius is not positive, ignore Search Center and Search Radius and just return all actors of class.
		for (TActorIterator<AActor> ItActor = TActorIterator<AActor>(World, SearchedActorClass); ItActor; ++ItActor)
		{
			MatchingActors.Add(*ItActor);
		}
	}

	ProcessItems(QueryInstance, MatchingActors);
	QueryInstance.AddItemData<UEnvQueryItemType_Actor>(MatchingActors);
}

FText UEnvQueryGenerator_ActorsOfClass::GetDescriptionTitle() const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("DescriptionTitle"), Super::GetDescriptionTitle());
	Args.Add(TEXT("ActorsClass"), FText::FromString(GetNameSafe(SearchedActorClass)));

	if (!GenerateOnlyActorsInRadius.IsDynamic() && !GenerateOnlyActorsInRadius.GetValue())
	{
		return FText::Format(LOCTEXT("DescriptionGenerateActors", "{DescriptionTitle}: generate set of actors of {ActorsClass}"), Args);
	}

	Args.Add(TEXT("DescribeContext"), UEnvQueryTypes::DescribeContext(SearchCenter));
	return FText::Format(LOCTEXT("DescriptionGenerateActorsAroundContext", "{DescriptionTitle}: generate set of actors of {ActorsClass} around {DescribeContext}"), Args);
};

FText UEnvQueryGenerator_ActorsOfClass::GetDescriptionDetails() const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("Radius"), FText::FromString(SearchRadius.ToString()));

	FText Desc = FText::Format(LOCTEXT("ActorsOfClassDescription", "radius: {Radius}"), Args);

	return Desc;
}

#undef LOCTEXT_NAMESPACE

