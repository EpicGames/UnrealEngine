// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Generators/EnvQueryGenerator_BlueprintBase.h"
#include "GameFramework/Actor.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_ActorBase.h"
#include "EnvironmentQuery/Contexts/EnvQueryContext_Querier.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_Point.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryGenerator_BlueprintBase)

#define LOCTEXT_NAMESPACE "EnvQueryGenerator"

namespace UE::AI::EQS::Private
{
	FORCEINLINE bool DoesImplementBPFunction(const FName FuncName, const UObject& Ob, const UClass* StopAtClass)
	{
		const UFunction* Function = Ob.GetClass()->FindFunctionByName(FuncName);
		checkSlow(Function)
		return (Function->GetOuter() != StopAtClass);
	}
}

UEnvQueryGenerator_BlueprintBase::UEnvQueryGenerator_BlueprintBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	using namespace UE::AI::EQS::Private;
	Context = UEnvQueryContext_Querier::StaticClass();
	ItemType = UEnvQueryItemType_Actor::StaticClass();
	GeneratedItemType = UEnvQueryItemType_Actor::StaticClass();

	const UClass* StopAtClass = UEnvQueryGenerator_BlueprintBase::StaticClass();
	const bool bImplementsGenerateFromVectors = DoesImplementBPFunction(GET_FUNCTION_NAME_CHECKED(UEnvQueryGenerator_BlueprintBase, DoItemGeneration), *this, StopAtClass);
	const bool bImplementsGenerateFromActors = DoesImplementBPFunction(GET_FUNCTION_NAME_CHECKED(UEnvQueryGenerator_BlueprintBase, DoItemGenerationFromActors), *this, StopAtClass);
	if (bImplementsGenerateFromVectors)
	{
		CallMode = ECallMode::FromVectors;
	}
	else if (bImplementsGenerateFromActors)
	{
		CallMode = ECallMode::FromActors;
	}

	if (CallMode == ECallMode::Invalid && GetClass() != StopAtClass)
	{
		UE_LOGF(LogEQS, Error, "Blueprint Generator class %ls doesn't override DoItemGeneration or DoItemGenerationFromActors and can't generate anything.", *GetNameSafe(GetClass()));
	}
}

void UEnvQueryGenerator_BlueprintBase::PostInitProperties()
{
	Super::PostInitProperties();
	ItemType = GeneratedItemType;
}

UWorld* UEnvQueryGenerator_BlueprintBase::GetWorld() const
{
	return CachedQueryInstance ? CachedQueryInstance->World : NULL;
}

void UEnvQueryGenerator_BlueprintBase::GenerateItems(FEnvQueryInstance& QueryInstance) const
{
	CachedQueryInstance = &QueryInstance;
	if (CallMode == ECallMode::FromVectors)
	{
		TArray<FVector> ContextLocations;
		QueryInstance.PrepareContext(Context, ContextLocations);
		const_cast<UEnvQueryGenerator_BlueprintBase*>(this)->DoItemGeneration(ContextLocations);
		
	}
	else if (CallMode == ECallMode::FromActors)
	{
		TArray<AActor*> ContextActors;
		QueryInstance.PrepareContext(Context, ContextActors);;
		const_cast<UEnvQueryGenerator_BlueprintBase*>(this)->DoItemGenerationFromActors(ContextActors);
	}
	else
	{
		UE_LOGF(LogEQS, Warning, "Blueprint generator %ls with class %ls doesn't override DoItemGeneration or DoItemGenerationFromActors and can't generate anything.", *GetName(), *GetNameSafe(GetClass()));
	}
	CachedQueryInstance = NULL;
}

void UEnvQueryGenerator_BlueprintBase::AddGeneratedVector(FVector Vector) const
{
	check(CachedQueryInstance);
	if (ensure(ItemType->IsChildOf<UEnvQueryItemType_ActorBase>() == false))
	{
		CachedQueryInstance->AddItemData<UEnvQueryItemType_Point>(Vector);
	}
	else
	{
		UE_LOGF(LogEQS, Error, "Trying to generate a Vector item while generator %ls is configured to produce Actor items"
			, *GetName());
	}
}

void UEnvQueryGenerator_BlueprintBase::AddGeneratedActor(AActor* Actor) const
{
	check(CachedQueryInstance);
	if (ensure(ItemType->IsChildOf<UEnvQueryItemType_ActorBase>()))
	{
		CachedQueryInstance->AddItemData<UEnvQueryItemType_Actor>(Actor);
	}
	else
	{
		UE_LOGF(LogEQS, Error, "Trying to generate an Actor item while generator %ls is configured to produce Vector items. Will use Actor\'s location, but please update your BP code."
			, *GetName());
		if (Actor)
		{
			CachedQueryInstance->AddItemData<UEnvQueryItemType_Point>(Actor->GetActorLocation());
		}
	}
}

UObject* UEnvQueryGenerator_BlueprintBase::GetQuerier() const
{
	check(CachedQueryInstance);
	return CachedQueryInstance->Owner.Get();
}

FText UEnvQueryGenerator_BlueprintBase::GetDescriptionTitle() const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("DescriptionTitle"), Super::GetDescriptionTitle());
	Args.Add(TEXT("DescribeGeneratorAction"), GeneratorsActionDescription);
	Args.Add(TEXT("DescribeContext"), UEnvQueryTypes::DescribeContext(Context));

	return FText::Format(LOCTEXT("DescriptionBlueprintImplementedGenerator", "{DescriptionTitle}: {DescribeGeneratorAction} around {DescribeContext}"), Args);
}

FText UEnvQueryGenerator_BlueprintBase::GetDescriptionDetails() const
{
	return FText::FromString(TEXT("None"));
}

#undef LOCTEXT_NAMESPACE

