// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Generators/EnvQueryGenerator_Composite.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_Point.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryGenerator_Composite)

#define LOCTEXT_NAMESPACE "EnvQueryGenerator"

UEnvQueryGenerator_Composite::UEnvQueryGenerator_Composite(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	ItemType = UEnvQueryItemType_Point::StaticClass();
	bHasMatchingItemType = true;
}

void UEnvQueryGenerator_Composite::GenerateItems(FEnvQueryInstance& QueryInstance) const 
{
	if (bHasMatchingItemType)
	{
		for (int32 Idx = 0; Idx < Generators.Num(); Idx++)
		{
			if (Generators[Idx])
			{
				FScopeCycleCounterUObject GeneratorScope(Generators[Idx]);
				Generators[Idx]->GenerateItems(QueryInstance);
				ensureMsgf(!Generators[Idx]->IsCurrentlyRunningAsync(), TEXT("Composite Generators do not yet support asynchronous processing."));
			}
		}
	}
}

FText UEnvQueryGenerator_Composite::GetDescriptionTitle() const
{
	FText Desc = Super::GetDescriptionTitle();
	for (int32 Idx = 0; Idx < Generators.Num(); Idx++)
	{
		if (Generators[Idx])
		{
			Desc = FText::Format(LOCTEXT("DescTitleExtention", "{0}\n  {1}"), Desc, Generators[Idx]->GetDescriptionTitle());
		}
	}

	return Desc;
};

bool UEnvQueryGenerator_Composite::IsValidGenerator() const
{
	if (!Super::IsValidGenerator() || Generators.Num() == 0)
	{
		return false;
	}

	bool bValid = true;
	for (const UEnvQueryGenerator* Generator : Generators)
	{
		if (Generator == nullptr || !Generator->IsValidGenerator())
		{
			bValid = false;
			break;
		}
	}

	return bValid;
}

void UEnvQueryGenerator_Composite::VerifyItemTypes()
{
	TSubclassOf<UEnvQueryItemType> CommonItemType = nullptr;
	bHasMatchingItemType = true;

	if (bAllowDifferentItemTypes)
	{
		// ignore safety and force user specified item type
		// REQUIRES proper memory layout between forced type and ALL item types used by child generators
		// this is advanced option and will NOT be validated!

		CommonItemType = ForcedItemType;
		bHasMatchingItemType = (ForcedItemType != nullptr);
	}
	else
	{
		for (int32 Idx = 0; Idx < Generators.Num(); Idx++)
		{
			if (Generators[Idx])
			{
				checkf(!Generators[Idx]->CanRunAsync(), TEXT("Async Generators such as [%s] are not supported members of Composite Generators."), *Generators[Idx]->GetName())

				if (CommonItemType)
				{
					if (CommonItemType != Generators[Idx]->ItemType)
					{
						bHasMatchingItemType = false;
						break;
					}
				}
				else
				{
					CommonItemType = Generators[Idx]->ItemType;
				}
			}
		}
	}

	if (bHasMatchingItemType)
	{
		ItemType = CommonItemType;
	}
	else
	{
		// any type will do, generator is not allowed to create items anyway
		ItemType = UEnvQueryItemType_Point::StaticClass();
	}
}

#undef LOCTEXT_NAMESPACE

