// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Generators/EnvQueryGenerator_Cone.h"
#include "EnvironmentQuery/Contexts/EnvQueryContext_Querier.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryGenerator_Cone)

#define LOCTEXT_NAMESPACE "EnvQueryGenerator"

UEnvQueryGenerator_Cone::UEnvQueryGenerator_Cone(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer)
{
	CenterActor = UEnvQueryContext_Querier::StaticClass();
	AlignedPointsDistance.DefaultValue = 100.f;
	ConeDegrees.DefaultValue = 90.f;
	AngleStep.DefaultValue = 10.f;
	Range.DefaultValue = 1000.f;
	bIncludeContextLocation = false;
}

void UEnvQueryGenerator_Cone::BindDataToDataProviders(FEnvQueryInstance& QueryInstance) const
{
	//Bind the necessary data to data providers
	UObject* BindOwner = QueryInstance.Owner.Get();
	AlignedPointsDistance.BindData(BindOwner, QueryInstance.QueryID);
	ConeDegrees.BindData(BindOwner, QueryInstance.QueryID);
	AngleStep.BindData(BindOwner, QueryInstance.QueryID);
	Range.BindData(BindOwner, QueryInstance.QueryID);
}

void UEnvQueryGenerator_Cone::GenerateItems(FEnvQueryInstance& QueryInstance) const
{
	TArray<AActor*> CenterActors;
	QueryInstance.PrepareContext(CenterActor, CenterActors);

	if (CenterActors.Num() <= 0)
	{
		return;
	}

	BindDataToDataProviders(QueryInstance);

	const float ConeDegreesValue = FMath::Clamp(FMath::Abs(ConeDegrees.GetValue()), 0.f, 359.f);
	if (ConeDegreesValue == 0)
	{
		return;
	}

	//Get the values from each data provider
	const float AlignedPointDistanceValue = AlignedPointsDistance.GetValue();
	const float AngleStepValue = FMath::Clamp(AngleStep.GetValue(), 1.f, 359.f);
	const float GenerationRange = FMath::Clamp(Range.GetValue(), 0.f, MAX_flt);
	const int32 MaxPointsPerAngleValue = FMath::FloorToInt(GenerationRange / AlignedPointDistanceValue);

	TArray<FNavLocation> GeneratedItems;
	GeneratedItems.Reserve(CenterActors.Num() * FMath::CeilToInt(ConeDegreesValue / AngleStepValue) * MaxPointsPerAngleValue + 1);

	//Generate points for each actor
	for (int32 CenterIndex = 0; CenterIndex < CenterActors.Num(); CenterIndex++)
	{
		const FVector ForwardVector = CenterActors[CenterIndex]->GetActorForwardVector();
		const FVector ActorLocation = CenterActors[CenterIndex]->GetActorLocation();
		
		for (float Angle = -(ConeDegreesValue * 0.5f); Angle < (ConeDegreesValue * 0.5f); Angle += AngleStepValue)
		{
			const FVector AxisDirection = ForwardVector.RotateAngleAxis(Angle, FVector::UpVector);

			// skipping PointIndex == 0 as that's the context's location
			for (int32 PointIndex = 1; PointIndex <= MaxPointsPerAngleValue; PointIndex++)
			{
				const FVector GeneratedLocation = ActorLocation + (AxisDirection * PointIndex * AlignedPointDistanceValue);
				GeneratedItems.Add(FNavLocation(GeneratedLocation));
			}
		}

		if (bIncludeContextLocation)
		{
			GeneratedItems.Add(FNavLocation(ActorLocation));
		}
	}	

	ProjectAndFilterNavPoints(GeneratedItems, QueryInstance);
	StoreNavPoints(GeneratedItems, QueryInstance);
}

FText UEnvQueryGenerator_Cone::GetDescriptionTitle() const
{
	return FText::Format(LOCTEXT("ConeDescriptionGenerateAroundContext", "{0}: generate in front of {1}"),
		Super::GetDescriptionTitle(), UEnvQueryTypes::DescribeContext(CenterActor));
}

FText UEnvQueryGenerator_Cone::GetDescriptionDetails() const
{
	FText Desc = FText::Format(LOCTEXT("ConeDescription", "degrees: {0}, angle step: {1}"),
		FText::FromString(ConeDegrees.ToString()), FText::FromString(AngleStep.ToString()));

	FText ProjDesc = ProjectionData.ToText(FEnvTraceData::Brief);
	if (!ProjDesc.IsEmpty())
	{
		FFormatNamedArguments ProjArgs;
		ProjArgs.Add(TEXT("Description"), Desc);
		ProjArgs.Add(TEXT("ProjectionDescription"), ProjDesc);
		Desc = FText::Format(LOCTEXT("ConeDescriptionWithProjection", "{Description}, {ProjectionDescription}"), ProjArgs);
	}

	return Desc;
}

#undef LOCTEXT_NAMESPACE

