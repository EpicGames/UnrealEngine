// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Generators/EnvQueryGenerator_CurrentLocation.h"
#include "AI/Navigation/NavigationTypes.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_Point.h"
#include "EnvironmentQuery/Contexts/EnvQueryContext_Querier.h"
#include "UObject/Class.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryGenerator_CurrentLocation)

#define LOCTEXT_NAMESPACE "EnvQueryGenerator"

UEnvQueryGenerator_CurrentLocation::UEnvQueryGenerator_CurrentLocation(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer)
{
	QueryContext = UEnvQueryContext_Querier::StaticClass();
	ItemType = UEnvQueryItemType_Point::StaticClass();
	ProjectionData.TraceMode = EEnvQueryTrace::None; // Set to none by default to preserve behavior to before the node could project location.
}

void UEnvQueryGenerator_CurrentLocation::GenerateItems(FEnvQueryInstance& QueryInstance) const
{
	TArray<FVector> ContextLocations;
	QueryInstance.PrepareContext(QueryContext, ContextLocations);

	TArray<FNavLocation> NavLocations;
	NavLocations.Reserve(ContextLocations.Num());
	for (const FVector& Location : ContextLocations)
	{
		NavLocations.Emplace(Location);
	}

	ProjectAndFilterNavPoints(NavLocations, QueryInstance);
	StoreNavPoints(NavLocations, QueryInstance);
}

FText UEnvQueryGenerator_CurrentLocation::GetDescriptionTitle() const
{
	if (ProjectionData.TraceMode != EEnvQueryTrace::None)
	{
		return FText::Format(LOCTEXT("CurrentLocationProjectedOn", "Current Location of {0} projected on {1}"), UEnvQueryTypes::DescribeContext(QueryContext), UEnum::GetDisplayValueAsText(ProjectionData.TraceMode));
	}
	else
	{
		return FText::Format(LOCTEXT("CurrentLocationOn", "Current Location of {0}"), UEnvQueryTypes::DescribeContext(QueryContext));
	}
};

FText UEnvQueryGenerator_CurrentLocation::GetDescriptionDetails() const
{
	return ProjectionData.ToText(FEnvTraceData::EDescriptionMode::Brief);
}

#undef LOCTEXT_NAMESPACE

