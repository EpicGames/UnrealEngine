// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Generators/EnvQueryGenerator_Donut.h"
#include "EnvironmentQuery/Contexts/EnvQueryContext_Querier.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryGenerator_Donut)

#define LOCTEXT_NAMESPACE "EnvQueryGenerator"

UEnvQueryGenerator_Donut::UEnvQueryGenerator_Donut(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	Center = UEnvQueryContext_Querier::StaticClass();
	InnerRadius.DefaultValue = 300.0f;
	OuterRadius.DefaultValue = 1000.0f;
	PointOnRingSpacingMethod = EEnvQueryPointSpacingMethod::ByNumberOfPoints;
	SpaceBetweenPoints.DefaultValue = 50.f;
	NumberOfRings.DefaultValue = 3;
	PointsPerRing.DefaultValue = 8;
	ArcDirection.DirMode = EEnvDirection::TwoPoints;
	ArcDirection.LineFrom = UEnvQueryContext_Querier::StaticClass();
	ArcDirection.Rotation = UEnvQueryContext_Querier::StaticClass();
	ArcAngle.DefaultValue = 360.f;
	bUseSpiralPattern = false;

	ProjectionData.TraceMode = EEnvQueryTrace::None;
}

FVector::FReal UEnvQueryGenerator_Donut::GetArcBisectorAngle(FEnvQueryInstance& QueryInstance) const
{
	FRotator::FReal BisectAngle = 0.;

	if (bDefineArc)
	{
		if (ArcDirection.DirMode == EEnvDirection::TwoPoints)
		{
			TArray<FVector> Start;
			TArray<FVector> End;
			QueryInstance.PrepareContext(ArcDirection.LineFrom, Start);
			QueryInstance.PrepareContext(ArcDirection.LineTo, End);

			if (Start.Num() > 0 && End.Num() > 0)
			{
				const FVector LineDir = (End[0] - Start[0]).GetSafeNormal();
				const FRotator LineRot = LineDir.Rotation();
				BisectAngle = LineRot.Yaw;
			}
		}
		else
		{
			TArray<FRotator> Rot;
			QueryInstance.PrepareContext(ArcDirection.Rotation, Rot);

			if (Rot.Num() > 0)
			{
				BisectAngle = Rot[0].Yaw;
			}
		}
	}

	return BisectAngle;
}

bool UEnvQueryGenerator_Donut::IsAngleAllowed(FVector::FReal TestAngleRad, FVector::FReal BisectAngleDeg, FVector::FReal AngleRangeDeg, bool bConstrainAngle) const
{
	if (bConstrainAngle)
	{
		const FVector::FReal TestAngleDeg = FMath::RadiansToDegrees(TestAngleRad);
		const FVector::FReal AngleDelta = FRotator::NormalizeAxis(TestAngleDeg - BisectAngleDeg);
		return (FMath::Abs(AngleDelta) - 0.01) < AngleRangeDeg;
	}

	return true;	
}

void UEnvQueryGenerator_Donut::GenerateItems(FEnvQueryInstance& QueryInstance) const
{
	TArray<FVector> CenterPoints;
	QueryInstance.PrepareContext(Center, CenterPoints);

	if (CenterPoints.Num() <= 0)
	{
		return;
	}

	TArray<FNavLocation> Points;
	GenerateDonutNavPoints(CenterPoints, Points, QueryInstance);

	ProjectAndFilterNavPoints(Points, QueryInstance);
	StoreNavPoints(Points, QueryInstance);
}

bool UEnvQueryGenerator_Donut::GenerateDonutNavPoints(const TArray<FVector>& CenterPoints, TArray<FNavLocation>& Points, FEnvQueryInstance& QueryInstance) const
{
	UObject* BindOwner = QueryInstance.Owner.Get();
	InnerRadius.BindData(BindOwner, QueryInstance.QueryID);
	OuterRadius.BindData(BindOwner, QueryInstance.QueryID);
	NumberOfRings.BindData(BindOwner, QueryInstance.QueryID);
	PointsPerRing.BindData(BindOwner, QueryInstance.QueryID);
	SpaceBetweenPoints.BindData(BindOwner, QueryInstance.QueryID);
	ArcAngle.BindData(BindOwner, QueryInstance.QueryID);

	const FVector::FReal ArcAngleValue = ArcAngle.GetValue();
	const FVector::FReal InnerRadiusValue = InnerRadius.GetValue();
	const FVector::FReal OuterRadiusValue = OuterRadius.GetValue();
	const int32 NumRings = NumberOfRings.GetValue();

	if ((InnerRadiusValue < 0.) || (OuterRadiusValue <= 0.) ||
		(InnerRadiusValue > OuterRadiusValue) || (NumRings < 1))
	{
		return false;
	}

	const FVector::FReal ArcBisectDeg = GetArcBisectorAngle(QueryInstance);
	const FVector::FReal ArcAngleDeg = FMath::Clamp(ArcAngleValue, 0., 360.);
	const FVector::FReal RadiusDelta = NumRings > 1 ? (OuterRadiusValue - InnerRadiusValue) / (NumRings - 1) : 0.;

	// Handle the two spacing methods differently
	if (PointOnRingSpacingMethod == EEnvQueryPointSpacingMethod::ByNumberOfPoints)
	{
		// ByNumberOfPoints: All rings have the same point count
		const int32 NumPoints = PointsPerRing.GetValue();
		if (NumPoints < 1)
		{
			return false;
		}

		const FVector::FReal AngleDelta = 2. * UE_DOUBLE_PI / NumPoints;
		FVector::FReal SectionAngle = FMath::DegreesToRadians(ArcBisectDeg);

		Points.Reserve(NumPoints * NumRings * CenterPoints.Num());

		if (!bUseSpiralPattern)
		{
			for (int32 SectionIdx = 0; SectionIdx < NumPoints; SectionIdx++, SectionAngle += AngleDelta)
			{
				if (IsAngleAllowed(SectionAngle, ArcBisectDeg, ArcAngleDeg, bDefineArc))
				{
					const FVector::FReal SinValue = FMath::Sin(SectionAngle);
					const FVector::FReal CosValue = FMath::Cos(SectionAngle);

					FVector::FReal RingRadius = InnerRadiusValue;
					for (int32 RingIdx = 0; RingIdx < NumRings; RingIdx++, RingRadius += RadiusDelta)
					{
						const FVector RingPos(RingRadius * CosValue, RingRadius * SinValue, 0.);
						for (int32 ContextIdx = 0; ContextIdx < CenterPoints.Num(); ContextIdx++)
						{
							const FNavLocation PointPos = FNavLocation(CenterPoints[ContextIdx] + RingPos);
							Points.Add(PointPos);
						}
					}
				}
			}
		}
		else
		{	// In order to spiral, we need to change the angle for each ring as well as each spoke.  By changing it as a
			// fraction of the SectionAngle, we guarantee that the offset won't cross to the starting point of the next
			// spoke.
			const FVector::FReal RingAngleDelta = AngleDelta / NumRings;
			FVector::FReal RingRadius = InnerRadiusValue;
			FVector::FReal CurrentRingAngleDelta = 0.f;

			// So, start with ring angle and then add section angle.
			for (int32 RingIdx = 0; RingIdx < NumRings; RingIdx++, RingRadius += RadiusDelta, CurrentRingAngleDelta += RingAngleDelta)
			{
				for (int32 SectionIdx = 0; SectionIdx < NumPoints; SectionIdx++, SectionAngle += AngleDelta)
				{
					FVector::FReal RingSectionAngle = CurrentRingAngleDelta + SectionAngle;
					if (IsAngleAllowed(RingSectionAngle, ArcBisectDeg, ArcAngleDeg, bDefineArc))
					{
						const FVector::FReal SinValue = FMath::Sin(RingSectionAngle);
						const FVector::FReal CosValue = FMath::Cos(RingSectionAngle);

						const FVector RingPos(RingRadius * CosValue, RingRadius * SinValue, 0.);
						for (int32 ContextIdx = 0; ContextIdx < CenterPoints.Num(); ContextIdx++)
						{
							const FNavLocation PointPos = FNavLocation(CenterPoints[ContextIdx] + RingPos);
							Points.Add(PointPos);
						}
					}
				}
			}
		}
	}
	else // BySpaceBetween
	{
		// BySpaceBetween: Each ring has different point count based on its circumference
		const FVector::FReal SpaceBetweenPointsValue = SpaceBetweenPoints.GetValue();
		if (SpaceBetweenPointsValue <= 0.0)
		{
			return false;
		}

		FVector::FReal RingRadius = InnerRadiusValue;

		// Reserve approximate space (use outer ring's count as upper bound)
		const FVector::FReal OuterArcLength = 2.0 * UE_DOUBLE_PI * OuterRadiusValue * ArcAngleDeg / 360.0;
		const int32 ApproxMaxPoints = FMath::CeilToInt(OuterArcLength / SpaceBetweenPointsValue) + 1;
		Points.Reserve(ApproxMaxPoints * NumRings * CenterPoints.Num());

		FVector::FReal SectionAngle = FMath::DegreesToRadians(ArcBisectDeg);
		// Generate points for each ring independently
		for (int32 RingIdx = 0; RingIdx < NumRings; RingIdx++, RingRadius += RadiusDelta)
		{
			// Calculate point count for this specific ring
			const FVector::FReal RingLength = 2.0 * UE_DOUBLE_PI * RingRadius;
			const int32 RingPointCount = FMath::CeilToInt(RingLength / SpaceBetweenPointsValue) + 1;

			// Generate points evenly distributed around this ring
			const FVector::FReal RingAngleDelta = 2.0 * UE_DOUBLE_PI / RingPointCount;

			for (int32 PointIdx = 0; PointIdx < RingPointCount; PointIdx++, SectionAngle += RingAngleDelta)
			{
				if (IsAngleAllowed(SectionAngle, ArcBisectDeg, ArcAngleDeg, bDefineArc))
				{
					const FVector::FReal SinValue = FMath::Sin(SectionAngle);
					const FVector::FReal CosValue = FMath::Cos(SectionAngle);
					const FVector RingPos(RingRadius * CosValue, RingRadius * SinValue, 0.);

					for (int32 ContextIdx = 0; ContextIdx < CenterPoints.Num(); ContextIdx++)
					{
						const FNavLocation PointPos = FNavLocation(CenterPoints[ContextIdx] + RingPos);
						Points.Add(PointPos);
					}
				}
			}

			// for spiral pattern we keep the section angle from one ring to the other. Else we just reset it
			SectionAngle = bUseSpiralPattern ? (SectionAngle + RingAngleDelta) : FMath::DegreesToRadians(ArcBisectDeg);
		}
	}

	return Points.Num() > 0;
}

FText UEnvQueryGenerator_Donut::GetDescriptionTitle() const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("DescriptionTitle"), Super::GetDescriptionTitle());
	Args.Add(TEXT("DescribeContext"), UEnvQueryTypes::DescribeContext(Center));

	return FText::Format(LOCTEXT("DescriptionGenerateDonutAroundContext", "{DescriptionTitle}: generate items around {DescribeContext}"), Args);
}

FText UEnvQueryGenerator_Donut::GetDescriptionDetails() const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("InnerRadius"), FText::FromString(InnerRadius.ToString()));
	Args.Add(TEXT("OuterRadius"), FText::FromString(OuterRadius.ToString()));
	Args.Add(TEXT("NumRings"), FText::FromString(NumberOfRings.ToString()));

	FText Desc;
	if (PointOnRingSpacingMethod == EEnvQueryPointSpacingMethod::ByNumberOfPoints)
	{
		Args.Add(TEXT("NumPerRing"), FText::FromString(PointsPerRing.ToString()));
		Desc = FText::Format(LOCTEXT("DonutDescriptionByCount", "radius: {InnerRadius} to {OuterRadius}\nrings: {NumRings}, points per ring: {NumPerRing}"), Args);
	}
	else
	{
		Args.Add(TEXT("Spacing"), FText::FromString(SpaceBetweenPoints.ToString()));
		Desc = FText::Format(LOCTEXT("DonutDescriptionBySpacing", "radius: {InnerRadius} to {OuterRadius}\nrings: {NumRings}, spacing: {Spacing} (points vary per ring)"), Args);
	}

	if (bDefineArc)
	{
		FFormatNamedArguments ArcArgs;
		ArcArgs.Add(TEXT("Description"), Desc);
		ArcArgs.Add(TEXT("AngleValue"), ArcAngle.DefaultValue);
		ArcArgs.Add(TEXT("ArcDirection"), ArcDirection.ToText());
		Desc = FText::Format(LOCTEXT("DescriptionWithArc", "{Description}\nLimit to {AngleValue} angle both sides on {ArcDirection}"), ArcArgs);
	}

	FText ProjDesc = ProjectionData.ToText(FEnvTraceData::Brief);
	if (!ProjDesc.IsEmpty())
	{
		FFormatNamedArguments ProjArgs;
		ProjArgs.Add(TEXT("Description"), Desc);
		ProjArgs.Add(TEXT("ProjectionDescription"), ProjDesc);
		Desc = FText::Format(LOCTEXT("OnCircle_DescriptionWithProjection", "{Description}, {ProjectionDescription}"), ProjArgs);
	}

	return Desc;
}

#if WITH_EDITOR

void UEnvQueryGenerator_Donut::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property != NULL)
	{
		const FName PropName = PropertyChangedEvent.MemberProperty->GetFName();
		if (PropName == GET_MEMBER_NAME_CHECKED(UEnvQueryGenerator_Donut, ArcAngle))
		{
			ArcAngle.DefaultValue = FMath::Clamp(ArcAngle.DefaultValue, 0.0f, 360.f);
			bDefineArc = (ArcAngle.DefaultValue < 360.f) && (ArcAngle.DefaultValue > 0.f);
		}
		else if (PropName == GET_MEMBER_NAME_CHECKED(UEnvQueryGenerator_Donut, NumberOfRings))
		{
			NumberOfRings.DefaultValue = FMath::Max(1, NumberOfRings.DefaultValue);
		}
		else if (PropName == GET_MEMBER_NAME_CHECKED(UEnvQueryGenerator_Donut, PointsPerRing))
		{
			PointsPerRing.DefaultValue = FMath::Max(1, PointsPerRing.DefaultValue);
		}
		else if (PropName == GET_MEMBER_NAME_CHECKED(UEnvQueryGenerator_Donut, SpaceBetweenPoints))
		{
			SpaceBetweenPoints.DefaultValue = FMath::Max(1.0f, SpaceBetweenPoints.DefaultValue);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE

