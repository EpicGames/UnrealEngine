// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Generators/EnvQueryGenerator_OnCircle.h"
#include "GameFramework/Actor.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_VectorBase.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_ActorBase.h"
#include "VisualLogger/VisualLogger.h"
#include "EnvironmentQuery/Contexts/EnvQueryContext_Querier.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_Point.h"
#include "EnvironmentQuery/EnvQueryTraceHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryGenerator_OnCircle)

#define LOCTEXT_NAMESPACE "EnvQueryGenerator"

//----------------------------------------------------------------------//
// UEnvQueryGenerator_OnCircle
//----------------------------------------------------------------------//
UEnvQueryGenerator_OnCircle::UEnvQueryGenerator_OnCircle(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer)
{
	CircleCenter = UEnvQueryContext_Querier::StaticClass();
	ItemType = UEnvQueryItemType_Point::StaticClass();
	CircleRadius.DefaultValue = 1000.0f;
	SpaceBetween.DefaultValue = 50.0f;
	NumberOfPoints.DefaultValue = 8;
	PointOnCircleSpacingMethod = EEnvQueryPointSpacingMethod::BySpaceBetween;
	ArcDirection.DirMode = EEnvDirection::TwoPoints;
	ArcDirection.LineFrom = UEnvQueryContext_Querier::StaticClass();
	ArcDirection.Rotation = UEnvQueryContext_Querier::StaticClass();
	ArcAngle.DefaultValue = 360.f;
	bIgnoreAnyContextActorsWhenGeneratingCircle = false;

	TraceData.bCanProjectDown = false;
	TraceData.bCanDisableTrace = true;
	TraceData.TraceMode = EEnvQueryTrace::Navigation;

	ProjectionData.TraceMode = EEnvQueryTrace::None;
}

void UEnvQueryGenerator_OnCircle::PostLoad()
{
	Super::PostLoad();

	if (ArcAngle.DefaultValue <= 0)
	{
		ArcAngle.DefaultValue = 360.f;
	}

	TraceData.OnPostLoad();
}

FVector UEnvQueryGenerator_OnCircle::CalcDirection(FEnvQueryInstance& QueryInstance) const
{
	UObject* Querier = QueryInstance.Owner.Get();
	check(Querier != NULL);

	// By default (if no defined arc is needed), don't rotate based on querier!  Instead, use a stable rotation
	// so the points on the circle don't rotate!
	FVector Direction(FVector::ForwardVector);
	if (bDefineArc)
	{
		// Try to use the Querier rotation for arc direction if it has one (by default).  If not, we're already
		// initialized (above) to a stable rotational direction.
		AActor* QuerierActor = Cast<AActor>(Querier);
		if (QuerierActor != nullptr)
		{
			Direction = QuerierActor->GetActorForwardVector();
		}

		if (ArcDirection.DirMode == EEnvDirection::TwoPoints)
		{
			TArray<FVector> Start;
			TArray<FVector> End;
			QueryInstance.PrepareContext(ArcDirection.LineFrom, Start);
			QueryInstance.PrepareContext(ArcDirection.LineTo, End);

			if (Start.Num() > 0 && End.Num() > 0)
			{
				Direction = (End[0] - Start[0]).GetSafeNormal();
			}
			else
			{
				if (QuerierActor != NULL)
				{
					UE_VLOG(Querier, LogEQS, Warning,
						TEXT("UEnvQueryGenerator_OnCircle::CalcDirection failed to calc direction in %s. Using querier facing."),
						*QueryInstance.QueryName);
				}
				else
				{
					UE_VLOG(Querier, LogEQS, Warning,
						TEXT("UEnvQueryGenerator_OnCircle::CalcDirection failed to calc direction in %s.  Querier is not an actor, so we'll just use the world 'forward' direction."),
						*QueryInstance.QueryName);
				}
			}
		}
		else
		{
			TArray<FRotator> Rot;
			QueryInstance.PrepareContext(ArcDirection.Rotation, Rot);

			if (Rot.Num() > 0)
			{
				Direction = Rot[0].Vector();
			}
			else
			{
				if (QuerierActor != NULL)
				{
					UE_VLOG(Querier, LogEQS, Warning, TEXT("UEnvQueryGenerator_OnCircle::CalcDirection failed to calc direction in %s. Using querier facing."), *QueryInstance.QueryName);
				}
				else
				{
					UE_VLOG(Querier, LogEQS, Warning,
						TEXT("UEnvQueryGenerator_OnCircle::CalcDirection failed to calc direction in %s.  Querier is not an actor, so we'll just use the world 'forward' direction."),
						*QueryInstance.QueryName);
				}
			}
		}
	}

	return Direction;
}

void UEnvQueryGenerator_OnCircle::GenerateItems(FEnvQueryInstance& QueryInstance) const
{
	UObject* QueryOwner = QueryInstance.Owner.Get();
	if (QueryOwner == nullptr)
	{
		return;
	}

	CircleRadius.BindData(QueryOwner, QueryInstance.QueryID);
	SpaceBetween.BindData(QueryOwner, QueryInstance.QueryID);
	ArcAngle.BindData(QueryOwner, QueryInstance.QueryID);
	CircleCenterZOffset.BindData(QueryOwner, QueryInstance.QueryID);
	NumberOfPoints.BindData(QueryOwner, QueryInstance.QueryID);
	ArcDirectionOffsetDegrees.BindData(QueryOwner, QueryInstance.QueryID);

	const float AngleDegree = ArcAngle.GetValue();
	const float RadiusValue = CircleRadius.GetValue();
	const float ItemSpace = SpaceBetween.GetValue();
	const int32 NumPoints = NumberOfPoints.GetValue();
	const float DirectionOffsetDegrees = ArcDirectionOffsetDegrees.GetValue();

	if ((AngleDegree <= 0.f) || 
		(AngleDegree > 360.f) || 
		(RadiusValue <= 0.f) || 
		((PointOnCircleSpacingMethod == EEnvQueryPointSpacingMethod::BySpaceBetween) && (ItemSpace <= 0.f)) ||
		((PointOnCircleSpacingMethod == EEnvQueryPointSpacingMethod::ByNumberOfPoints) && (NumPoints <= 0)))
	{
		return;
	}

	// first generate points on a circle
	const float CircumferenceLength = 2.f * PI * RadiusValue;
	const float ArcAnglePercentage = AngleDegree / 360.f;
	const float ArcLength = CircumferenceLength * ArcAnglePercentage;

	int32 StepsCount;
	int32 NumStepsBetween;
	switch (PointOnCircleSpacingMethod)
	{
		case EEnvQueryPointSpacingMethod::BySpaceBetween:
		{
			StepsCount = FMath::CeilToInt(ArcLength / ItemSpace) + 1;
			NumStepsBetween = StepsCount - 1;
			break;
		}

		case EEnvQueryPointSpacingMethod::ByNumberOfPoints:
		{
			StepsCount = NumPoints;
			NumStepsBetween = StepsCount;
			break;
		}

		default:
		{
			ensure(false);
			return;
		}
	}

	const float AngleStep = AngleDegree / NumStepsBetween;

	FVector StartDirection = CalcDirection(QueryInstance);
	StartDirection = StartDirection.RotateAngleAxis(DirectionOffsetDegrees - AngleDegree/2, FVector::UpVector) * RadiusValue;
	
	// gather context with raw data, so it can be used by derived generators
	FEnvQueryContextData ContextData;
	const bool bSuccess = QueryInstance.PrepareContext(CircleCenter, ContextData);

	if (bSuccess && ContextData.ValueType && ContextData.ValueType->IsChildOf(UEnvQueryItemType_VectorBase::StaticClass()))
	{
		UEnvQueryItemType_VectorBase* DefTypeOb = (UEnvQueryItemType_VectorBase*)ContextData.ValueType->GetDefaultObject();
		const uint16 DefTypeValueSize = DefTypeOb->GetValueSize();

		TArray<AActor*> ContextActorsToIgnoreWhenGeneratingCircle;
		if (bIgnoreAnyContextActorsWhenGeneratingCircle)
		{
			UEnvQueryItemType_ActorBase* DefEnvQueryActorBaseItemTypeOb = Cast<UEnvQueryItemType_ActorBase>(ContextData.ValueType->GetDefaultObject());
			if (DefEnvQueryActorBaseItemTypeOb != nullptr)
			{
				uint8* ActorRawData = ContextData.RawData.GetData();

				for (int32 ValueIndex = 0; ValueIndex < ContextData.NumValues; ValueIndex++)
				{
					AActor* Actor = DefEnvQueryActorBaseItemTypeOb->GetActor(ActorRawData);
					if (Actor)
					{
						ContextActorsToIgnoreWhenGeneratingCircle.Add(Actor);
					}
					ActorRawData += DefTypeValueSize;
				}
			}
		}

		uint8* RawData = ContextData.RawData.GetData();
		const FVector CircleCenterOffset = FVector(0.f, 0.f, CircleCenterZOffset.GetValue());

		for (int32 ValueIndex = 0; ValueIndex < ContextData.NumValues; ValueIndex++)
		{
			const FVector ContextItemLocation = DefTypeOb->GetItemLocation(RawData) + CircleCenterOffset;
			GenerateItemsForCircle(RawData, DefTypeOb, ContextItemLocation, StartDirection, ContextActorsToIgnoreWhenGeneratingCircle, StepsCount, AngleStep, QueryInstance);

			RawData += DefTypeValueSize;
		}
	}
}

void UEnvQueryGenerator_OnCircle::GenerateItemsForCircle(uint8* ContextRawData, UEnvQueryItemType* ContextItemType,
	const FVector& CenterLocation, const FVector& StartDirection, 
	const TArray<AActor*>& IgnoredActors,
	int32 StepsCount, float AngleStep, FEnvQueryInstance& OutQueryInstance) const
{
	TArray<FNavLocation> ItemCandidates;
	ItemCandidates.AddZeroed(StepsCount);

	for (int32 Step = 0; Step < StepsCount; ++Step)
	{
		ItemCandidates[Step] = FNavLocation(CenterLocation + StartDirection.RotateAngleAxis(AngleStep*Step, FVector::UpVector));
	}

	switch (TraceData.TraceMode)
	{
		case EEnvQueryTrace::Navigation:
		{
			ANavigationData* NavData = const_cast<ANavigationData*>(FEQSHelpers::FindNavigationDataForQuery(OutQueryInstance));
			const UObject* Querier = OutQueryInstance.Owner.Get();
			if (NavData && Querier)
			{
				FEQSHelpers::RunNavRaycasts(*NavData, *Querier, TraceData, CenterLocation, ItemCandidates);
			}
			break;
		}

		case EEnvQueryTrace::GeometryByChannel:
		case EEnvQueryTrace::GeometryByProfile:
		{
			FEQSHelpers::RunPhysRaycasts(OutQueryInstance.World, TraceData, CenterLocation, ItemCandidates, IgnoredActors);
			break;
		}

		case EEnvQueryTrace::NavigationOverLedges:
		{
			ANavigationData* NavData = const_cast<ANavigationData*>(FEQSHelpers::FindNavigationDataForQuery(OutQueryInstance));
			const UObject* Querier = OutQueryInstance.Owner.Get();
			if (NavData && Querier)
			{
				FEQSHelpers::RunRaycastsOnNavHitOnlyWalls(*NavData, *Querier, TraceData, CenterLocation, ItemCandidates, IgnoredActors);
			}
		}
			break;

		case EEnvQueryTrace::None:
		{
			// Just accept the ItemCandidates as they already are (points on a circle), without using navigation OR collision.
			break;
		}

		default:
		{
			UE_VLOG(Cast<AActor>(OutQueryInstance.Owner.Get()), LogEQS, Warning, TEXT("UEnvQueryGenerator_OnCircle::CalcDirection has invalid value for TraceData.TraceMode.  Query: %s"), *OutQueryInstance.QueryName);
			break;
		}
	}

	ProjectAndFilterNavPoints(ItemCandidates, OutQueryInstance);
	AddItemDataForCircle(ContextRawData, ContextItemType, ItemCandidates, OutQueryInstance);
}

void UEnvQueryGenerator_OnCircle::AddItemDataForCircle(uint8* ContextRawData, UEnvQueryItemType* ContextItemType,
	const TArray<FNavLocation>& Locations, FEnvQueryInstance& OutQueryInstance) const
{
	StoreNavPoints(Locations, OutQueryInstance);
}

FText UEnvQueryGenerator_OnCircle::GetDescriptionTitle() const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("DescriptionTitle"), Super::GetDescriptionTitle());
	Args.Add(TEXT("DescribeContext"), UEnvQueryTypes::DescribeContext(CircleCenter));

	return FText::Format(LOCTEXT("DescriptionGenerateCircleAroundContext", "{DescriptionTitle}: generate items on circle around {DescribeContext}"), Args);
}

FText UEnvQueryGenerator_OnCircle::GetDescriptionDetails() const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("Radius"), FText::FromString(CircleRadius.ToString()));
	Args.Add(TEXT("ItemSpacing"), FText::FromString(SpaceBetween.ToString()));
	Args.Add(TEXT("NumberOfPoints"), FText::FromString(NumberOfPoints.ToString()));
	Args.Add(TEXT("TraceData"), TraceData.ToText(FEnvTraceData::Detailed));

	FText Desc = PointOnCircleSpacingMethod == EEnvQueryPointSpacingMethod::BySpaceBetween 
		? FText::Format(LOCTEXT("OnCircleDescriptionSpan", "radius: {Radius}, item span: {ItemSpacing}\n{TraceData}"), Args)
		: FText::Format(LOCTEXT("OnCircleDescriptionNumItems", "radius: {Radius}, number of items to generate: {NumberOfPoints}\n{TraceData}"), Args);

	if (bDefineArc)
	{
		FFormatNamedArguments ArcArgs;
		ArcArgs.Add(TEXT("Description"), Desc);
		ArcArgs.Add(TEXT("AngleValue"), FText::FromString(ArcAngle.ToString()));
		ArcArgs.Add(TEXT("ArcDirection"), ArcDirection.ToText());
		Desc = FText::Format(LOCTEXT("DescriptionWithArc", "{Description}\nLimit to {AngleValue} angle both sides on {ArcDirection}"), ArcArgs);
	}

	FText ProjDesc = ProjectionData.ToText(FEnvTraceData::Brief);
	if (!ProjDesc.IsEmpty())
	{
		FFormatNamedArguments ProjArgs;
		ProjArgs.Add(TEXT("Description"), Desc);
		ProjArgs.Add(TEXT("ProjectionDescription"), ProjDesc);
		Desc = FText::Format(LOCTEXT("OnCircle_DescriptionWithProjection", "{Description}, {ProjectionDescription}"), ProjArgs);
	}

	return Desc;
}

#if WITH_EDITOR

void UEnvQueryGenerator_OnCircle::PostEditChangeProperty( FPropertyChangedEvent& PropertyChangedEvent) 
{
	if (PropertyChangedEvent.Property != NULL)
	{
		const FName PropName = PropertyChangedEvent.MemberProperty->GetFName();
		if (PropName == GET_MEMBER_NAME_CHECKED(UEnvQueryGenerator_OnCircle, ArcAngle))
		{
			ArcAngle.DefaultValue = FMath::Clamp(ArcAngle.DefaultValue, 0.0f, 360.f);
			bDefineArc = (ArcAngle.DefaultValue < 360.f) && (ArcAngle.DefaultValue > 0.f);
		}
		else if (PropName == GET_MEMBER_NAME_CHECKED(UEnvQueryGenerator_OnCircle, CircleRadius))
		{
			if (CircleRadius.DefaultValue <= 0.f)
			{
				CircleRadius.DefaultValue = 100.f;
			}
		}
		else if (PropName == GET_MEMBER_NAME_CHECKED(UEnvQueryGenerator_OnCircle, SpaceBetween))
		{
			SpaceBetween.DefaultValue = FMath::Max(1.0f, SpaceBetween.DefaultValue);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE

