// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Generators/EnvQueryGenerator_PathingGrid.h"
#include "NavMesh/RecastNavMesh.h"
#include "NavFilters/NavigationQueryFilter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryGenerator_PathingGrid)

#define LOCTEXT_NAMESPACE "EnvQueryGenerator"

UEnvQueryGenerator_PathingGrid::UEnvQueryGenerator_PathingGrid(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	ProjectionData.SetNavmeshOnly();
	GridSize.DefaultValue = 1000.0f;
	SpaceBetween.DefaultValue = 100.0f;
	PathToItem.DefaultValue = true;
	ScanRangeMultiplier.DefaultValue = 1.5f;
}

#if WITH_RECAST
namespace PathGridHelpers
{
	static bool HasPath(const FRecastDebugPathfindingData& NodePool, const NavNodeRef& NodeRef)
	{
		FRecastDebugPathfindingNode SearchKey(NodeRef);
		const FRecastDebugPathfindingNode* MyNode = NodePool.Nodes.Find(SearchKey);
		return MyNode != nullptr;
	}
}
#endif

void UEnvQueryGenerator_PathingGrid::ProjectAndFilterNavPoints(TArray<FNavLocation>& Points, FEnvQueryInstance& QueryInstance) const
{
	Super::ProjectAndFilterNavPoints(Points, QueryInstance);
#if WITH_RECAST
	UObject* DataOwner = QueryInstance.Owner.Get();
	PathToItem.BindData(DataOwner, QueryInstance.QueryID);
	ScanRangeMultiplier.BindData(DataOwner, QueryInstance.QueryID);

	bool bPathToItem = PathToItem.GetValue();
	float RangeMultiplierValue = ScanRangeMultiplier.GetValue();

	const ANavigationData* QueryNavData = FEQSHelpers::FindNavigationDataForQuery(QueryInstance);
	const ARecastNavMesh* NavMeshData = Cast<const ARecastNavMesh>(QueryNavData);
	if (NavMeshData == nullptr || DataOwner == nullptr)
	{
		return;
	}

	TArray<FVector> ContextLocations;
	QueryInstance.PrepareContext(GenerateAround, ContextLocations);

	FSharedConstNavQueryFilter NavQueryFilter = NavigationFilter ? UNavigationQueryFilter::GetQueryFilter(*NavMeshData, DataOwner, NavigationFilter) : NavMeshData->GetDefaultQueryFilter();
	if (!NavQueryFilter.IsValid())
	{
		UE_LOGF(LogEQS, Error, "%ls (%d:%ls) can't obtain navigation filter! NavData:%ls FilterClass:%ls",
			*QueryInstance.QueryName, QueryInstance.OptionIndex, *OptionName,
			*GetNameSafe(NavMeshData), *GetNameSafe(NavigationFilter.Get()));
		return;
	}

	FSharedNavQueryFilter NavigationFilterCopy = NavQueryFilter->GetCopy();
	if (NavigationFilterCopy.IsValid())
	{
		TArray<NavNodeRef> Polys;
		TArray<FNavLocation> HitLocations;
		const FVector ProjectionExtent(ProjectionData.ExtentX, ProjectionData.ExtentX, (ProjectionData.ProjectDown + ProjectionData.ProjectUp) / 2);

		for (int32 ContextIdx = 0; ContextIdx < ContextLocations.Num() && Points.Num(); ContextIdx++)
		{
			FVector::FReal CollectDistanceSq = 0.0f;
			for (int32 Idx = 0; Idx < Points.Num(); Idx++)
			{
				const FVector::FReal TestDistanceSq = FVector::DistSquared(Points[Idx].Location, ContextLocations[ContextIdx]);
				CollectDistanceSq = FMath::Max(CollectDistanceSq, TestDistanceSq);
			}

			const FVector::FReal MaxPathDistance = FMath::Sqrt(CollectDistanceSq) * RangeMultiplierValue;

			Polys.Reset();

			FRecastDebugPathfindingData NodePoolData;
			NodePoolData.Flags = ERecastDebugPathfindingFlags::Basic;

			NavMeshData->GetPolysWithinPathingDistance(ContextLocations[ContextIdx], MaxPathDistance, Polys, NavigationFilterCopy, nullptr, &NodePoolData);

			for (int32 Idx = Points.Num() - 1; Idx >= 0; Idx--)
			{
				bool bHasPath = PathGridHelpers::HasPath(NodePoolData, Points[Idx].NodeRef);
				if (!bHasPath && Points[Idx].NodeRef != INVALID_NAVNODEREF)
				{
					// try projecting it again, maybe it will match valid poly on different height
					HitLocations.Reset();
					FVector TestPt(Points[Idx].Location.X, Points[Idx].Location.Y, ContextLocations[ContextIdx].Z);

					NavMeshData->ProjectPointMulti(TestPt, HitLocations, ProjectionExtent, TestPt.Z - ProjectionData.ProjectDown, TestPt.Z + ProjectionData.ProjectUp, NavigationFilterCopy, nullptr);
					for (int32 HitIdx = 0; HitIdx < HitLocations.Num(); HitIdx++)
					{
						const bool bHasPathTest = PathGridHelpers::HasPath(NodePoolData, HitLocations[HitIdx].NodeRef);
						if (bHasPathTest)
						{
							Points[Idx] = HitLocations[HitIdx];
							Points[Idx].Location.Z += ProjectionData.PostProjectionVerticalOffset;
							bHasPath = true;
							break;
						}
					}
				}

				if (!bHasPath)
				{
					Points.RemoveAt(Idx);
				}
			}
		}
	}
#endif // WITH_RECAST
}

#undef LOCTEXT_NAMESPACE

