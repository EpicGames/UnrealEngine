// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Generators/EnvQueryGenerator_PerceivedActors.h"
#include "EnvironmentQuery/Contexts/EnvQueryContext_Querier.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_Actor.h"
#include "GameFramework/Pawn.h"
#include "Perception/AIPerceptionListenerInterface.h"
#include "Perception/AIPerceptionComponent.h"
#include "VisualLogger/VisualLogger.h"
#include "GameFramework/Controller.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryGenerator_PerceivedActors)

#define LOCTEXT_NAMESPACE "EnvQueryGenerator"


namespace UE::AI::EQS::Internal
{
	void RemoveDuplicatesSorted(TArray<AActor*>& InOutActors)
	{
		if (InOutActors.Num() <= 1)
		{
			return;
		}

		InOutActors.Sort();

		AActor* PrevValue = InOutActors[0];
		for (int i = 1; i < InOutActors.Num();)
		{
			if (InOutActors[i] == PrevValue)
			{
				const int32 Num = InOutActors.Num();
				int Skip = 1;
				while ((i + Skip) < Num && InOutActors[i + Skip] == PrevValue)
				{
					++Skip;
				}

				InOutActors.RemoveAt(i, Skip, EAllowShrinking::No);
				continue;
			}
			PrevValue = InOutActors[i++];
		}
	}
}


UEnvQueryGenerator_PerceivedActors::UEnvQueryGenerator_PerceivedActors(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer)
{
	ItemType = UEnvQueryItemType_Actor::StaticClass();

	SearchRadius.DefaultValue = -1.f;
	ListenerContext = UEnvQueryContext_Querier::StaticClass();
}

void UEnvQueryGenerator_PerceivedActors::GenerateItems(FEnvQueryInstance& QueryInstance) const
{
	UObject* QueryOwner = QueryInstance.Owner.Get();
	if (QueryOwner == nullptr)
	{	
		return;
	}
		
	UWorld* World = GEngine->GetWorldFromContextObject(QueryOwner, EGetWorldErrorMode::LogAndReturnNull);
	if (World == nullptr)
	{
		return;
	}

	TArray<AActor*> AllPerceivedActors;
	TArray<AActor*> Listeners;
	QueryInstance.PrepareContext(ListenerContext, Listeners);

	SearchRadius.BindData(QueryOwner, QueryInstance.QueryID);
	const float RadiusValue = SearchRadius.GetValue();
	const float RadiusSq = FMath::Square(RadiusValue);

	for (AActor* ListenerActor : Listeners)
	{
		IAIPerceptionListenerInterface* Listener = Cast<IAIPerceptionListenerInterface>(ListenerActor);
		if (Listener == nullptr && ListenerActor != nullptr)
		{
			APawn* ListenerAsPawn = Cast<APawn>(QueryOwner);
			if (ListenerAsPawn)
			{
				Listener = Cast<IAIPerceptionListenerInterface>(ListenerAsPawn->GetController());
			}
		}

		if (Listener == nullptr)
		{
			UE_VLOG(QueryOwner, LogEQS, Error, TEXT("Tried to use EnvQueryGenerator_PerceivedActors while query context actor %s doesn\'t represent a valid perception listener")
				, *GetNameSafe(ListenerActor));
			continue;
		}

		UAIPerceptionComponent* PerceptionComponent = Listener->GetPerceptionComponent();
		if (PerceptionComponent == nullptr)
		{
			UE_VLOG(QueryOwner, LogEQS, Error, TEXT("Tried to use EnvQueryGenerator_PerceivedActors while query context actor\'s %s UAIPerceptionComponent is missing")
				, *ListenerActor->GetName());
			continue;
		}

		const FVector ListenerLocation = ListenerActor->GetActorLocation();
		TArray<AActor*> LocalPerceivedActors;
		if (bIncludeKnownActors)
		{
			PerceptionComponent->GetKnownPerceivedActors(SenseToUse, LocalPerceivedActors);
		}
		else
		{
			PerceptionComponent->GetCurrentlyPerceivedActors(SenseToUse, LocalPerceivedActors);
		}
		
		for (AActor* PerceivedActor : LocalPerceivedActors)
		{
			if (PerceivedActor 
				&& (RadiusValue <= 0 || FVector::DistSquared(ListenerLocation, PerceivedActor->GetActorLocation()) < RadiusSq)
				&& (!AllowedActorClass || PerceivedActor->IsA(AllowedActorClass.Get())))
			{
				AllPerceivedActors.Add(PerceivedActor);
			}
		}
	}

	UE::AI::EQS::Internal::RemoveDuplicatesSorted(AllPerceivedActors);

	QueryInstance.AddItemData<UEnvQueryItemType_Actor>(AllPerceivedActors);
}

FText UEnvQueryGenerator_PerceivedActors::GetDescriptionTitle() const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("DescribeContext"), UEnvQueryTypes::DescribeContext(ListenerContext));
	return FText::Format(LOCTEXT("DescriptionGeneratePerceivedActors", "Actors perceived by {DescribeContext}"), Args);
};

FText UEnvQueryGenerator_PerceivedActors::GetDescriptionDetails() const
{
	FFormatNamedArguments Args;
	if (SearchRadius.IsDynamic() == false && SearchRadius.GetValue() <= 0.f)
	{
		Args.Add(TEXT("Radius"), FText::FromString(TEXT("Perception Range")));
	}
	else
	{
		Args.Add(TEXT("Radius"), FText::FromString(SearchRadius.ToString()));
	}

	if (SenseToUse)
	{
		Args.Add(TEXT("Sense"), FText::FromString(SenseToUse->GetName()));
	}
	else
	{
		Args.Add(TEXT("Sense"), FText::FromString(TEXT("All senses")));
	}
	
	if (AllowedActorClass)
	{
		Args.Add(TEXT("ActorClass"), FText::FromString(AllowedActorClass->GetName()));
		return FText::Format(LOCTEXT("PerceivedActorsOfClassDescription", "radius: {Radius}\nsense: {Sense}\nactors of class: {ActorClass}"), Args);
	}

	return FText::Format(LOCTEXT("PerceivedActorsDescription", "radius: {Radius}\nsense: {Sense}"), Args);
}

#undef LOCTEXT_NAMESPACE

