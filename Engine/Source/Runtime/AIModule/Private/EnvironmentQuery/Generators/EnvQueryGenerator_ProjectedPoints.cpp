// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Generators/EnvQueryGenerator_ProjectedPoints.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_Point.h"
#include "EnvironmentQuery/EnvQueryTraceHelpers.h"
#include "GameFramework/Pawn.h"
#include "NavigationData.h"
#include "NavigationSystem.h"
#include "NavigationSystemTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryGenerator_ProjectedPoints)

UEnvQueryGenerator_ProjectedPoints::UEnvQueryGenerator_ProjectedPoints(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	ProjectionData.TraceMode = EEnvQueryTrace::Navigation;
	ProjectionData.bCanProjectDown = true;
	ProjectionData.bCanDisableTrace = true;
	ProjectionData.ExtentX = 0.0f;

	ItemType = UEnvQueryItemType_Point::StaticClass();
}

void UEnvQueryGenerator_ProjectedPoints::ProjectAndFilterNavPoints(TArray<FNavLocation>& Points, FEnvQueryInstance& QueryInstance) const
{
	const ANavigationData* NavData = nullptr;
	if (ProjectionData.TraceMode != EEnvQueryTrace::None)
	{
		NavData = FEQSHelpers::FindNavigationDataForQuery(QueryInstance, NavDataOverrideContext);
	}

	const UObject* Querier = QueryInstance.Owner.Get();
	if (NavData && Querier && (ProjectionData.TraceMode == EEnvQueryTrace::Navigation))
	{
		FEQSHelpers::RunNavProjection(*NavData, *Querier, ProjectionData, Points, FEQSHelpers::ETraceMode::Discard);
	}

	if (ProjectionData.TraceMode == EEnvQueryTrace::GeometryByChannel || ProjectionData.TraceMode == EEnvQueryTrace::GeometryByProfile)
	{
		FEQSHelpers::RunPhysProjection(QueryInstance.World, ProjectionData, Points);
	}
}

void UEnvQueryGenerator_ProjectedPoints::StoreNavPoints(const TArray<FNavLocation>& Points, FEnvQueryInstance& QueryInstance) const
{
	const int32 InitialElementsCount = QueryInstance.Items.Num();
	QueryInstance.ReserveItemData(InitialElementsCount + Points.Num());
	for (int32 Idx = 0; Idx < Points.Num(); Idx++)
	{
		// store using default function to handle creating new data entry 
		QueryInstance.AddItemData<UEnvQueryItemType_Point>(Points[Idx]);
	}

	FEnvQueryOptionInstance& OptionInstance = QueryInstance.Options[QueryInstance.OptionIndex];
	OptionInstance.bHasNavLocations = (ProjectionData.TraceMode == EEnvQueryTrace::Navigation);
}

void UEnvQueryGenerator_ProjectedPoints::PostLoad()
{
	Super::PostLoad();
	ProjectionData.OnPostLoad();
}

