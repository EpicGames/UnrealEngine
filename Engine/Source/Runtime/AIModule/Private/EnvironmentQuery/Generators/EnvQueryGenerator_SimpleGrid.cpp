// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Generators/EnvQueryGenerator_SimpleGrid.h"
#include "EnvironmentQuery/Contexts/EnvQueryContext_Querier.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryGenerator_SimpleGrid)

#define LOCTEXT_NAMESPACE "EnvQueryGenerator"

UEnvQueryGenerator_SimpleGrid::UEnvQueryGenerator_SimpleGrid(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	GenerateAround = UEnvQueryContext_Querier::StaticClass();
	GridSize.DefaultValue = 500.0f;
	SpaceBetween.DefaultValue = 100.0f;
}

void UEnvQueryGenerator_SimpleGrid::GenerateItems(FEnvQueryInstance& QueryInstance) const
{
	UObject* BindOwner = QueryInstance.Owner.Get();
	GridSize.BindData(BindOwner, QueryInstance.QueryID);
	SpaceBetween.BindData(BindOwner, QueryInstance.QueryID);

	const float DensityValue = SpaceBetween.GetValue();
	const float RadiusValue = GridSize.GetValue();

	if (DensityValue > 0.f && RadiusValue > 0.f)
	{
		const int32 ItemCount = FPlatformMath::TruncToInt((RadiusValue * 2.0f / DensityValue) + 1);

		TArray<FVector> ContextLocations;
		QueryInstance.PrepareContext(GenerateAround, ContextLocations);

		const uint64 TotalItemsCount = static_cast<uint64>(ItemCount) * static_cast<uint64>(ItemCount) * static_cast<uint64>(ContextLocations.Num());

		if (TotalItemsCount <= static_cast<uint64>(TNumericLimits<int32>::Max()))
		{
			const int32 ItemCountHalf = ItemCount / 2;

			TArray<FNavLocation> GridPoints;
			GridPoints.Reserve(static_cast<int32>(TotalItemsCount));

			for (FVector ContextLocation : ContextLocations)
			{
				for (int32 IndexX = 0; IndexX < ItemCount; ++IndexX)
				{
					for (int32 IndexY = 0; IndexY < ItemCount; ++IndexY)
					{
						const FVector ItemRelativeLocation(DensityValue * (IndexX - ItemCountHalf), DensityValue * (IndexY - ItemCountHalf), 0);
						FNavLocation TestPoint = FNavLocation(ContextLocation - ItemRelativeLocation);
						GridPoints.Add(MoveTemp(TestPoint));
					}
				}
			}

			ProjectAndFilterNavPoints(GridPoints, QueryInstance);
			StoreNavPoints(GridPoints, QueryInstance);
		}
		else
		{
			UE_VLOG_UELOG(QueryInstance.Owner.Get(), LogEQS, Error
				, TEXT("%hs, query %s: attempting to generate %llu items. That's way more than expected or supported. Please review the query's config.")
				, __FUNCTION__, *QueryInstance.QueryName, TotalItemsCount);
		}
	}
	else
	{
		const UObject* Querier = QueryInstance.Owner.Get();
		UE_CVLOG(DensityValue <= 0.f, Querier, LogEQS, Error
			, TEXT("%hs, query %s: Zero or negative `SpaceBetween` values are not supported")
			, __FUNCTION__, *QueryInstance.QueryName);
		UE_CVLOG(RadiusValue <= 0.f, Querier, LogEQS, Error
			, TEXT("%hs, query %s: Zero or negative `GridSize` values are not supported")
			, __FUNCTION__, *QueryInstance.QueryName);
	}
}

FText UEnvQueryGenerator_SimpleGrid::GetDescriptionTitle() const
{
	return FText::Format(LOCTEXT("SimpleGridDescriptionGenerateAroundContext", "{0}: generate around {1}"),
		Super::GetDescriptionTitle(), UEnvQueryTypes::DescribeContext(GenerateAround));
};

FText UEnvQueryGenerator_SimpleGrid::GetDescriptionDetails() const
{
	FText Desc = FText::Format(LOCTEXT("SimpleGridDescription", "radius: {0}, space between: {1}"),
		FText::FromString(GridSize.ToString()), FText::FromString(SpaceBetween.ToString()));

	FText ProjDesc = ProjectionData.ToText(FEnvTraceData::Brief);
	if (!ProjDesc.IsEmpty())
	{
		FFormatNamedArguments ProjArgs;
		ProjArgs.Add(TEXT("Description"), Desc);
		ProjArgs.Add(TEXT("ProjectionDescription"), ProjDesc);
		Desc = FText::Format(LOCTEXT("SimpleGridDescriptionWithProjection", "{Description}, {ProjectionDescription}"), ProjArgs);
	}

	return Desc;
}

#undef LOCTEXT_NAMESPACE

