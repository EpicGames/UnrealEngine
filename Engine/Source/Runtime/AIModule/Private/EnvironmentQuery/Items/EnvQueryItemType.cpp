// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Items/EnvQueryItemType.h"
#include "EnvironmentQuery/EnvQueryManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryItemType)

UEnvQueryItemType::UEnvQueryItemType(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	// register in known types 
	if (HasAnyFlags(RF_ClassDefaultObject) && !GetClass()->HasAnyClassFlags(CLASS_Abstract))
	{
		UEnvQueryManager::RegisteredItemTypes.Add(GetClass());
	}
}

void UEnvQueryItemType::FinishDestroy()
{
	// unregister from known types 
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		UEnvQueryManager::RegisteredItemTypes.RemoveSingleSwap(GetClass());
	}

	Super::FinishDestroy();
}

void UEnvQueryItemType::AddBlackboardFilters(FBlackboardKeySelector& KeySelector, UObject* FilterOwner) const
{
}

bool UEnvQueryItemType::StoreInBlackboard(FBlackboardKeySelector& KeySelector, UBlackboardComponent* Blackboard, const uint8* RawData) const
{
	return false;
}

FString UEnvQueryItemType::GetDescription(const uint8* RawData) const
{
	return TEXT("item");
}

