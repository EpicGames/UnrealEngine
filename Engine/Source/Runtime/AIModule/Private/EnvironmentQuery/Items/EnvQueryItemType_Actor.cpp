// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Items/EnvQueryItemType_Actor.h"
#include "UObject/WeakObjectPtr.h"
#include "GameFramework/Actor.h"
#include "AITypes.h"
#include "EnvironmentQuery/EnvQueryTypes.h"

#if WITH_EDITOR
#include "Selection.h"
#include "Editor.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryItemType_Actor)

UEnvQueryItemType_Actor::UEnvQueryItemType_Actor(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	ValueSize = sizeof(FWeakObjectPtr);
}

AActor* UEnvQueryItemType_Actor::GetValue(const uint8* RawData)
{
	FWeakObjectPtr WeakObjPtr = GetValueFromMemory<FWeakObjectPtr>(RawData);
	return (AActor*)(WeakObjPtr.Get());
}

void UEnvQueryItemType_Actor::SetValue(uint8* RawData, const FWeakObjectPtr& Value)
{
	FWeakObjectPtr WeakObjPtr(Value);
	SetValueInMemory<FWeakObjectPtr>(RawData, WeakObjPtr);
}

FVector UEnvQueryItemType_Actor::GetItemLocation(const uint8* RawData) const
{
	AActor* MyActor = UEnvQueryItemType_Actor::GetValue(RawData);
	return MyActor ? MyActor->GetActorLocation() : FAISystem::InvalidLocation;
}

FRotator UEnvQueryItemType_Actor::GetItemRotation(const uint8* RawData) const
{
	AActor* MyActor = UEnvQueryItemType_Actor::GetValue(RawData);
	return MyActor ? MyActor->GetActorRotation() : FAISystem::InvalidRotation;
}

AActor* UEnvQueryItemType_Actor::GetActor(const uint8* RawData) const
{
	return UEnvQueryItemType_Actor::GetValue(RawData);
}

void UEnvQueryItemType_Actor::SetContextHelper(FEnvQueryContextData& ContextData, const AActor* SingleActor)
{
	ContextData.ValueType = UEnvQueryItemType_Actor::StaticClass();
	ContextData.NumValues = 1;
	ContextData.RawData.SetNumUninitialized(sizeof(FWeakObjectPtr));

	UEnvQueryItemType_Actor::SetValue(ContextData.RawData.GetData(), SingleActor);
}

void UEnvQueryItemType_Actor::SetContextHelper(FEnvQueryContextData& ContextData, const TArray<const AActor*>& MultipleActors)
{
	ContextData.ValueType = UEnvQueryItemType_Actor::StaticClass();
	ContextData.NumValues = MultipleActors.Num();
	ContextData.RawData.SetNumUninitialized(sizeof(FWeakObjectPtr) * MultipleActors.Num());

	checkf(ContextData.RawData.Num() == sizeof(FWeakObjectPtr) * MultipleActors.Num(), TEXT("Failed to allocate the appropriate amount of memory"));

	uint8* RawData = (uint8*)ContextData.RawData.GetData();
	for (int32 ActorIndex = 0; ActorIndex < MultipleActors.Num(); ActorIndex++)
	{
		UEnvQueryItemType_Actor::SetValue(RawData, MultipleActors[ActorIndex]);
		RawData += sizeof(FWeakObjectPtr);
	}
}

void UEnvQueryItemType_Actor::SetContextHelper(FEnvQueryContextData& ContextData, const TArray<AActor*>& MultipleActors)
{
	ContextData.ValueType = UEnvQueryItemType_Actor::StaticClass();
	ContextData.NumValues = MultipleActors.Num();
	ContextData.RawData.SetNumUninitialized(sizeof(FWeakObjectPtr) * MultipleActors.Num());

	checkf(ContextData.RawData.Num() == sizeof(FWeakObjectPtr) * MultipleActors.Num(), TEXT("Failed to allocate the appropriate amount of memory"));

	uint8* RawData = (uint8*)ContextData.RawData.GetData();
	for (int32 ActorIndex = 0; ActorIndex < MultipleActors.Num(); ActorIndex++)
	{
		UEnvQueryItemType_Actor::SetValue(RawData, MultipleActors[ActorIndex]);
		RawData += sizeof(FWeakObjectPtr);
	}
}

void UEnvQueryItemType_Actor::AddEditorSelectedActors(FEnvQueryContextData& ContextData)
{
#if WITH_EDITOR
	if (GEditor && !GEditor->IsPlayingSessionInEditor())
	{
		TArray<AActor*> SelectedActors;
		SelectedActors.Reserve(GEditor->GetSelectedActorCount());
		for (FSelectionIterator It = GEditor->GetSelectedActorIterator(); It; ++It)
		{
			if (AActor* Actor = Cast<AActor>(*It))
			{
				SelectedActors.Add(Actor);
			}
		}
		SetContextHelper(ContextData, SelectedActors);
	}
#endif // WITH_EDITOR
}

