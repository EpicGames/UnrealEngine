// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Items/EnvQueryItemType_ActorBase.h"
#include "GameFramework/Actor.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Object.h"
#include "BehaviorTree/BlackboardComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryItemType_ActorBase)

void UEnvQueryItemType_ActorBase::AddBlackboardFilters(FBlackboardKeySelector& KeySelector, UObject* FilterOwner) const
{
	Super::AddBlackboardFilters(KeySelector, FilterOwner);
	KeySelector.AddObjectFilter(FilterOwner, GetClass()->GetFName(), AActor::StaticClass());
}

bool UEnvQueryItemType_ActorBase::StoreInBlackboard(FBlackboardKeySelector& KeySelector, UBlackboardComponent* Blackboard, const uint8* RawData) const
{
	bool bStored = Super::StoreInBlackboard(KeySelector, Blackboard, RawData);
	if (!bStored && KeySelector.SelectedKeyType == UBlackboardKeyType_Object::StaticClass())
	{
		UObject* MyObject = GetActor(RawData);
		Blackboard->SetValue<UBlackboardKeyType_Object>(KeySelector.GetSelectedKeyID(), MyObject);

		bStored = true;
	}

	return bStored;
}

FString UEnvQueryItemType_ActorBase::GetDescription(const uint8* RawData) const
{
	const AActor* Actor = GetActor(RawData);
	return GetNameSafe(Actor);
}

AActor* UEnvQueryItemType_ActorBase::GetActor(const uint8* RawData) const
{
	return NULL;
}

