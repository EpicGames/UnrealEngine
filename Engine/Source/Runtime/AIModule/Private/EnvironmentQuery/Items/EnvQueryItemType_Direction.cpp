// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Items/EnvQueryItemType_Direction.h"
#include "EnvironmentQuery/EnvQueryTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryItemType_Direction)

UEnvQueryItemType_Direction::UEnvQueryItemType_Direction(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	ValueSize = sizeof(FVector);
}

FVector UEnvQueryItemType_Direction::GetValue(const uint8* RawData)
{
	return GetValueFromMemory<FVector>(RawData);
}

void UEnvQueryItemType_Direction::SetValue(uint8* RawData, const FVector& Value)
{
	return SetValueInMemory<FVector>(RawData, Value);
}

FRotator UEnvQueryItemType_Direction::GetValueRot(const uint8* RawData)
{
	return GetValueFromMemory<FVector>(RawData).Rotation();
}

void UEnvQueryItemType_Direction::SetValueRot(uint8* RawData, const FRotator& Value)
{
	return SetValueInMemory<FVector>(RawData, Value.Vector());
}

FRotator UEnvQueryItemType_Direction::GetItemRotation(const uint8* RawData) const
{
	return UEnvQueryItemType_Direction::GetValueRot(RawData);
}

FString UEnvQueryItemType_Direction::GetDescription(const uint8* RawData) const
{
	FRotator Rot = GetItemRotation(RawData);
	return FString::Printf(TEXT("(P=%.0f,Y=%.0f,R=%.0f)"), Rot.Pitch, Rot.Yaw, Rot.Roll);
}

void UEnvQueryItemType_Direction::SetContextHelper(FEnvQueryContextData& ContextData, const FVector& SingleDirection)
{
	ContextData.ValueType = UEnvQueryItemType_Direction::StaticClass();
	ContextData.NumValues = 1;
	ContextData.RawData.SetNumUninitialized(sizeof(FVector));

	UEnvQueryItemType_Direction::SetValue((uint8*)ContextData.RawData.GetData(), SingleDirection);
}

void UEnvQueryItemType_Direction::SetContextHelper(FEnvQueryContextData& ContextData, const FRotator& SingleRotation)
{
	ContextData.ValueType = UEnvQueryItemType_Direction::StaticClass();
	ContextData.NumValues = 1;
	ContextData.RawData.SetNumUninitialized(sizeof(FVector));

	UEnvQueryItemType_Direction::SetValueRot((uint8*)ContextData.RawData.GetData(), SingleRotation);
}

void UEnvQueryItemType_Direction::SetContextHelper(FEnvQueryContextData& ContextData, const TArray<FVector>& MultipleDirections)
{
	ContextData.ValueType = UEnvQueryItemType_Direction::StaticClass();
	ContextData.NumValues = MultipleDirections.Num();
	ContextData.RawData.SetNumUninitialized(sizeof(FVector) * MultipleDirections.Num());

	checkf(ContextData.RawData.Num() == sizeof(FVector) * MultipleDirections.Num(), TEXT("Failed to allocate the appropriate amount of memory"));

	uint8* RawData = (uint8*)ContextData.RawData.GetData();
	for (int32 DirectionIndex = 0; DirectionIndex < MultipleDirections.Num(); DirectionIndex++)
	{
		UEnvQueryItemType_Direction::SetValue(RawData, MultipleDirections[DirectionIndex]);
		RawData += sizeof(FVector);
	}
}

void UEnvQueryItemType_Direction::SetContextHelper(FEnvQueryContextData& ContextData, const TArray<FRotator>& MultipleRotations)
{
	ContextData.ValueType = UEnvQueryItemType_Direction::StaticClass();
	ContextData.NumValues = MultipleRotations.Num();
	ContextData.RawData.SetNumUninitialized(sizeof(FVector) * MultipleRotations.Num());

	checkf(ContextData.RawData.Num() == sizeof(FVector) * MultipleRotations.Num(), TEXT("Failed to allocate the appropriate amount of memory"));

	uint8* RawData = (uint8*)ContextData.RawData.GetData();
	for (int32 RotationIndex = 0; RotationIndex < MultipleRotations.Num(); RotationIndex++)
	{
		UEnvQueryItemType_Direction::SetValueRot(RawData, MultipleRotations[RotationIndex]);
		RawData += sizeof(FVector);
	}
}

