// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Items/EnvQueryItemType_Point.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryItemType_Point)

template<>
void FEnvQueryInstance::AddItemData<UEnvQueryItemType_Point, FVector>(FVector ItemValue)
{
	const FNavLocation Item = FNavLocation(ItemValue);
	AddItemData<UEnvQueryItemType_Point>(Item);
}

UEnvQueryItemType_Point::UEnvQueryItemType_Point(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	ValueSize = sizeof(FNavLocation);
}

FNavLocation UEnvQueryItemType_Point::GetValue(const uint8* RawData)
{
	return GetValueFromMemory<FNavLocation>(RawData);
}

void UEnvQueryItemType_Point::SetValue(uint8* RawData, const FNavLocation& Value)
{
	return SetValueInMemory<FNavLocation>(RawData, Value);
}

FVector UEnvQueryItemType_Point::GetItemLocation(const uint8* RawData) const
{
	return UEnvQueryItemType_Point::GetValue(RawData);
}

FNavLocation UEnvQueryItemType_Point::GetItemNavLocation(const uint8* RawData) const
{
	return UEnvQueryItemType_Point::GetValue(RawData);
}

void UEnvQueryItemType_Point::SetItemNavLocation(uint8* RawData, const FNavLocation& Value) const
{
	UEnvQueryItemType_Point::SetValue(RawData, Value);
}

void UEnvQueryItemType_Point::SetContextHelper(FEnvQueryContextData& ContextData, const FVector& SinglePoint)
{
	ContextData.ValueType = UEnvQueryItemType_Point::StaticClass();
	ContextData.NumValues = 1;
	ContextData.RawData.SetNumUninitialized(sizeof(FNavLocation));

	UEnvQueryItemType_Point::SetValue((uint8*)ContextData.RawData.GetData(), FNavLocation(SinglePoint));
}

void UEnvQueryItemType_Point::SetContextHelper(FEnvQueryContextData& ContextData, const TArray<FVector>& MultiplePoints)
{
	ContextData.ValueType = UEnvQueryItemType_Point::StaticClass();
	ContextData.NumValues = MultiplePoints.Num();
	ContextData.RawData.SetNumUninitialized(sizeof(FNavLocation)* MultiplePoints.Num());

	checkf(ContextData.RawData.Num() == sizeof(FNavLocation) * MultiplePoints.Num(), TEXT("Failed to allocate the appropriate amount of memory"));

	uint8* RawData = (uint8*)ContextData.RawData.GetData();
	for (int32 PointIndex = 0; PointIndex < MultiplePoints.Num(); PointIndex++)
	{
		UEnvQueryItemType_Point::SetValue(RawData, FNavLocation(MultiplePoints[PointIndex]));
		RawData += sizeof(FNavLocation);
	}
}

