// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Items/EnvQueryItemType_VectorBase.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Vector.h"
#include "BehaviorTree/BlackboardComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryItemType_VectorBase)

void UEnvQueryItemType_VectorBase::AddBlackboardFilters(FBlackboardKeySelector& KeySelector, UObject* FilterOwner) const
{
	KeySelector.AddVectorFilter(FilterOwner, GetClass()->GetFName());
}

bool UEnvQueryItemType_VectorBase::StoreInBlackboard(FBlackboardKeySelector& KeySelector, UBlackboardComponent* Blackboard, const uint8* RawData) const
{
	bool bStored = false;
	if (KeySelector.SelectedKeyType == UBlackboardKeyType_Vector::StaticClass())
	{
		const FVector MyLocation = GetItemLocation(RawData);
		Blackboard->SetValue<UBlackboardKeyType_Vector>(KeySelector.GetSelectedKeyID(), MyLocation);

		bStored = true;
	}

	return bStored;
}

FString UEnvQueryItemType_VectorBase::GetDescription(const uint8* RawData) const
{
	FVector PointLocation = GetItemLocation(RawData);
	return FString::Printf(TEXT("(X=%.0f,Y=%.0f,Z=%.0f)"), PointLocation.X, PointLocation.Y, PointLocation.Z);
}

FVector UEnvQueryItemType_VectorBase::GetItemLocation(const uint8* RawData) const
{
	return FVector::ZeroVector;
}

FRotator UEnvQueryItemType_VectorBase::GetItemRotation(const uint8* RawData) const
{
	return FRotator::ZeroRotator;
}

