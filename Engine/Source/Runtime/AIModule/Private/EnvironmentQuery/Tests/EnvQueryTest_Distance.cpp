// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Tests/EnvQueryTest_Distance.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_VectorBase.h"
#include "EnvironmentQuery/Contexts/EnvQueryContext_Querier.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryTest_Distance)

#define ENVQUERYTEST_DISTANCE_NAN_DETECTION 1

namespace
{
	FORCEINLINE float CalcDistance3D(const FVector& PosA, const FVector& PosB)
	{
		// Static cast this to a float, for EQS scoring purposes float precision is OK.
		return static_cast<float>(FVector::Distance(PosA, PosB));
	}

	FORCEINLINE float CalcDistance2D(const FVector& PosA, const FVector& PosB)
	{
		// Static cast this to a float, for EQS scoring purposes float precision is OK.
		return static_cast<float>(FVector::Dist2D(PosA, PosB));
	}

	FORCEINLINE float CalcDistanceZ(const FVector& PosA, const FVector& PosB)
	{
		// Static cast this to a float, for EQS scoring purposes float precision is OK.
		return static_cast<float>(PosB.Z - PosA.Z);
	}

	FORCEINLINE float CalcDistanceAbsoluteZ(const FVector& PosA, const FVector& PosB)
	{
		// Static cast this to a float, for EQS scoring purposes float precision is OK.
		return static_cast<float>(FMath::Abs(PosB.Z - PosA.Z));
	}

	FORCEINLINE void CheckItemLocationForNaN(const FVector& ItemLocation, UObject* QueryOwner, int32 Index, uint8 TestMode)
	{
#if ENVQUERYTEST_DISTANCE_NAN_DETECTION
		ensureMsgf(!ItemLocation.ContainsNaN(), TEXT("EnvQueryTest_Distance NaN in ItemLocation with owner %s. X=%f,Y=%f,Z=%f. Index:%d, TesMode:%d"), *GetPathNameSafe(QueryOwner), ItemLocation.X, ItemLocation.Y, ItemLocation.Z, Index, TestMode);
#endif
	}

	FORCEINLINE void CheckContextLocationForNaN(const FVector& ContextLocation, UObject* QueryOwner, int32 Index, uint8 TestMode)
	{
#if ENVQUERYTEST_DISTANCE_NAN_DETECTION
		ensureMsgf(!ContextLocation.ContainsNaN(), TEXT("EnvQueryTest_Distance NaN in ContextLocations with owner %s. X=%f,Y=%f,Z=%f. Index:%d, TesMode:%d"), *GetPathNameSafe(QueryOwner), ContextLocation.X, ContextLocation.Y, ContextLocation.Z, Index, TestMode);
#endif
	}
}

UEnvQueryTest_Distance::UEnvQueryTest_Distance(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	DistanceTo = UEnvQueryContext_Querier::StaticClass();
	Cost = EEnvTestCost::Low;
	ValidItemType = UEnvQueryItemType_VectorBase::StaticClass();
}

void UEnvQueryTest_Distance::RunTest(FEnvQueryInstance& QueryInstance) const
{
	UObject* QueryOwner = QueryInstance.Owner.Get();
	if (QueryOwner == nullptr)
	{
		return;
	}

	FloatValueMin.BindData(QueryOwner, QueryInstance.QueryID);
	float MinThresholdValue = FloatValueMin.GetValue();

	FloatValueMax.BindData(QueryOwner, QueryInstance.QueryID);
	float MaxThresholdValue = FloatValueMax.GetValue();

	// don't support context Item here, it doesn't make any sense
	TArray<FVector> ContextLocations;
	if (!QueryInstance.PrepareContext(DistanceTo, ContextLocations))
	{
		return;
	}

	switch (TestMode)
	{
		case EEnvTestDistance::Distance3D:	
			for (FEnvQueryInstance::ItemIterator It(this, QueryInstance); It; ++It)
			{
				const FVector ItemLocation = GetItemLocation(QueryInstance, It.GetIndex());
				CheckItemLocationForNaN(ItemLocation, QueryOwner, It.GetIndex(), TestMode.GetIntValue());
				for (int32 ContextIndex = 0; ContextIndex < ContextLocations.Num(); ContextIndex++)
				{
					CheckContextLocationForNaN(ContextLocations[ContextIndex], QueryOwner, ContextIndex, TestMode.GetIntValue());
					const float Distance = CalcDistance3D(ItemLocation, ContextLocations[ContextIndex]);
					It.SetScore(TestPurpose, FilterType, Distance, MinThresholdValue, MaxThresholdValue);
				}
			}
			break;

		case EEnvTestDistance::Distance2D:	
			for (FEnvQueryInstance::ItemIterator It(this, QueryInstance); It; ++It)
			{
				const FVector ItemLocation = GetItemLocation(QueryInstance, It.GetIndex());
				CheckItemLocationForNaN(ItemLocation, QueryOwner, It.GetIndex(), TestMode.GetIntValue());
				for (int32 ContextIndex = 0; ContextIndex < ContextLocations.Num(); ContextIndex++)
				{
					CheckContextLocationForNaN(ContextLocations[ContextIndex], QueryOwner, ContextIndex, TestMode.GetIntValue());
					const float Distance = CalcDistance2D(ItemLocation, ContextLocations[ContextIndex]);
					It.SetScore(TestPurpose, FilterType, Distance, MinThresholdValue, MaxThresholdValue);
				}
			}
			break;

		case EEnvTestDistance::DistanceZ:	
			for (FEnvQueryInstance::ItemIterator It(this, QueryInstance); It; ++It)
			{
				const FVector ItemLocation = GetItemLocation(QueryInstance, It.GetIndex());
				CheckItemLocationForNaN(ItemLocation, QueryOwner, It.GetIndex(), TestMode.GetIntValue());
				for (int32 ContextIndex = 0; ContextIndex < ContextLocations.Num(); ContextIndex++)
				{
					CheckContextLocationForNaN(ContextLocations[ContextIndex], QueryOwner, ContextIndex, TestMode.GetIntValue());
					const float Distance = CalcDistanceZ(ItemLocation, ContextLocations[ContextIndex]);
					It.SetScore(TestPurpose, FilterType, Distance, MinThresholdValue, MaxThresholdValue);
				}
			}
			break;

		case EEnvTestDistance::DistanceAbsoluteZ:
			for (FEnvQueryInstance::ItemIterator It(this, QueryInstance); It; ++It)
			{
				const FVector ItemLocation = GetItemLocation(QueryInstance, It.GetIndex());
				CheckItemLocationForNaN(ItemLocation, QueryOwner, It.GetIndex(), TestMode.GetIntValue());
				for (int32 ContextIndex = 0; ContextIndex < ContextLocations.Num(); ContextIndex++)
				{
					CheckContextLocationForNaN(ContextLocations[ContextIndex], QueryOwner, ContextIndex, TestMode.GetIntValue());
					const float Distance = CalcDistanceAbsoluteZ(ItemLocation, ContextLocations[ContextIndex]);
					It.SetScore(TestPurpose, FilterType, Distance, MinThresholdValue, MaxThresholdValue);
				}
			}
			break;

		default:
			checkNoEntry();
			return;
	}
}

FText UEnvQueryTest_Distance::GetDescriptionTitle() const
{
	FString ModeDesc;
	switch (TestMode)
	{
		case EEnvTestDistance::Distance3D:
			ModeDesc = TEXT("");
			break;

		case EEnvTestDistance::Distance2D:
			ModeDesc = TEXT(" 2D");
			break;

		case EEnvTestDistance::DistanceZ:
			ModeDesc = TEXT(" Z");
			break;

		default:
			break;
	}

	return FText::FromString(FString::Printf(TEXT("%s%s: to %s"), 
		*Super::GetDescriptionTitle().ToString(), *ModeDesc,
		*UEnvQueryTypes::DescribeContext(DistanceTo).ToString()));
}

FText UEnvQueryTest_Distance::GetDescriptionDetails() const
{
	return DescribeFloatTestParams();
}

