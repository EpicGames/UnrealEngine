// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Tests/EnvQueryTest_Dot.h"
#include "AITypes.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_VectorBase.h"
#include "EnvironmentQuery/Contexts/EnvQueryContext_Querier.h"
#include "EnvironmentQuery/Contexts/EnvQueryContext_Item.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryTest_Dot)

UEnvQueryTest_Dot::UEnvQueryTest_Dot(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	Cost = EEnvTestCost::Low;
	ValidItemType = UEnvQueryItemType_VectorBase::StaticClass();
	LineA.DirMode = EEnvDirection::Rotation;
	LineA.Rotation = UEnvQueryContext_Querier::StaticClass();
	LineB.DirMode = EEnvDirection::TwoPoints;
	LineB.LineFrom = UEnvQueryContext_Querier::StaticClass();
	LineB.LineTo = UEnvQueryContext_Item::StaticClass();

	TestMode = EEnvTestDot::Dot3D;
	bAbsoluteValue = false;
}

void UEnvQueryTest_Dot::RunTest(FEnvQueryInstance& QueryInstance) const
{
	UObject* QueryOwner = QueryInstance.Owner.Get();
	if (QueryOwner == nullptr)
	{
		return;
	}

	FloatValueMin.BindData(QueryOwner, QueryInstance.QueryID);
	float MinThresholdValue = FloatValueMin.GetValue();

	FloatValueMax.BindData(QueryOwner, QueryInstance.QueryID);
	float MaxThresholdValue = FloatValueMax.GetValue();

	// gather all possible directions: for contexts different than Item
	TArray<FVector> LineADirs;
	const bool bUpdateLineAPerItem = RequiresPerItemUpdates(LineA.LineFrom, LineA.LineTo, LineA.Rotation, LineA.DirMode == EEnvDirection::Rotation);
	if (!bUpdateLineAPerItem)
	{
		GatherLineDirections(LineADirs, QueryInstance, LineA.LineFrom, LineA.LineTo, LineA.Rotation, LineA.DirMode == EEnvDirection::Rotation);
		if (LineADirs.Num() == 0)
		{
			return;
		}
	}

	TArray<FVector> LineBDirs;
	const bool bUpdateLineBPerItem = RequiresPerItemUpdates(LineB.LineFrom, LineB.LineTo, LineB.Rotation, LineB.DirMode == EEnvDirection::Rotation);
	if (!bUpdateLineBPerItem)
	{
		GatherLineDirections(LineBDirs, QueryInstance, LineB.LineFrom, LineB.LineTo, LineB.Rotation, LineB.DirMode == EEnvDirection::Rotation);
		if (LineBDirs.Num() == 0)
		{
			return;
		}
	}

	// loop through all items
	for (FEnvQueryInstance::ItemIterator It(this, QueryInstance); It; ++It)
	{
		// update lines for contexts using current item
		if (bUpdateLineAPerItem || bUpdateLineBPerItem)
		{
			const FVector ItemLocation = (LineA.DirMode == EEnvDirection::Rotation && LineB.DirMode == EEnvDirection::Rotation) ? FVector::ZeroVector : GetItemLocation(QueryInstance, It.GetIndex());
			const FRotator ItemRotation = (LineA.DirMode == EEnvDirection::Rotation || LineB.DirMode == EEnvDirection::Rotation) ? GetItemRotation(QueryInstance, It.GetIndex()) : FRotator::ZeroRotator;

			if (bUpdateLineAPerItem)
			{
				LineADirs.Reset();
				GatherLineDirections(LineADirs, QueryInstance, LineA.LineFrom, LineA.LineTo, LineA.Rotation, LineA.DirMode == EEnvDirection::Rotation, ItemLocation, ItemRotation);
			}

			if (bUpdateLineBPerItem)
			{
				LineBDirs.Reset();
				GatherLineDirections(LineBDirs, QueryInstance, LineB.LineFrom, LineB.LineTo, LineB.Rotation, LineB.DirMode == EEnvDirection::Rotation, ItemLocation, ItemRotation);
			}
		}

		// perform test for each line pair
		for (int32 LineAIndex = 0; LineAIndex < LineADirs.Num(); LineAIndex++)
		{
			for (int32 LineBIndex = 0; LineBIndex < LineBDirs.Num(); LineBIndex++)
			{
				float DotValue = 0.f;
				switch (TestMode)
				{
					case EEnvTestDot::Dot3D:
						DotValue = static_cast<float>(FVector::DotProduct(LineADirs[LineAIndex], LineBDirs[LineBIndex]));
						break;

					case EEnvTestDot::Dot2D:
						DotValue = static_cast<float>(LineADirs[LineAIndex].CosineAngle2D(LineBDirs[LineBIndex]));
						break;

					default:
						UE_LOGF(LogEQS, Error, "Invalid TestMode in EnvQueryTest_Dot in query %ls!", *QueryInstance.QueryName);
						break;
				}
				
				// invalid LineADirs, LineBDirs? 
				if (FMath::IsNaN(DotValue))
				{
					DotValue = 0.f;
				}
				else if (bAbsoluteValue)
				{
					DotValue = FMath::Abs(DotValue);
				}

				It.SetScore(TestPurpose, FilterType, DotValue, MinThresholdValue, MaxThresholdValue);
			}
		}
	}
}

void UEnvQueryTest_Dot::GatherLineDirections(TArray<FVector>& Directions, FEnvQueryInstance& QueryInstance, const FVector& ItemLocation,
	TSubclassOf<UEnvQueryContext> LineFrom, TSubclassOf<UEnvQueryContext> LineTo) const
{
	TArray<FVector> ContextLocationFrom;
	if (IsContextPerItem(LineFrom))
	{
		ContextLocationFrom.Add(ItemLocation);
	}
	else
	{
		QueryInstance.PrepareContext(LineFrom, ContextLocationFrom);
	}
	
	for (int32 FromIndex = 0; FromIndex < ContextLocationFrom.Num(); FromIndex++)
	{
		if (!FAISystem::IsValidLocation(ContextLocationFrom[FromIndex]))
		{
			continue;
		}

		TArray<FVector> ContextLocationTo;
		if (IsContextPerItem(LineTo))
		{
			ContextLocationTo.Add(ItemLocation);
		}
		else
		{
			QueryInstance.PrepareContext(LineTo, ContextLocationTo);
		}
		
		for (int32 ToIndex = 0; ToIndex < ContextLocationTo.Num(); ToIndex++)
		{
			if (FAISystem::IsValidLocation(ContextLocationTo[ToIndex]))
			{
				const FVector Dir = (ContextLocationTo[ToIndex] - ContextLocationFrom[FromIndex]).GetSafeNormal();
				Directions.Add(Dir);
			}	
		}
	}
}

void UEnvQueryTest_Dot::GatherLineDirections(TArray<FVector>& Directions, FEnvQueryInstance& QueryInstance, const FRotator& ItemRotation, TSubclassOf<UEnvQueryContext> LineDirection) const
{
	TArray<FRotator> ContextRotations;
	if (IsContextPerItem(LineDirection))
	{
		ContextRotations.Add(ItemRotation);
	}
	else
	{
		QueryInstance.PrepareContext(LineDirection, ContextRotations);
	}

	for (int32 RotationIndex = 0; RotationIndex < ContextRotations.Num(); RotationIndex++)
	{
		const FVector Dir = ContextRotations[RotationIndex].Vector();
		Directions.Add(Dir);
	}
}

void UEnvQueryTest_Dot::GatherLineDirections(TArray<FVector>& Directions, FEnvQueryInstance& QueryInstance,
	TSubclassOf<UEnvQueryContext> LineFrom, TSubclassOf<UEnvQueryContext> LineTo, TSubclassOf<UEnvQueryContext> LineDirection, bool bUseDirectionContext,
	const FVector& ItemLocation, const FRotator& ItemRotation) const
{
	if (bUseDirectionContext)
	{
		GatherLineDirections(Directions, QueryInstance, ItemRotation, LineDirection);
	}
	else
	{
		GatherLineDirections(Directions, QueryInstance, ItemLocation, LineFrom, LineTo);
	}
}

bool UEnvQueryTest_Dot::RequiresPerItemUpdates(TSubclassOf<UEnvQueryContext> LineFrom, TSubclassOf<UEnvQueryContext> LineTo, TSubclassOf<UEnvQueryContext> LineDirection, bool bUseDirectionContext) const
{
	bool bRequirePerItemUpdate = false;
	if (bUseDirectionContext)
	{
		bRequirePerItemUpdate = IsContextPerItem(LineDirection);
	}
	else
	{
		bRequirePerItemUpdate = IsContextPerItem(LineFrom) || IsContextPerItem(LineTo);
	}

	return bRequirePerItemUpdate;
}

FText UEnvQueryTest_Dot::GetDescriptionTitle() const
{
	FString ModeDesc;
	switch (TestMode)
	{
		case EEnvTestDot::Dot3D:
			ModeDesc = TEXT("");
			break;

		case EEnvTestDot::Dot2D:
			ModeDesc = TEXT(" 2D");
			break;

		default:
			break;
	}

	return FText::FromString(FString::Printf(TEXT("%s%s%s: %s and %s"), bAbsoluteValue ? TEXT("Absolute ") : TEXT(""),
		*Super::GetDescriptionTitle().ToString(), *ModeDesc, *LineA.ToText().ToString(), *LineB.ToText().ToString()));
}

FText UEnvQueryTest_Dot::GetDescriptionDetails() const
{
	return DescribeFloatTestParams();
}

