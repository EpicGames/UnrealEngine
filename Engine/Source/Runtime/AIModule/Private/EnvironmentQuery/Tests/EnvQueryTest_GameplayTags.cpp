// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Tests/EnvQueryTest_GameplayTags.h"

#include "GameFramework/Actor.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_ActorBase.h"
#include "BehaviorTree/BTNode.h"
#include "GameplayTagAssetInterface.h"
#include "UObject/ObjectSaveContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryTest_GameplayTags)

UEnvQueryTest_GameplayTags::UEnvQueryTest_GameplayTags(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer)
{
	Cost = EEnvTestCost::Low;
	SetWorkOnFloatValues(false);
	bUpdatedToUseQuery = false;

	// To search for GameplayTags, currently we require the item type to be an actor.  Certainly it must at least be a
	// class of some sort to be able to find the interface required.
	ValidItemType = UEnvQueryItemType_ActorBase::StaticClass();
	
	bRejectIncompatibleItems = false;
}

void UEnvQueryTest_GameplayTags::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	// If this is a new object, it's already converted to new format.  If it was old, this should've already been set to
	// true in PostLoad.  This prevents us from erroneously trying to import old data multiple times and so obliterating
	// current data.
	bUpdatedToUseQuery = true;

	Super::PreSave(ObjectSaveContext);
}

void UEnvQueryTest_GameplayTags::PostLoad()
{	// Backwards compatability with old format.  Converts to new query setup.
	if (!bUpdatedToUseQuery)
	{
		bUpdatedToUseQuery = true;

		// This should never happen except if we are loading from an old format version of the file, but just in case,
		// make sure the query is empty.  If it's not, we don't want to overwrite it with old data.
		if (TagQueryToMatch.IsEmpty())
		{
			FGameplayTagQueryExpression TagExpression;
			switch  (TagsToMatch)
			{
				case EGameplayContainerMatchType::All:
					TagExpression.AllTagsMatch();
					break;

				case EGameplayContainerMatchType::Any:
					TagExpression.AnyTagsMatch();
					break;

				default:
					check(false); // TODO: Fix with message
					break;
			}

			TagExpression.AddTags(GameplayTags);
			TagQueryToMatch = FGameplayTagQuery::BuildQuery(TagExpression);

			// Clear out old data so we don't keep serializing unneeded junk.
			GameplayTags.Reset();
		}
	}

	Super::PostLoad();
}

bool UEnvQueryTest_GameplayTags::SatisfiesTest(const IGameplayTagAssetInterface* ItemGameplayTagAssetInterface) const
{
	// Currently we're requiring that the test only be run on items that implement the interface.  In theory, we could
	// (instead of checking) support correctly passing or failing on items that don't implement the interface or
	// just have a nullptr item by testing them as though they have the interface with no gameplay tags.  However, that
	// seems potentially confusing, since certain tests could actually pass.
	check(ItemGameplayTagAssetInterface != nullptr);

	FGameplayTagContainer OwnedGameplayTags;
	ItemGameplayTagAssetInterface->GetOwnedGameplayTags(OwnedGameplayTags);

	return OwnedGameplayTags.MatchesQuery(TagQueryToMatch);
}

void UEnvQueryTest_GameplayTags::RunTest(FEnvQueryInstance& QueryInstance) const
{
	UObject* QueryOwner = QueryInstance.Owner.Get();
	if (QueryOwner == nullptr)
	{
		return;
	}

	BoolValue.BindData(QueryOwner, QueryInstance.QueryID);
	bool bWantsValid = BoolValue.GetValue();

	// If no GameplayTagAssetInterface is found then defer to the user preferred behavior
	const EEnvItemStatus::Type IncompatibleStatus = bRejectIncompatibleItems ? EEnvItemStatus::Failed : EEnvItemStatus::Passed;

	// loop through all items
	for (FEnvQueryInstance::ItemIterator It(this, QueryInstance); It; ++It)
	{
		const AActor* ItemActor = GetItemActor(QueryInstance, It.GetIndex());
		const IGameplayTagAssetInterface* GameplayTagAssetInterface = Cast<const IGameplayTagAssetInterface>(ItemActor);
		if (bSearchComponentsForTags && GameplayTagAssetInterface == nullptr)
		{
			GameplayTagAssetInterface = ItemActor->FindComponentByInterface<const IGameplayTagAssetInterface>();
		}

		if (GameplayTagAssetInterface)
		{
			bool bSatisfiesTest = SatisfiesTest(GameplayTagAssetInterface);

			// bWantsValid is the basically the opposite of bInverseCondition in BTDecorator.  Possibly we should
			// rename to make these more consistent.
			It.SetScore(TestPurpose, FilterType, bSatisfiesTest, bWantsValid);
		}
		else 
		{
			It.ForceItemState(IncompatibleStatus);
		}
	}
}

FText UEnvQueryTest_GameplayTags::GetDescriptionDetails() const
{
	return FText::FromString(TagQueryToMatch.GetDescription());
}

void UEnvQueryTest_GameplayTags::SetTagQueryToMatch(FGameplayTagQuery& GameplayTagQuery)
{
	TagQueryToMatch = GameplayTagQuery;
}

