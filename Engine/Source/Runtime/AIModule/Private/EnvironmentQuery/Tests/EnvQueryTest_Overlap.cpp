// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Tests/EnvQueryTest_Overlap.h"
#include "UObject/Package.h"
#include "CollisionQueryParams.h"
#include "WorldCollision.h"
#include "Engine/World.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_VectorBase.h"
#include "EnvironmentQuery/Contexts/EnvQueryContext_Querier.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryTest_Overlap)

#define LOCTEXT_NAMESPACE "EnvQueryGenerator"

UEnvQueryTest_Overlap::UEnvQueryTest_Overlap(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	Cost = EEnvTestCost::High;
	ValidItemType = UEnvQueryItemType_VectorBase::StaticClass();
	SetWorkOnFloatValues(false);
}

void UEnvQueryTest_Overlap::RunTest(FEnvQueryInstance& QueryInstance) const
{
	UObject* DataOwner = QueryInstance.Owner.Get();
	BoolValue.BindData(DataOwner, QueryInstance.QueryID);

	bool bWantsHit = BoolValue.GetValue();
	
	FCollisionQueryParams OverlapParams(SCENE_QUERY_STAT(EnvQueryOverlap), OverlapData.bOverlapComplex);
	
	const ECollisionChannel OverlapCollisionChannel = OverlapData.OverlapChannel;
	const FVector TraceExtent(OverlapData.ExtentX, OverlapData.ExtentY, OverlapData.ExtentZ);

	FCollisionShape CollisionShape;
	switch (OverlapData.OverlapShape)
	{
		case EEnvOverlapShape::Box:
			CollisionShape = FCollisionShape::MakeBox(TraceExtent);
			break;

		case EEnvOverlapShape::Sphere:
			CollisionShape = FCollisionShape::MakeSphere(FloatCastChecked<float>(TraceExtent.X, UE::LWC::DefaultFloatPrecision));
			break;

		case EEnvOverlapShape::Capsule:
			CollisionShape = FCollisionShape::MakeCapsule(FloatCastChecked<float>(TraceExtent.X, UE::LWC::DefaultFloatPrecision), FloatCastChecked<float>(TraceExtent.Z, UE::LWC::DefaultFloatPrecision));
			break;

		default:
			return;
	}

	typedef bool(UEnvQueryTest_Overlap::*FRunOverlapSignature)(const FVector&, const FCollisionShape&, const TArray<AActor*>&, UWorld*, enum ECollisionChannel, const FCollisionQueryParams&) const;
	FRunOverlapSignature OverlapFunc = (OverlapData.bOnlyBlockingHits ? (FRunOverlapSignature)&UEnvQueryTest_Overlap::RunOverlapBlocking : (FRunOverlapSignature)&UEnvQueryTest_Overlap::RunOverlap);

	TArray<AActor*> IgnoredActors;
	if (OverlapData.bSkipOverlapQuerier)
	{
		QueryInstance.PrepareContext(UEnvQueryContext_Querier::StaticClass(), IgnoredActors);
	}

	for (FEnvQueryInstance::ItemIterator It(this, QueryInstance); It; ++It)
	{
		const FVector ItemLocation = GetItemLocation(QueryInstance, It.GetIndex());
		AActor* ItemActor = GetItemActor(QueryInstance, It.GetIndex());
		IgnoredActors.Push(ItemActor);

		const bool bHit = (this->*OverlapFunc)(ItemLocation + OverlapData.ShapeOffset, CollisionShape, IgnoredActors, QueryInstance.World, OverlapCollisionChannel, OverlapParams);
		It.SetScore(TestPurpose, FilterType, bHit, bWantsHit);
		
		IgnoredActors.Pop(EAllowShrinking::No);
	}
}


FText UEnvQueryTest_Overlap::GetDescriptionTitle() const
{
	UEnum* ShapeEnum = StaticEnum<EEnvOverlapShape::Type>();
	FString ShapeDesc = ShapeEnum->GetDisplayNameTextByValue(OverlapData.OverlapShape).ToString();

	return FText::FromString(FString::Printf(TEXT("%s: %s"), 
		*Super::GetDescriptionTitle().ToString(), *ShapeDesc));
}

FText UEnvQueryTest_Overlap::GetDescriptionDetails() const
{
	UEnum* ShapeEnum = StaticEnum<EEnvOverlapShape::Type>();
	FString ShapeDesc = ShapeEnum->GetDisplayNameTextByValue(OverlapData.OverlapShape).ToString();

	UEnum* ChannelEnum = StaticEnum<ECollisionChannel>();
	FString ChannelDesc = ChannelEnum->GetDisplayNameTextByValue(OverlapData.OverlapChannel).ToString();

	FString SizeDesc;
	switch (OverlapData.OverlapShape)
	{
		case EEnvOverlapShape::Box:
			SizeDesc = FString::Printf(TEXT("X = %f, Y = %f, Z = %f"), OverlapData.ExtentX, OverlapData.ExtentY, OverlapData.ExtentZ);
			break;

		case EEnvOverlapShape::Sphere:
			SizeDesc = FString::Printf(TEXT("Radius = %f"), OverlapData.ExtentX);
			break;

		case EEnvOverlapShape::Capsule:
			SizeDesc = FString::Printf(TEXT("Width = %f, Height = %f"), OverlapData.ExtentX, OverlapData.ExtentZ);
			break;

		default:
			SizeDesc = FString("Invalid.");
			break;
	}

	return FText::FromString(FString::Printf(TEXT("Using a %s where %s in channel: %s"), *ShapeDesc, *SizeDesc, *ChannelDesc));
}

bool UEnvQueryTest_Overlap::RunOverlap(const FVector& ItemPos, const FCollisionShape& CollisionShape, const TArray<AActor*>& IgnoredActors, UWorld* World, enum ECollisionChannel Channel, const FCollisionQueryParams& Params) const
{
	FCollisionQueryParams OverlapParams(Params);
	OverlapParams.AddIgnoredActors(IgnoredActors);

	const bool bHit = World->OverlapAnyTestByChannel(ItemPos, FQuat::Identity, Channel, CollisionShape, OverlapParams);
	return bHit;
}

bool UEnvQueryTest_Overlap::RunOverlapBlocking(const FVector& ItemPos, const FCollisionShape& CollisionShape, const TArray<AActor*>& IgnoredActors, UWorld* World, enum ECollisionChannel Channel, const FCollisionQueryParams& Params) const
{
	FCollisionQueryParams OverlapParams(Params);
	OverlapParams.AddIgnoredActors(IgnoredActors);

	const bool bHit = World->OverlapBlockingTestByChannel(ItemPos, FQuat::Identity, Channel, CollisionShape, OverlapParams);
	return bHit;
}

#undef LOCTEXT_NAMESPACE

