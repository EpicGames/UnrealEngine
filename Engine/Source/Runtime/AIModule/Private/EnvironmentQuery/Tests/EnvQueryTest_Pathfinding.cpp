// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Tests/EnvQueryTest_Pathfinding.h"
#include "AI/Navigation/NavAgentInterface.h"
#include "Engine/World.h"
#include "NavigationData.h"
#include "NavigationSystem.h"
#include "NavFilters/NavigationQueryFilter.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_VectorBase.h"
#include "EnvironmentQuery/Contexts/EnvQueryContext_Querier.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryTest_Pathfinding)

#define LOCTEXT_NAMESPACE "EnvQueryGenerator"

UEnvQueryTest_Pathfinding::UEnvQueryTest_Pathfinding(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	Context = UEnvQueryContext_Querier::StaticClass();
	Cost = EEnvTestCost::High;
	ValidItemType = UEnvQueryItemType_VectorBase::StaticClass();
	TestMode = EEnvTestPathfinding::PathExist;
	PathFromContext.DefaultValue = true;
	SkipUnreachable.DefaultValue = true;
	FloatValueMin.DefaultValue = 1000.0f;
	FloatValueMax.DefaultValue = 1000.0f;

	SetWorkOnFloatValues(TestMode != EEnvTestPathfinding::PathExist);
}

void UEnvQueryTest_Pathfinding::RunTest(FEnvQueryInstance& QueryInstance) const
{
	UObject* QueryOwner = QueryInstance.Owner.Get();
	BoolValue.BindData(QueryOwner, QueryInstance.QueryID);
	PathFromContext.BindData(QueryOwner, QueryInstance.QueryID);
	SkipUnreachable.BindData(QueryOwner, QueryInstance.QueryID);
	FloatValueMin.BindData(QueryOwner, QueryInstance.QueryID);
	FloatValueMax.BindData(QueryOwner, QueryInstance.QueryID);

	bool bWantsPath = BoolValue.GetValue();
	bool bPathToItem = PathFromContext.GetValue();
	bool bDiscardFailed = SkipUnreachable.GetValue();
	float MinThresholdValue = FloatValueMin.GetValue();
	float MaxThresholdValue = FloatValueMax.GetValue();

	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(QueryInstance.World);
	if (NavSys == nullptr || QueryOwner == nullptr)
	{
		return;
	}

	const ANavigationData* NavData = FEQSHelpers::FindNavigationDataForQuery(QueryInstance, NavDataOverrideContext);
	if (NavData == nullptr)
	{
		return;
	}

	TArray<FVector> ContextLocations;
	if (!QueryInstance.PrepareContext(Context, ContextLocations))
	{
		return;
	}

	EPathFindingMode::Type PFMode(EPathFindingMode::Regular);
	FSharedConstNavQueryFilter NavFilter = UNavigationQueryFilter::GetQueryFilter(*NavData, QueryOwner, GetNavFilterClass(QueryInstance));

	if (GetWorkOnFloatValues())
	{
		FFindPathSignature FindPathFunc;
		FindPathFunc.BindUObject(this, TestMode == EEnvTestPathfinding::PathLength ?
			(bPathToItem ? &UEnvQueryTest_Pathfinding::FindPathLengthTo : &UEnvQueryTest_Pathfinding::FindPathLengthFrom) :
			(bPathToItem ? &UEnvQueryTest_Pathfinding::FindPathCostTo : &UEnvQueryTest_Pathfinding::FindPathCostFrom) );

		NavData->BeginBatchQuery();
		for (FEnvQueryInstance::ItemIterator It(this, QueryInstance); It; ++It)
		{
			const FVector ItemLocation = GetItemLocation(QueryInstance, It.GetIndex());
			for (int32 ContextIndex = 0; ContextIndex < ContextLocations.Num(); ContextIndex++)
			{
				const float PathValue = FindPathFunc.Execute(ItemLocation, ContextLocations[ContextIndex], PFMode, *NavData, *NavSys, NavFilter, QueryOwner);
				It.SetScore(TestPurpose, FilterType, PathValue, MinThresholdValue, MaxThresholdValue);

				if (bDiscardFailed && PathValue >= BIG_NUMBER)
				{
					It.ForceItemState(EEnvItemStatus::Failed);
				}
			}
		}
		NavData->FinishBatchQuery();
	}
	else
	{
		NavData->BeginBatchQuery();
		if (bPathToItem)
		{
			for (FEnvQueryInstance::ItemIterator It(this, QueryInstance); It; ++It)
			{
				const FVector ItemLocation = GetItemLocation(QueryInstance, It.GetIndex());
				for (int32 ContextIndex = 0; ContextIndex < ContextLocations.Num(); ContextIndex++)
				{
					const bool bFoundPath = TestPathTo(ItemLocation, ContextLocations[ContextIndex], PFMode, *NavData, *NavSys, NavFilter, QueryOwner);
					It.SetScore(TestPurpose, FilterType, bFoundPath, bWantsPath);
				}
			}
		}
		else
		{
			for (FEnvQueryInstance::ItemIterator It(this, QueryInstance); It; ++It)
			{
				const FVector ItemLocation = GetItemLocation(QueryInstance, It.GetIndex());
				for (int32 ContextIndex = 0; ContextIndex < ContextLocations.Num(); ContextIndex++)
				{
					const bool bFoundPath = TestPathFrom(ItemLocation, ContextLocations[ContextIndex], PFMode, *NavData, *NavSys, NavFilter, QueryOwner);
					It.SetScore(TestPurpose, FilterType, bFoundPath, bWantsPath);
				}
			}
		}
		NavData->FinishBatchQuery();
	}
}

FText UEnvQueryTest_Pathfinding::GetDescriptionTitle() const
{
	FString ModeDesc[] = { TEXT("PathExist"), TEXT("PathCost"), TEXT("PathLength") };

	FString DirectionDesc = PathFromContext.IsDynamic() ?
		FString::Printf(TEXT("%s, direction: %s"), *UEnvQueryTypes::DescribeContext(Context).ToString(), *PathFromContext.ToString()) :
		FString::Printf(TEXT("%s %s"), PathFromContext.DefaultValue ? TEXT("from") : TEXT("to"), *UEnvQueryTypes::DescribeContext(Context).ToString());

	return FText::FromString(FString::Printf(TEXT("%s: %s"), *ModeDesc[TestMode], *DirectionDesc));
}

FText UEnvQueryTest_Pathfinding::GetDescriptionDetails() const
{
	// When in EEnvTestPurpose::Score mode, this test will not discard unreachable items, but rather score them as 0.
	FText DiscardOrScoreDesc = (TestPurpose == EEnvTestPurpose::Score)
		? LOCTEXT("ScoreUnreachable", "score unreachable as 0")
		: LOCTEXT("DiscardUnreachable", "discard unreachable");
	
	FText Desc2;
	if (SkipUnreachable.IsDynamic())
	{
		Desc2 = FText::Format(FText::FromString("{0}: {1}"), DiscardOrScoreDesc, FText::FromString(SkipUnreachable.ToString()));
	}
	else if (SkipUnreachable.DefaultValue)
	{
		Desc2 = MoveTemp(DiscardOrScoreDesc);
	}

	FText TestParamDesc = GetWorkOnFloatValues() ? DescribeFloatTestParams() : DescribeBoolTestParams("existing path");
	if (!Desc2.IsEmpty())
	{
		return FText::Format(FText::FromString("{0}\n{1}"), Desc2, TestParamDesc);
	}

	return TestParamDesc;
}

#if WITH_EDITOR
void UEnvQueryTest_Pathfinding::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UEnvQueryTest_Pathfinding,TestMode))
	{
		SetWorkOnFloatValues(TestMode != EEnvTestPathfinding::PathExist);
	}
}
#endif

void UEnvQueryTest_Pathfinding::PostLoad()
{
	Super::PostLoad();
	
	SetWorkOnFloatValues(TestMode != EEnvTestPathfinding::PathExist);
}

bool UEnvQueryTest_Pathfinding::TestPathFrom(const FVector& ItemPos, const FVector& ContextPos, EPathFindingMode::Type Mode, const ANavigationData& NavData, UNavigationSystemV1& NavSys, FSharedConstNavQueryFilter NavFilter, const UObject* PathOwner) const
{
	FPathFindingQuery Query(PathOwner, NavData, ItemPos, ContextPos, NavFilter);
	Query.SetAllowPartialPaths(false);

	const bool bPathExists = NavSys.TestPathSync(Query, Mode);
	return bPathExists;
}

bool UEnvQueryTest_Pathfinding::TestPathTo(const FVector& ItemPos, const FVector& ContextPos, EPathFindingMode::Type Mode, const ANavigationData& NavData, UNavigationSystemV1& NavSys, FSharedConstNavQueryFilter NavFilter, const UObject* PathOwner) const
{
	FPathFindingQuery Query(PathOwner, NavData, ContextPos, ItemPos, NavFilter);
	Query.SetAllowPartialPaths(false);

	const bool bPathExists = NavSys.TestPathSync(Query, Mode);
	return bPathExists;
}

float UEnvQueryTest_Pathfinding::FindPathCostFrom(const FVector& ItemPos, const FVector& ContextPos, EPathFindingMode::Type Mode, const ANavigationData& NavData, UNavigationSystemV1& NavSys, FSharedConstNavQueryFilter NavFilter, const UObject* PathOwner) const
{
	FPathFindingQuery Query(PathOwner, NavData, ItemPos, ContextPos, NavFilter);
	Query.SetAllowPartialPaths(false);

	FPathFindingResult Result = NavSys.FindPathSync(Query, Mode);
	// Static cast this to a float, for EQS scoring purposes float precision is OK.
	return (Result.IsSuccessful()) ? static_cast<float>(Result.Path->GetCost()) : BIG_NUMBER;
}

float UEnvQueryTest_Pathfinding::FindPathCostTo(const FVector& ItemPos, const FVector& ContextPos, EPathFindingMode::Type Mode, const ANavigationData& NavData, UNavigationSystemV1& NavSys, FSharedConstNavQueryFilter NavFilter, const UObject* PathOwner) const
{
	FPathFindingQuery Query(PathOwner, NavData, ContextPos, ItemPos, NavFilter);
	Query.SetAllowPartialPaths(false);

	FPathFindingResult Result = NavSys.FindPathSync(Query, Mode);
	// Static cast this to a float, for EQS scoring purposes float precision is OK.
	return (Result.IsSuccessful()) ? static_cast<float>(Result.Path->GetCost()) : BIG_NUMBER;
}

float UEnvQueryTest_Pathfinding::FindPathLengthFrom(const FVector& ItemPos, const FVector& ContextPos, EPathFindingMode::Type Mode, const ANavigationData& NavData, UNavigationSystemV1& NavSys, FSharedConstNavQueryFilter NavFilter, const UObject* PathOwner) const
{
	FPathFindingQuery Query(PathOwner, NavData, ItemPos, ContextPos, NavFilter);
	Query.SetAllowPartialPaths(false);

	FPathFindingResult Result = NavSys.FindPathSync(Query, Mode);
	// Static cast this to a float, for EQS scoring purposes float precision is OK.
	return (Result.IsSuccessful()) ? static_cast<float>(Result.Path->GetLength()) : BIG_NUMBER;
}

float UEnvQueryTest_Pathfinding::FindPathLengthTo(const FVector& ItemPos, const FVector& ContextPos, EPathFindingMode::Type Mode, const ANavigationData& NavData, UNavigationSystemV1& NavSys, FSharedConstNavQueryFilter NavFilter, const UObject* PathOwner) const
{
	FPathFindingQuery Query(PathOwner, NavData, ContextPos, ItemPos, NavFilter);
	Query.SetAllowPartialPaths(false);

	FPathFindingResult Result = NavSys.FindPathSync(Query, Mode);
	// Static cast this to a float, for EQS scoring purposes float precision is OK.
	return (Result.IsSuccessful()) ? static_cast<float>(Result.Path->GetLength()) : BIG_NUMBER;
}

// Deprecated
ANavigationData* UEnvQueryTest_Pathfinding::FindNavigationData(UNavigationSystemV1& NavSys, UObject* Owner) const
{
	INavAgentInterface* NavAgent = Cast<INavAgentInterface>(Owner);
	if (NavAgent)
	{
		return NavSys.GetNavDataForProps(NavAgent->GetNavAgentPropertiesRef(), NavAgent->GetNavAgentLocation());
	}

	return NavSys.GetDefaultNavDataInstance(FNavigationSystem::DontCreate);
}

TSubclassOf<UNavigationQueryFilter> UEnvQueryTest_Pathfinding::GetNavFilterClass(FEnvQueryInstance& QueryInstance) const
{
	return FilterClass;
}

#undef LOCTEXT_NAMESPACE

