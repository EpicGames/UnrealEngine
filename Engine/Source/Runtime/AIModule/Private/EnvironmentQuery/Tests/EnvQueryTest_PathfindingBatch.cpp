// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Tests/EnvQueryTest_PathfindingBatch.h"
#include "Engine/World.h"
#include "NavigationSystem.h"
#include "NavFilters/NavigationQueryFilter.h"
#include "NavMesh/RecastNavMesh.h"
#include "NavMesh/RecastQueryFilter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryTest_PathfindingBatch)

#define LOCTEXT_NAMESPACE "EnvQueryGenerator"

UEnvQueryTest_PathfindingBatch::UEnvQueryTest_PathfindingBatch(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	ScanRangeMultiplier.DefaultValue = 1.5f;
}

#if WITH_RECAST
namespace NodePoolHelpers
{
	static bool HasPath(const FRecastDebugPathfindingData& NodePool, const FNavigationProjectionWork& TestPt)
	{
		FRecastDebugPathfindingNode SearchKey(TestPt.OutLocation.NodeRef);
		const FRecastDebugPathfindingNode* MyNode = NodePool.Nodes.Find(SearchKey);
		return MyNode != nullptr;
	}

	static float GetPathLength(const FRecastDebugPathfindingData& NodePool, const FNavigationProjectionWork& TestPt, const dtQueryFilter* Filter)
	{
		FRecastDebugPathfindingNode SearchKey(TestPt.OutLocation.NodeRef);
		const FRecastDebugPathfindingNode* MyNode = NodePool.Nodes.Find(SearchKey);
		if (MyNode)
		{
			const FVector::FReal LastSegmentLength = FVector::Dist(MyNode->NodePos, TestPt.OutLocation.Location);
			// Static cast this to a float, for EQS scoring purposes float precision is OK.
			return static_cast<float>(MyNode->Length + LastSegmentLength);
		}

		return BIG_NUMBER;
	}

	static float GetPathCost(const FRecastDebugPathfindingData& NodePool, const FNavigationProjectionWork& TestPt, const dtQueryFilter* Filter)
	{
		FRecastDebugPathfindingNode SearchKey(TestPt.OutLocation.NodeRef);
		const FRecastDebugPathfindingNode* MyNode = NodePool.Nodes.Find(SearchKey);
		if (MyNode)
		{
			// Static cast this to a float, for EQS scoring purposes float precision is OK.
			return static_cast<float>(MyNode->TotalCost);
		}

		return BIG_NUMBER;
	}

	typedef float(*PathParamFunc)(const FRecastDebugPathfindingData&, const FNavigationProjectionWork&, const dtQueryFilter*);
}

void UEnvQueryTest_PathfindingBatch::RunTest(FEnvQueryInstance& QueryInstance) const
{
	UObject* QueryOwner = QueryInstance.Owner.Get();
	BoolValue.BindData(QueryOwner, QueryInstance.QueryID);
	PathFromContext.BindData(QueryOwner, QueryInstance.QueryID);
	SkipUnreachable.BindData(QueryOwner, QueryInstance.QueryID);
	FloatValueMin.BindData(QueryOwner, QueryInstance.QueryID);
	FloatValueMax.BindData(QueryOwner, QueryInstance.QueryID);
	ScanRangeMultiplier.BindData(QueryOwner, QueryInstance.QueryID);

	bool bWantsPath = BoolValue.GetValue();
	bool bPathToItem = PathFromContext.GetValue();
	bool bDiscardFailed = SkipUnreachable.GetValue();
	float MinThresholdValue = FloatValueMin.GetValue();
	float MaxThresholdValue = FloatValueMax.GetValue();
	float RangeMultiplierValue = ScanRangeMultiplier.GetValue();

	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(QueryInstance.World);
	if (NavSys == nullptr || QueryOwner == nullptr)
	{
		return;
	}

	const ANavigationData* NavData = FEQSHelpers::FindNavigationDataForQuery(QueryInstance, NavDataOverrideContext);
	const ARecastNavMesh* NavMeshData = Cast<ARecastNavMesh>(NavData);
	if (NavMeshData == nullptr)
	{
		return;
	}

	TArray<FVector> ContextLocations;
	if (!QueryInstance.PrepareContext(Context, ContextLocations))
	{
		return;
	}

	TArray<FNavigationProjectionWork> TestPoints;
	TArray<FVector::FReal> CollectDistanceSq;
	CollectDistanceSq.Init(0.0f, ContextLocations.Num());

	TSubclassOf<UNavigationQueryFilter> NavFilterToUse = GetNavFilterClass(QueryInstance);

	FSharedNavQueryFilter NavigationFilterCopy = NavFilterToUse ?
		UNavigationQueryFilter::GetQueryFilter(*NavMeshData, QueryOwner, NavFilterToUse)->GetCopy() :
		NavMeshData->GetDefaultQueryFilter()->GetCopy();

	NavigationFilterCopy->SetBacktrackingEnabled(!bPathToItem);
	const dtQueryFilter* NavQueryFilter = ((const FRecastQueryFilter*)NavigationFilterCopy->GetImplementation())->GetAsDetourQueryFilter();

	{
		// scope for perf timers

		// can't use FEnvQueryInstance::ItemIterator yet, since it has built in scoring functionality
		for (int32 ItemIdx = 0; ItemIdx < QueryInstance.Items.Num(); ItemIdx++)
		{
			if (QueryInstance.Items[ItemIdx].IsValid())
			{
				const FVector ItemLocation = GetItemLocation(QueryInstance, ItemIdx);
				TestPoints.Add(FNavigationProjectionWork(ItemLocation));

				for (int32 ContextIdx = 0; ContextIdx < ContextLocations.Num(); ContextIdx++)
				{
					const FVector::FReal TestDistanceSq = FVector::DistSquared(ItemLocation, ContextLocations[ContextIdx]);
					CollectDistanceSq[ContextIdx] = FMath::Max(CollectDistanceSq[ContextIdx], TestDistanceSq);
				}
			}
		}

		NavMeshData->BatchProjectPoints(TestPoints, NavMeshData->GetDefaultQueryExtent(), NavigationFilterCopy);
	}

	TArray<FRecastDebugPathfindingData> NodePoolData;
	NodePoolData.SetNum(ContextLocations.Num());

	{
		// scope for perf timer

		TArray<NavNodeRef> Polys;
		for (int32 ContextIdx = 0; ContextIdx < ContextLocations.Num(); ContextIdx++)
		{
			const FVector::FReal MaxPathDistance = FMath::Sqrt(CollectDistanceSq[ContextIdx]) * RangeMultiplierValue;

			Polys.Reset();
			NodePoolData[ContextIdx].Flags = ERecastDebugPathfindingFlags::PathLength;

			NavMeshData->GetPolysWithinPathingDistance(ContextLocations[ContextIdx], MaxPathDistance, Polys, NavigationFilterCopy, nullptr, &NodePoolData[ContextIdx]);
		}
	}

	int32 ProjectedItemIdx = 0;
	if (GetWorkOnFloatValues())
	{
		NodePoolHelpers::PathParamFunc Func[] = { nullptr, NodePoolHelpers::GetPathCost, NodePoolHelpers::GetPathLength };
		FEnvQueryInstance::ItemIterator It(this, QueryInstance);
		for (It.IgnoreTimeLimit(); It; ++It, ProjectedItemIdx++)
		{
			for (int32 ContextIndex = 0; ContextIndex < ContextLocations.Num(); ContextIndex++)
			{
				const float PathValue = Func[TestMode](NodePoolData[ContextIndex], TestPoints[ProjectedItemIdx], NavQueryFilter);
				It.SetScore(TestPurpose, FilterType, PathValue, MinThresholdValue, MaxThresholdValue);

				if (bDiscardFailed && PathValue >= BIG_NUMBER)
				{
					It.ForceItemState(EEnvItemStatus::Failed);
				}
			}
		}
	}
	else
	{
		FEnvQueryInstance::ItemIterator It(this, QueryInstance);
		for (It.IgnoreTimeLimit(); It; ++It, ProjectedItemIdx++)
		{
			for (int32 ContextIndex = 0; ContextIndex < ContextLocations.Num(); ContextIndex++)
			{
				const bool bFoundPath = NodePoolHelpers::HasPath(NodePoolData[ContextIndex], TestPoints[ProjectedItemIdx]);
				It.SetScore(TestPurpose, FilterType, bFoundPath, bWantsPath);
			}
		}
	}
}

#else

void UEnvQueryTest_PathfindingBatch::RunTest(FEnvQueryInstance& QueryInstance) const
{
	// can't do anything without navmesh
}

#endif

FText UEnvQueryTest_PathfindingBatch::GetDescriptionTitle() const
{
	return FText::Format(LOCTEXT("PathfindingBatch","{0} [batch]"), Super::GetDescriptionTitle());
}

#undef LOCTEXT_NAMESPACE

