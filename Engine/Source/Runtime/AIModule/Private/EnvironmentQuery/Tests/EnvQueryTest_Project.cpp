// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Tests/EnvQueryTest_Project.h"
#include "AI/Navigation/NavigationTypes.h"
#include "NavFilters/NavigationQueryFilter.h"
#include "NavigationData.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_VectorBase.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_Point.h"
#include "EnvironmentQuery/EnvQueryTraceHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryTest_Project)

UEnvQueryTest_Project::UEnvQueryTest_Project(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	Cost = EEnvTestCost::Medium;
	ValidItemType = UEnvQueryItemType_VectorBase::StaticClass();
	SetWorkOnFloatValues(false);

	ProjectionData.TraceMode = EEnvQueryTrace::Navigation;
	ProjectionData.bCanProjectDown = true;
	ProjectionData.bCanDisableTrace = false;
	ProjectionData.ExtentX = 0.0f;
}

void UEnvQueryTest_Project::RunTest(FEnvQueryInstance& QueryInstance) const
{
	UObject* QueryOwner = QueryInstance.Owner.Get();
	if (QueryOwner == nullptr)
	{
		return;
	}

	BoolValue.BindData(QueryOwner, QueryInstance.QueryID);
	bool bWantsProjected = BoolValue.GetValue();

	// item type: Point = can test, can modify
	// item type: Actor/VectorBase = can only test

	UEnvQueryItemType_Point* ItemTypeCDO = QueryInstance.ItemType && QueryInstance.ItemType->IsChildOf(UEnvQueryItemType_Point::StaticClass()) ?
		QueryInstance.ItemType->GetDefaultObject<UEnvQueryItemType_Point>() : nullptr;

	if (ProjectionData.TraceMode == EEnvQueryTrace::Navigation)
	{
		const ANavigationData* NavData = FEQSHelpers::FindNavigationDataForQuery(QueryInstance);
		if (NavData)
		{
			FSharedConstNavQueryFilter NavigationFilter = UNavigationQueryFilter::GetQueryFilter(*NavData, QueryOwner, ProjectionData.NavigationFilter);
			TArray<FNavigationProjectionWork> Workload;
			Workload.Reserve(QueryInstance.Items.Num());

			const FVector VerticalOffset = (ProjectionData.ProjectDown == ProjectionData.ProjectUp) ?
				FVector::ZeroVector : FVector(0, 0, (ProjectionData.ProjectUp - ProjectionData.ProjectDown) / 2);

			for (int32 Idx = 0; Idx < QueryInstance.Items.Num(); Idx++)
			{
				if (QueryInstance.Items[Idx].IsValid())
				{
					const FVector& ItemLocation = GetItemLocation(QueryInstance, Idx);
					Workload.Add(FNavigationProjectionWork(ItemLocation + VerticalOffset));
				}
			}

			const FVector ProjectionExtent(ProjectionData.ExtentX, ProjectionData.ExtentX, (ProjectionData.ProjectDown + ProjectionData.ProjectUp) / 2);
			NavData->BatchProjectPoints(Workload, ProjectionExtent, NavigationFilter);

			int32 Idx = 0;
			if (ProjectionData.PostProjectionVerticalOffset == 0.f)
			{
				for (FEnvQueryInstance::ItemIterator It(this, QueryInstance); It; ++It, Idx++)
				{
					const bool bProjected = Workload[Idx].bResult;
					if (bProjected && ItemTypeCDO)
					{
						ItemTypeCDO->SetItemNavLocation(It.GetItemData(), Workload[Idx].OutLocation);
					}
					It.SetScore(TestPurpose, FilterType, bProjected, bWantsProjected);
				}
			}
			else
			{
				const FVector PostProjectionVerticalOffset(0.f, 0.f, ProjectionData.PostProjectionVerticalOffset);
				for (FEnvQueryInstance::ItemIterator It(this, QueryInstance); It; ++It, Idx++)
				{
					const bool bProjected = Workload[Idx].bResult;
					if (bProjected && ItemTypeCDO)
					{
						ItemTypeCDO->SetItemNavLocation(It.GetItemData(), FNavLocation(Workload[Idx].OutLocation.Location + PostProjectionVerticalOffset, Workload[Idx].OutLocation.NodeRef));
					}
					It.SetScore(TestPurpose, FilterType, bProjected, bWantsProjected);
				}
			}
		}
	}
	else if (ProjectionData.TraceMode == EEnvQueryTrace::GeometryByChannel || ProjectionData.TraceMode == EEnvQueryTrace::GeometryByProfile)
	{
		TArray<FNavLocation> Workload;
		TArray<uint8> TraceHits;

		Workload.Reserve(QueryInstance.Items.Num());
		for (int32 Idx = 0; Idx < QueryInstance.Items.Num(); Idx++)
		{
			if (QueryInstance.Items[Idx].IsValid())
			{
				const FVector& ItemLocation = GetItemLocation(QueryInstance, Idx);
				Workload.Add(FNavLocation(ItemLocation));
			}
		}

		FEQSHelpers::RunPhysProjection(QueryInstance.World, ProjectionData, Workload, TraceHits);

		int32 Idx = 0;
		for (FEnvQueryInstance::ItemIterator It(this, QueryInstance); It; ++It, Idx++)
		{
			const bool bProjected = TraceHits.IsValidIndex(Idx) && TraceHits[Idx];
			if (bProjected && ItemTypeCDO)
			{
				ItemTypeCDO->SetItemNavLocation(It.GetItemData(), Workload[Idx]);
			}

			It.SetScore(TestPurpose, FilterType, bProjected, bWantsProjected);
		}
	}
}

FText UEnvQueryTest_Project::GetDescriptionDetails() const
{
	return DescribeBoolTestParams(TEXT("projected"));
}

