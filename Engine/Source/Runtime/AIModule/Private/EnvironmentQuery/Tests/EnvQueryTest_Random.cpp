// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Tests/EnvQueryTest_Random.h"
#include "AISystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryTest_Random)

UEnvQueryTest_Random::UEnvQueryTest_Random(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	Cost = EEnvTestCost::Low;

	// "Random" is completely agnostic on item type!  Any type at all is valid!
	ValidItemType = UEnvQueryItemType::StaticClass();
}

void UEnvQueryTest_Random::RunTest(FEnvQueryInstance& QueryInstance) const
{
	UObject* QueryOwner = QueryInstance.Owner.Get();
	if (QueryOwner == nullptr)
	{
		return;
	}

	FloatValueMin.BindData(QueryOwner, QueryInstance.QueryID);
	float MinThresholdValue = FloatValueMin.GetValue();

	FloatValueMax.BindData(QueryOwner, QueryInstance.QueryID);
	float MaxThresholdValue = FloatValueMax.GetValue();

	// loop through all items
	for (FEnvQueryInstance::ItemIterator It(this, QueryInstance); It; ++It)
	{
		It.SetScore(TestPurpose, FilterType, UAISystem::GetRandomStream().FRand(), MinThresholdValue, MaxThresholdValue);
	}
}

FText UEnvQueryTest_Random::GetDescriptionDetails() const
{
	return DescribeFloatTestParams();
}

