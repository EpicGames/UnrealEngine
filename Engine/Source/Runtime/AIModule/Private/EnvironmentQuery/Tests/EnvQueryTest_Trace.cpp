// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvironmentQuery/Tests/EnvQueryTest_Trace.h"
#include "UObject/Package.h"
#include "CollisionQueryParams.h"
#include "WorldCollision.h"
#include "Engine/World.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_VectorBase.h"
#include "EnvironmentQuery/Contexts/EnvQueryContext_Querier.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryTest_Trace)

#define LOCTEXT_NAMESPACE "EnvQueryGenerator"

namespace EnvQueryTest_Trace_Helpers
{
	template <EEnvTraceShape::Type Shape>
	bool TraceShape(const FVector& Start, const FVector& End, UWorld* World, enum ECollisionChannel Channel, const FCollisionQueryParams& TraceParams, const FVector3f& Extent, const FCollisionResponseParams& ResponseParams)
	{
		if constexpr (Shape == EEnvTraceShape::Line)
		{
			return World->LineTraceTestByChannel(Start, End, Channel, TraceParams, ResponseParams);
		}
		else if constexpr (Shape == EEnvTraceShape::Box)
		{
			return World->SweepTestByChannel(Start, End, FQuat((End - Start).Rotation()), Channel, FCollisionShape::MakeBox(Extent), TraceParams, ResponseParams);
		}
		else if constexpr (Shape == EEnvTraceShape::Sphere)
		{
			return World->SweepTestByChannel(Start, End, FQuat::Identity, Channel, FCollisionShape::MakeSphere(Extent.X), TraceParams, ResponseParams);
		}
		else if constexpr (Shape == EEnvTraceShape::Capsule)
		{
			return World->SweepTestByChannel(Start, End, FQuat::Identity, Channel, FCollisionShape::MakeCapsule(Extent.X, Extent.Z), TraceParams, ResponseParams);
		}
		else
		{
			[] <bool cond = false>() { static_assert(cond, "Unsupported value of EEnvTraceShape received in TraceShape"); }(); // static_assert must be type-dependent to avoid "ill-formed" code
			return false;
		}
	}

	template <bool bTraceToItem, EEnvTraceShape::Type Shape>
	bool TraceShapeWithDir(const FVector& ItemPos, const FVector& ContextPos, UWorld* World, enum ECollisionChannel Channel, const FCollisionQueryParams& TraceParams, const FVector3f& Extent, const FCollisionResponseParams& ResponseParams)
	{
		if constexpr (bTraceToItem)
		{
			return TraceShape<Shape>(ContextPos, ItemPos, World, Channel, TraceParams, Extent, ResponseParams);
		}
		else
		{
			return TraceShape<Shape>(ItemPos, ContextPos, World, Channel, TraceParams, Extent, ResponseParams);
		}
	}

	// Use a templated function execute the traces in order to avoid using any branch or function pointer inside the item loop
	template <bool bTraceToItem, EEnvTraceShape::Type Shape>
	void RunTraces(const UEnvQueryTest_Trace& Query, const TArrayView<FVector>& ContextLocations, FEnvQueryInstance& QueryInstance, float ContextZ, float ItemZ, const FEnvTraceData& TraceData, EEnvTestPurpose::Type TestPurpose, EEnvTestFilterType::Type FilterType, bool bWantsHit, const FCollisionQueryParams& TraceParams)
	{
		ECollisionChannel TraceCollisionChannel = ECC_WorldStatic;
		FCollisionResponseParams ResponseParams = FCollisionResponseParams::DefaultResponseParam;
		if (TraceData.TraceMode == EEnvQueryTrace::Type::GeometryByProfile)
		{
			if (!UCollisionProfile::GetChannelAndResponseParams(TraceData.TraceProfileName, TraceCollisionChannel, ResponseParams))
			{
				UE_VLOG_UELOG(QueryInstance.Owner.Get(), LogEQS, Error, 
					TEXT("Unable to fetch collision channel and response from TraceProfileName %s, test %s for query %s will automatically fail"),
					*TraceData.TraceProfileName.ToString(),
					*Query.GetName(),
					*QueryInstance.QueryName);

				for (FEnvQueryInstance::ItemIterator It(&Query, QueryInstance); It; ++It)
				{
					It.SetScore(TestPurpose, FilterType, !bWantsHit, bWantsHit);
				}

				return;
			}
		}
		else if (TraceData.TraceMode == EEnvQueryTrace::Type::GeometryByChannel)
		{
			TraceCollisionChannel = UEngineTypes::ConvertToCollisionChannel(TraceData.TraceChannel);
		}

		FVector3f TraceExtent(TraceData.ExtentX, TraceData.ExtentY, TraceData.ExtentZ);

		for (int32 ContextIndex = 0; ContextIndex < ContextLocations.Num(); ContextIndex++)
		{
			ContextLocations[ContextIndex].Z += ContextZ;
		}

		for (FEnvQueryInstance::ItemIterator It(&Query, QueryInstance); It; ++It)
		{
			const FVector ItemLocation = Query.GetItemLocation(QueryInstance, It.GetIndex()) + FVector(0, 0, ItemZ);
			AActor* ItemActor = Query.GetItemActor(QueryInstance, It.GetIndex());

			FCollisionQueryParams PerItemTraceParams(TraceParams);
			PerItemTraceParams.AddIgnoredActor(ItemActor);

			for (int32 ContextIndex = 0; ContextIndex < ContextLocations.Num(); ContextIndex++)
			{
				const bool bHit = TraceShapeWithDir<bTraceToItem, Shape>(ItemLocation, ContextLocations[ContextIndex], QueryInstance.World, TraceCollisionChannel, PerItemTraceParams, TraceExtent, ResponseParams);
				It.SetScore(TestPurpose, FilterType, bHit, bWantsHit);
			}
		}
	}
}

UEnvQueryTest_Trace::UEnvQueryTest_Trace(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	Cost = EEnvTestCost::High;
	ValidItemType = UEnvQueryItemType_VectorBase::StaticClass();
	SetWorkOnFloatValues(false);
	
	Context = UEnvQueryContext_Querier::StaticClass();
	TraceData.SetGeometryOnly();
}

void UEnvQueryTest_Trace::RunTest(FEnvQueryInstance& QueryInstance) const
{
	UObject* DataOwner = QueryInstance.Owner.Get();
	BoolValue.BindData(DataOwner, QueryInstance.QueryID);
	TraceFromContext.BindData(DataOwner, QueryInstance.QueryID);
	ItemHeightOffset.BindData(DataOwner, QueryInstance.QueryID);
	ContextHeightOffset.BindData(DataOwner, QueryInstance.QueryID);

	bool bWantsHit = BoolValue.GetValue();
	bool bTraceToItem = TraceFromContext.GetValue();
	float ItemZ = ItemHeightOffset.GetValue();
	float ContextZ = ContextHeightOffset.GetValue();

	TArray<FVector> ContextLocations;
	if (!QueryInstance.PrepareContext(Context, ContextLocations))
	{
		return;
	}

	FCollisionQueryParams TraceParams(SCENE_QUERY_STAT(EnvQueryTrace), TraceData.bTraceComplex);

	TArray<AActor*> IgnoredActors;
	if (QueryInstance.PrepareContext(Context, IgnoredActors))
	{
		TraceParams.AddIgnoredActors(IgnoredActors);
	}

	switch (TraceData.TraceShape)
	{
	case EEnvTraceShape::Line:
		if (bTraceToItem)
		{
			EnvQueryTest_Trace_Helpers::RunTraces<true, EEnvTraceShape::Line>(*this, ContextLocations, QueryInstance, ContextZ, ItemZ, TraceData, TestPurpose, FilterType, bWantsHit, TraceParams);
		}
		else
		{
			EnvQueryTest_Trace_Helpers::RunTraces<false, EEnvTraceShape::Line>(*this, ContextLocations, QueryInstance, ContextZ, ItemZ, TraceData, TestPurpose, FilterType, bWantsHit, TraceParams);
		}
		break;
	case EEnvTraceShape::Box:
		if (bTraceToItem)
		{
			EnvQueryTest_Trace_Helpers::RunTraces<true, EEnvTraceShape::Box>(*this, ContextLocations, QueryInstance, ContextZ, ItemZ, TraceData, TestPurpose, FilterType, bWantsHit, TraceParams);
		}
		else
		{
			EnvQueryTest_Trace_Helpers::RunTraces<false, EEnvTraceShape::Box>(*this, ContextLocations, QueryInstance, ContextZ, ItemZ, TraceData, TestPurpose, FilterType, bWantsHit, TraceParams);
		}
		break;
	case EEnvTraceShape::Sphere:
		if (bTraceToItem)
		{
			EnvQueryTest_Trace_Helpers::RunTraces<true, EEnvTraceShape::Sphere>(*this, ContextLocations, QueryInstance, ContextZ, ItemZ, TraceData, TestPurpose, FilterType, bWantsHit, TraceParams);
		}
		else
		{
			EnvQueryTest_Trace_Helpers::RunTraces<false, EEnvTraceShape::Sphere>(*this, ContextLocations, QueryInstance, ContextZ, ItemZ, TraceData, TestPurpose, FilterType, bWantsHit, TraceParams);
		}
		break;
	case EEnvTraceShape::Capsule:
		if (bTraceToItem)
		{
			EnvQueryTest_Trace_Helpers::RunTraces<true, EEnvTraceShape::Capsule>(*this, ContextLocations, QueryInstance, ContextZ, ItemZ, TraceData, TestPurpose, FilterType, bWantsHit, TraceParams);
		}
		else
		{
			EnvQueryTest_Trace_Helpers::RunTraces<false, EEnvTraceShape::Capsule>(*this, ContextLocations, QueryInstance, ContextZ, ItemZ, TraceData, TestPurpose, FilterType, bWantsHit, TraceParams);
		}
		break;
	}
}

void UEnvQueryTest_Trace::PostLoad()
{
	Super::PostLoad();
	TraceData.OnPostLoad();
}

FText UEnvQueryTest_Trace::GetDescriptionTitle() const
{
	UEnum* ChannelEnum = StaticEnum<ETraceTypeQuery>();
	FString ChannelDesc = ChannelEnum->GetDisplayNameTextByValue(TraceData.TraceChannel).ToString();

	FString DirectionDesc = TraceFromContext.IsDynamic() ?
		FString::Printf(TEXT("%s, direction: %s"), *UEnvQueryTypes::DescribeContext(Context).ToString(), *TraceFromContext.ToString()) :
		FString::Printf(TEXT("%s %s"), TraceFromContext.DefaultValue ? TEXT("from") : TEXT("to"), *UEnvQueryTypes::DescribeContext(Context).ToString());

	return FText::FromString(FString::Printf(TEXT("%s: %s on %s"), 
		*Super::GetDescriptionTitle().ToString(), *DirectionDesc, *ChannelDesc));
}

FText UEnvQueryTest_Trace::GetDescriptionDetails() const
{
	return FText::Format(FText::FromString("{0}\n{1}"),
		TraceData.ToText(FEnvTraceData::Detailed), DescribeBoolTestParams("hit"));
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

bool UEnvQueryTest_Trace::RunLineTraceTo(const FVector& ItemPos, const FVector& ContextPos, AActor* ItemActor, UWorld* World, enum ECollisionChannel Channel, const FCollisionQueryParams& Params, const FVector& Extent)
{
	FCollisionQueryParams TraceParams(Params);
	TraceParams.AddIgnoredActor(ItemActor);

	const bool bHit = World->LineTraceTestByChannel(ContextPos, ItemPos, Channel, TraceParams);
	return bHit;
}

bool UEnvQueryTest_Trace::RunLineTraceFrom(const FVector& ItemPos, const FVector& ContextPos, AActor* ItemActor, UWorld* World, enum ECollisionChannel Channel, const FCollisionQueryParams& Params, const FVector& Extent)
{
	FCollisionQueryParams TraceParams(Params);
	TraceParams.AddIgnoredActor(ItemActor);

	const bool bHit = World->LineTraceTestByChannel(ItemPos, ContextPos, Channel, TraceParams);
	return bHit;
}

bool UEnvQueryTest_Trace::RunBoxTraceTo(const FVector& ItemPos, const FVector& ContextPos, AActor* ItemActor, UWorld* World, enum ECollisionChannel Channel, const FCollisionQueryParams& Params, const FVector& Extent)
{
	FCollisionQueryParams TraceParams(Params);
	TraceParams.AddIgnoredActor(ItemActor);

	const bool bHit = World->SweepTestByChannel(ContextPos, ItemPos, FQuat((ItemPos - ContextPos).Rotation()), Channel, FCollisionShape::MakeBox(Extent), TraceParams);
	return bHit;
}

bool UEnvQueryTest_Trace::RunBoxTraceFrom(const FVector& ItemPos, const FVector& ContextPos, AActor* ItemActor, UWorld* World, enum ECollisionChannel Channel, const FCollisionQueryParams& Params, const FVector& Extent)
{
	FCollisionQueryParams TraceParams(Params);
	TraceParams.AddIgnoredActor(ItemActor);

	const bool bHit = World->SweepTestByChannel(ItemPos, ContextPos, FQuat((ContextPos - ItemPos).Rotation()), Channel, FCollisionShape::MakeBox(Extent), TraceParams);
	return bHit;
}

bool UEnvQueryTest_Trace::RunSphereTraceTo(const FVector& ItemPos, const FVector& ContextPos, AActor* ItemActor, UWorld* World, enum ECollisionChannel Channel, const FCollisionQueryParams& Params, const FVector& Extent)
{
	FCollisionQueryParams TraceParams(Params);
	TraceParams.AddIgnoredActor(ItemActor);

	const bool bHit = World->SweepTestByChannel(ContextPos, ItemPos, FQuat::Identity, Channel, FCollisionShape::MakeSphere(FloatCastChecked<float>(Extent.X, UE::LWC::DefaultFloatPrecision)), TraceParams);
	return bHit;
}

bool UEnvQueryTest_Trace::RunSphereTraceFrom(const FVector& ItemPos, const FVector& ContextPos, AActor* ItemActor, UWorld* World, enum ECollisionChannel Channel, const FCollisionQueryParams& Params, const FVector& Extent)
{
	FCollisionQueryParams TraceParams(Params);
	TraceParams.AddIgnoredActor(ItemActor);

	const bool bHit = World->SweepTestByChannel(ItemPos, ContextPos, FQuat::Identity, Channel, FCollisionShape::MakeSphere(FloatCastChecked<float>(Extent.X, UE::LWC::DefaultFloatPrecision)), TraceParams);
	return bHit;
}

bool UEnvQueryTest_Trace::RunCapsuleTraceTo(const FVector& ItemPos, const FVector& ContextPos, AActor* ItemActor, UWorld* World, enum ECollisionChannel Channel, const FCollisionQueryParams& Params, const FVector& Extent)
{
	FCollisionQueryParams TraceParams(Params);
	TraceParams.AddIgnoredActor(ItemActor);

	const bool bHit = World->SweepTestByChannel(ContextPos, ItemPos, FQuat::Identity, Channel, FCollisionShape::MakeCapsule(FloatCastChecked<float>(Extent.X, UE::LWC::DefaultFloatPrecision), FloatCastChecked<float>(Extent.Z, UE::LWC::DefaultFloatPrecision)), TraceParams);
	return bHit;
}

bool UEnvQueryTest_Trace::RunCapsuleTraceFrom(const FVector& ItemPos, const FVector& ContextPos, AActor* ItemActor, UWorld* World, enum ECollisionChannel Channel, const FCollisionQueryParams& Params, const FVector& Extent)
{
	FCollisionQueryParams TraceParams(Params);
	TraceParams.AddIgnoredActor(ItemActor);

	const bool bHit = World->SweepTestByChannel(ItemPos, ContextPos, FQuat::Identity, Channel, FCollisionShape::MakeCapsule(FloatCastChecked<float>(Extent.X, UE::LWC::DefaultFloatPrecision), FloatCastChecked<float>(Extent.Z, UE::LWC::DefaultFloatPrecision)), TraceParams);
	return bHit;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE

