// Copyright Epic Games, Inc. All Rights Reserved.
#include "EnvironmentQuery/Tests/EnvQueryTest_Volume.h"

#include "EngineUtils.h"
#include "EnvironmentQuery/Items/EnvQueryItemType_VectorBase.h"
#include "GameFramework/Volume.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnvQueryTest_Volume)

#define LOCTEXT_NAMESPACE "EnvQueryGenerator"

UEnvQueryTest_Volume::UEnvQueryTest_Volume(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	Cost = EEnvTestCost::Medium;
	ValidItemType = UEnvQueryItemType_VectorBase::StaticClass();
	SetWorkOnFloatValues(false);

	bDoComplexVolumeTest = false;
	bSkipTestIfNoVolumes = false;
}

#if WITH_EDITOR
void UEnvQueryTest_Volume::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	Cost = VolumeContext.Get() ? EEnvTestCost::Medium : EEnvTestCost::High;
}
#endif

void UEnvQueryTest_Volume::RunTest(FEnvQueryInstance& QueryInstance) const
{
	const UObject* DataOwner = QueryInstance.Owner.Get();
	BoolValue.BindData(DataOwner, QueryInstance.QueryID);

	struct FVolumeBoundsCache
	{
		public:
			FVolumeBoundsCache(AVolume& InVolume) : Volume(InVolume)
			{
				BoundingBox = Volume.GetBounds().GetBox();
			}

			bool IsPointInsideBoundingBox(const FVector TestPoint) const
			{
				return BoundingBox.IsInside(TestPoint);
			}
			bool EncompassesPoint(const FVector TestPoint) const
			{
				return Volume.EncompassesPoint(TestPoint);
			}

		private:
			AVolume& Volume;
			FBox BoundingBox;
	};

	TArray<AActor*> ContextVolumeActors;
	TArray<FVolumeBoundsCache> ContextVolumes;
	if (VolumeContext)
	{
		QueryInstance.PrepareContext(VolumeContext, ContextVolumeActors);
		ContextVolumes.Reserve(ContextVolumeActors.Num());
		for (AActor* VolumeActor : ContextVolumeActors)
		{
			AVolume* Volume = Cast<AVolume>(VolumeActor);
			if (Volume)
			{			
				ContextVolumes.Add(FVolumeBoundsCache(*Volume));
			}
		}

	}
	else if (VolumeClass)
	{
		for (TActorIterator<AVolume> It(QueryInstance.World, VolumeClass); It; ++It)
		{
			AVolume* Volume = *It;
			if (Volume)
			{
				ContextVolumes.Add(FVolumeBoundsCache(*Volume));
			}
		}
	}

	const bool bWantsInside = BoolValue.GetValue();	

	// Initially Items are marked as "passed", so in principle we only need to run the code below if there are volumes 
	// to test and we're not allowed to skip this test.
	// We can however also skip if there are no volumes, but bWantsInside == false since the code below would only result
	// in assigning "passed" to all Items, which is the default state, as described.
	if (ContextVolumes.Num() > 0 || (bSkipTestIfNoVolumes == false && bWantsInside == true))
	{
		for (FEnvQueryInstance::ItemIterator It(this, QueryInstance); It; ++It)
		{
			const FVector ItemLocation = GetItemLocation(QueryInstance, It.GetIndex());
			bool bPointIsInsideAVolume = false;
			for (const FVolumeBoundsCache& VolumeAndBounds : ContextVolumes)
			{
				if (VolumeAndBounds.IsPointInsideBoundingBox(ItemLocation))
				{
					if (!bDoComplexVolumeTest || VolumeAndBounds.EncompassesPoint(ItemLocation))
					{
						bPointIsInsideAVolume = true;
						break;
					}
				}
			}
			It.SetScore(TestPurpose, FilterType, bPointIsInsideAVolume, bWantsInside);
		}
	}
}

FText UEnvQueryTest_Volume::GetDescriptionTitle() const
{	
	FString VolumeDesc("No Source Volume Context/Class set");
	if (VolumeContext)
	{
		VolumeDesc = UEnvQueryTypes::DescribeContext(VolumeContext).ToString();
	}
	else if (VolumeClass)
	{
		VolumeDesc = VolumeClass->GetDescription();
	}

	return FText::FromString(FString::Printf(TEXT("%s : %s (%s)"), 
		*Super::GetDescriptionTitle().ToString(), *VolumeDesc, bDoComplexVolumeTest ? TEXT("Complex Physic Volume Test") : TEXT("Simple Bounding Box Test")));
}

FText UEnvQueryTest_Volume::GetDescriptionDetails() const
{
	return GetDescriptionTitle();
}

#undef LOCTEXT_NAMESPACE

