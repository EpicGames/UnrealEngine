// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebugger/GameplayDebuggerCategory_AI.h"

#if WITH_GAMEPLAY_DEBUGGER_MENU

#include "GameFramework/Pawn.h"
#include "ShowFlags.h"
#include "PrimitiveViewRelevance.h"
#include "Components/PrimitiveComponent.h"
#include "NavigationSystem.h"
#include "GameFramework/PlayerController.h"
#include "Engine/Engine.h"
#include "EngineGlobals.h"
#include "Navigation/PathFollowingComponent.h"
#include "NavMesh/RecastNavMesh.h"
#include "Engine/Canvas.h"
#include "AIController.h"
#include "BehaviorTree/BehaviorTreeComponent.h"
#include "Engine/Texture2D.h"
#include "DynamicMeshBuilder.h"
#include "DebugRenderSceneProxy.h"
#include "GameplayTasksComponent.h"
#include "GameFramework/Character.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "DrawDebugHelpers.h"
#include "EngineUtils.h"

namespace FGameplayDebuggerCategoryTweakables
{
	int32 bDrawOverheadIcons = 1;
	bool bDrawPaths = false;
}

namespace
{
	FAutoConsoleVariableRef CVars_GameplayDebuggerCategory_AI[] = {
		FAutoConsoleVariableRef(TEXT("ai.debug.DrawOverheadIcons"), FGameplayDebuggerCategoryTweakables::bDrawOverheadIcons, TEXT("Should default AI overhead icons be drawn"), ECVF_Default),
		FAutoConsoleVariableRef(TEXT("ai.debug.DrawPaths"), FGameplayDebuggerCategoryTweakables::bDrawPaths, TEXT("Should AI paths be drawn"), ECVF_Default)
	};
}


FGameplayDebuggerCategory_AI::FGameplayDebuggerCategory_AI()
{
	bShowOnlyWithDebugActor = false;
	RawLastPath = nullptr;
	LastPathUpdateTime = 0.;

	SetDataPackReplication<FRepData>(&DataPack);
	PathDataPackId = SetDataPackReplication<FRepDataPath>(&PathDataPack, EGameplayDebuggerDataPack::ResetOnActorChange);
}

TSharedRef<FGameplayDebuggerCategory> FGameplayDebuggerCategory_AI::MakeInstance()
{
	return MakeShareable(new FGameplayDebuggerCategory_AI());
}

void FGameplayDebuggerCategory_AI::FRepData::Serialize(FArchive& Ar)
{
	Ar << ControllerName;
	Ar << PawnName;
	Ar << MovementBaseInfo;
	Ar << MovementModeInfo;
	Ar << PathFollowingInfo;
	Ar << NextPathPointIndex;
	Ar << PathGoalLocation;
	Ar << CurrentAITask;
	Ar << CurrentAIState;
	Ar << CurrentAIAssets;
	Ar << NavDataInfo;
	Ar << MontageInfo;
	Ar << TaskQueueInfo;
	Ar << TickingTaskInfo;
	Ar << NumTasksInQueue;
	Ar << NumTickingTasks;

	uint32 BitFlags =
		((bIsUsingPathFollowing ? 1 : 0) << 0) |
		((bIsUsingCharacter ? 1 : 0) << 1) |
		((bIsUsingBehaviorTree ? 1 : 0) << 2) |
		((bIsUsingGameplayTasks ? 1 : 0) << 3) |
		((bPathHasGoalActor ? 1 : 0) << 4) |
		((bHasController ? 1 : 0) << 5);

	Ar << BitFlags;

	bIsUsingPathFollowing = (BitFlags & (1 << 0)) != 0;
	bIsUsingCharacter = (BitFlags & (1 << 1)) != 0;
	bIsUsingBehaviorTree = (BitFlags & (1 << 2)) != 0;
	bIsUsingGameplayTasks = (BitFlags & (1 << 3)) != 0;
	bPathHasGoalActor = (BitFlags & (1 << 4)) != 0;
	bHasController = (BitFlags & (1 << 5)) != 0;
}

void FGameplayDebuggerCategory_AI::FRepDataPath::Serialize(FArchive& Ar)
{
	int32 NumCorridor = PathCorridor.Num();
	Ar << NumCorridor;
	if (Ar.IsLoading())
	{
		PathCorridor.SetNum(NumCorridor);
	}

	for (int32 Idx = 0; Idx < NumCorridor; Idx++)
	{
		Ar << PathCorridor[Idx].Points;
		Ar << PathCorridor[Idx].Color;
	}

	Ar << PathPoints;
}

static FString DescribeTaskHelper(const UGameplayTask& TaskOb)
{
	const UObject* OwnerOb = Cast<const UObject>(TaskOb.GetTaskOwner());
#if ENABLE_VISUAL_LOG
	FString StateName = *TaskOb.GetTaskStateName();
#else
	const TCHAR* StateName = TEXT("UnknownTaskStateName");
#endif
	return FString::Printf(TEXT("\n  {white}%s%s {%s}%s:%d {white}Owner:{yellow}%s {white}Res:{yellow}%s"),
		*TaskOb.GetName(),
		TaskOb.GetInstanceName() != NAME_None ? *FString::Printf(TEXT(" {yellow}[%s]"), *TaskOb.GetInstanceName().ToString()) : TEXT(""),
		TaskOb.IsActive() ? TEXT("green") : TEXT("orange"),
		ToCStr(StateName),
		TaskOb.GetPriority(),
		*GetNameSafe(OwnerOb),
		TaskOb.GetRequiredResources().IsEmpty() ? TEXT("None") : *TaskOb.GetRequiredResources().GetDebugDescription());
}

void FGameplayDebuggerCategory_AI::CollectData(APlayerController* OwnerPC, AActor* DebugActor)
{
	APawn* MyPawn = Cast<APawn>(DebugActor);
	ACharacter* MyChar = Cast<ACharacter>(MyPawn);
	DataPack.PawnName = MyPawn ? MyPawn->GetHumanReadableName() : FString("{red}No selected pawn.");
	DataPack.bIsUsingCharacter = (MyChar != nullptr);

	AAIController* MyController = MyPawn ? Cast<AAIController>(MyPawn->GetController()) : nullptr;
	DataPack.bHasController = (MyController != nullptr);
	if (MyController)
	{
		if (IsValid(MyController))
		{
			DataPack.ControllerName = MyController->GetName();
		}
		else
		{
			DataPack.ControllerName = TEXT("Controller PENDING KILL");
		}
	}
	else
	{
		DataPack.ControllerName = TEXT("No Controller");
	}

	if (MyPawn && IsValidChecked(MyPawn))
	{
		UCharacterMovementComponent* CharMovementComp = MyChar ? MyChar->GetCharacterMovement() : nullptr;
		if (CharMovementComp)
		{
			const FMovementBaseInterfaceData* MovementBaseData = MyPawn->GetMovementBaseInterfaceData();
			const UObject* FloorObject = MovementBaseData && MovementBaseData->IsValid() ? MovementBaseData->GetMovementBaseObjectOwner() : nullptr;
			DataPack.MovementBaseInfo = MovementBaseData && MovementBaseData->GetMovementBaseObject() ? FString::Printf(TEXT("%s.%s"), *GetNameSafe(FloorObject), *MovementBaseData->GetMovementBaseObject()->GetName()) : FString("None");
			DataPack.MovementModeInfo = CharMovementComp->GetMovementName();
		}

		UBehaviorTreeComponent* BehaviorComp = MyController ? Cast<UBehaviorTreeComponent>(MyController->BrainComponent) : nullptr;
		DataPack.bIsUsingBehaviorTree = (BehaviorComp != nullptr);
		if (BehaviorComp)
		{
			DataPack.CurrentAITask = BehaviorComp->DescribeActiveTasks();
			DataPack.CurrentAIState = BehaviorComp->IsRunning() ? TEXT("Running") : BehaviorComp->IsPaused() ? TEXT("Paused") : TEXT("Inactive");
			DataPack.CurrentAIAssets = BehaviorComp->DescribeActiveTrees();
		}

		UGameplayTasksComponent* TasksComponent = MyController ? MyController->GetGameplayTasksComponent() : nullptr;
		DataPack.bIsUsingGameplayTasks = (TasksComponent != nullptr);
		if (TasksComponent)
		{
			for (FConstGameplayTaskIterator It = TasksComponent->GetTickingTaskIterator(); It; ++It)
			{
				const UGameplayTask* TaskOb = *It;
				if (TaskOb)
				{
					DataPack.TickingTaskInfo += DescribeTaskHelper(*TaskOb);
					DataPack.NumTickingTasks++;
				}
			}

			for (FConstGameplayTaskIterator It = TasksComponent->GetPriorityQueueIterator(); It; ++It)
			{
				const UGameplayTask* TaskOb = *It;
				if (TaskOb)
				{
					DataPack.TaskQueueInfo += DescribeTaskHelper(*TaskOb);
					DataPack.NumTasksInQueue++;
				}
			}
		}

		DataPack.MontageInfo = MyChar ? GetNameSafe(MyChar->GetCurrentMontage()) : FString();

		UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(MyPawn->GetWorld());
		const ANavigationData* NavData = MyController && NavSys 
			? NavSys->GetNavDataForProps(MyController->GetNavAgentPropertiesRef(), MyController->GetNavAgentLocation()) 
			: nullptr;
		DataPack.NavDataInfo = NavData ? NavData->GetConfig().Name.ToString() : FString();

		CollectPathData(MyController);
	}
	else
	{
		PathDataPack.PathCorridor.Reset();
		PathDataPack.PathPoints.Reset();
	}
}

void FGameplayDebuggerCategory_AI::CollectPathData(AAIController* DebugAI)
{
	UPathFollowingComponent* PathComp = DebugAI ? DebugAI->GetPathFollowingComponent() : nullptr;
	DataPack.bIsUsingPathFollowing = (PathComp != nullptr);

	if (PathComp)
	{
		TArray<FString> Tokens;
		TArray<EPathFollowingDebugTokens::Type> Flags;
		PathComp->GetDebugStringTokens(Tokens, Flags);

		for (int32 Idx = 0; Idx < Tokens.Num(); Idx++)
		{
			switch (Flags[Idx])
			{
			case EPathFollowingDebugTokens::Description:
				DataPack.PathFollowingInfo += Tokens[Idx];
				break;

			case EPathFollowingDebugTokens::ParamName:
				DataPack.PathFollowingInfo += TEXT(", {yellow}");
				DataPack.PathFollowingInfo += Tokens[Idx];
				DataPack.PathFollowingInfo += TEXT(":");
				break;

			case EPathFollowingDebugTokens::PassedValue:
				DataPack.PathFollowingInfo += TEXT("{yellow}");
				DataPack.PathFollowingInfo += Tokens[Idx];
				break;

			case EPathFollowingDebugTokens::FailedValue:
				DataPack.PathFollowingInfo += TEXT("{orange}");
				DataPack.PathFollowingInfo += Tokens[Idx];
				break;

			default:
				break;
			}
		}

		FNavigationPath* CurrentPath = PathComp->GetPath().Get();
		if (CurrentPath)
		{
			DataPack.bPathHasGoalActor = (CurrentPath->GetGoalActor() != nullptr);
			DataPack.PathGoalLocation = CurrentPath->GetGoalLocation();
			DataPack.NextPathPointIndex = PathComp->GetNextPathIndex();
		}

		if ((CurrentPath != RawLastPath) || (CurrentPath && (CurrentPath->GetLastUpdateTime() != LastPathUpdateTime) ))
		{
			RawLastPath = CurrentPath;
			PathDataPack = FRepDataPath();

			if (CurrentPath)
			{
				LastPathUpdateTime = CurrentPath->GetLastUpdateTime();

#if WITH_RECAST
				const FNavMeshPath* NavMeshPath = CurrentPath->CastPath<FNavMeshPath>();
				const ARecastNavMesh* NavData = Cast<const ARecastNavMesh>(CurrentPath->GetNavigationDataUsed());
				if (NavMeshPath && NavData)
				{
					for (int32 Idx = 0; Idx < NavMeshPath->PathCorridor.Num(); Idx++)
					{
						FRepDataPath::FPoly PolyData;
						NavData->GetPolyVerts(NavMeshPath->PathCorridor[Idx], PolyData.Points);
						
						const uint32 AreaId = NavData->GetPolyAreaID(NavMeshPath->PathCorridor[Idx]);
						PolyData.Color = NavData->GetAreaIDColor(IntCastChecked<uint8>(AreaId));

						PathDataPack.PathCorridor.Add(PolyData);
					}
				}
#endif // WITH_RECAST

				for (int32 Idx = 0; Idx < CurrentPath->GetPathPoints().Num(); Idx++)
				{
					PathDataPack.PathPoints.Add(CurrentPath->GetPathPoints()[Idx].Location);
				}
			}
		}
	}
}

void FGameplayDebuggerCategory_AI::OnDataPackReplicated(int32 DataPackId)
{
	if (DataPackId == PathDataPackId)
	{
		MarkRenderStateDirty();
	}
}

void FGameplayDebuggerCategory_AI::DrawData(APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext)
{
	UWorld* MyWorld = CanvasContext.GetWorld();
	AActor* SelectedActor = FindLocalDebugActor();

	const bool bReducedMode = IsSimulateInEditor();
	bShowCategoryName = !bReducedMode || DataPack.bHasController;

	if (FGameplayDebuggerCategoryTweakables::bDrawOverheadIcons && OwnerPC)
	{
		DrawPawnIcons(MyWorld, SelectedActor, OwnerPC->GetPawn(), CanvasContext);
	}
	
	if (SelectedActor)
	{
		DrawOverheadInfo(*SelectedActor, CanvasContext);
	}

	if (FGameplayDebuggerCategoryTweakables::bDrawPaths && DataPack.bIsUsingPathFollowing)
	{
		DrawPath(MyWorld);
	}

	if (DataPack.bHasController)
	{
		CanvasContext.Printf(TEXT("Controller Name: {yellow}%s"), *DataPack.ControllerName);
		CanvasContext.Printf(TEXT("Pawn Name: {yellow}%s"), *DataPack.PawnName);
	}

	if (DataPack.bIsUsingCharacter)
	{
		const bool bIsStandalone = IsCategoryLocal() && IsCategoryAuth();
		CanvasContext.Printf(TEXT("%sMovement Mode: {yellow}%s{white}, Base: {yellow}%s")
			, bIsStandalone ? TEXT("") : TEXT("(Server) ")
			, *DataPack.MovementModeInfo, *DataPack.MovementBaseInfo);
		
		if (IsCategoryLocal() && !IsCategoryAuth())
		{
			const ACharacter* MyChar = Cast<ACharacter>(SelectedActor);
			// In principle data collection should take place in CollectData, but we don't call that on clients for this category
			// and we do want to display client-side data here as well, so we bend the rules a bit and gather and display data instantly here
			if (const UCharacterMovementComponent* CharMovementComp = MyChar ? MyChar->GetCharacterMovement() : nullptr)
			{
				const FMovementBaseInterfaceData* MovementBaseData = MyChar->GetMovementBaseInterfaceData();
				const UObject* FloorObject = MovementBaseData && MovementBaseData->IsValid() ? MovementBaseData->GetMovementBaseObjectOwner() : nullptr;
				const FString ClientMovementBaseInfo = MovementBaseData && MovementBaseData->GetMovementBaseObject() ? FString::Printf(TEXT("%s.%s"), *GetNameSafe(FloorObject), *MovementBaseData->GetMovementBaseObject()->GetName()) : FString("None");
				const FString ClientMovementModeInfo = CharMovementComp->GetMovementName();

				CanvasContext.Printf(TEXT("(Client) Movement Mode: {yellow}%s{white}, Base: {yellow}%s"), *ClientMovementModeInfo, *ClientMovementBaseInfo);
			}
		}
		
		CanvasContext.Printf(TEXT("NavData: {yellow}%s{white}, Path following: {yellow}%s"), *DataPack.NavDataInfo, *DataPack.PathFollowingInfo);
	}

	if (DataPack.bIsUsingBehaviorTree)
	{
		CanvasContext.Printf(TEXT("Behavior: {yellow}%s{white}, Tree: {yellow}%s"), *DataPack.CurrentAIState, *DataPack.CurrentAIAssets);
		CanvasContext.Printf(TEXT("Active task: {yellow}%s"), *DataPack.CurrentAITask);
	}

	if (DataPack.bIsUsingGameplayTasks)
	{
		if (DataPack.NumTickingTasks > 0)
		{
			CanvasContext.Printf(TEXT("Ticking tasks: {yellow}%d%s"), DataPack.NumTickingTasks, *DataPack.TickingTaskInfo);
		}

		CanvasContext.Printf(TEXT("Gameplay tasks: {yellow}%d%s"), DataPack.NumTasksInQueue, *DataPack.TaskQueueInfo);
	}

	if (DataPack.bIsUsingCharacter)
	{
		CanvasContext.Printf(TEXT("Montage: {yellow}%s"), *DataPack.MontageInfo);
	}
}

FDebugRenderSceneProxy* FGameplayDebuggerCategory_AI::CreateDebugSceneProxy(const UPrimitiveComponent* InComponent, FDebugDrawDelegateHelper*& OutDelegateHelper)
{
	class FPathDebugRenderSceneProxy final : public FDebugRenderSceneProxy
	{
	public:
		SIZE_T GetTypeHash() const override
		{
			static size_t UniquePointer;
			return reinterpret_cast<size_t>(&UniquePointer);
		}

		FPathDebugRenderSceneProxy(const UPrimitiveComponent* InPrimitiveCComponent, const FString &InViewFlagName)
			: FDebugRenderSceneProxy(InPrimitiveCComponent)
		{
			DrawType = FDebugRenderSceneProxy::SolidAndWireMeshes;
			DrawAlpha = 90;
			ViewFlagName = InViewFlagName;
			ViewFlagIndex = uint32(FEngineShowFlags::FindIndexByName(*ViewFlagName));
		}

		FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
		{
			const bool bCanShow = View->Family->EngineShowFlags.GetSingleFlag(ViewFlagIndex);

			FPrimitiveViewRelevance Result;
			Result.bDrawRelevance = Result.bSeparateTranslucency = Result.bNormalTranslucency = bCanShow;
			Result.bDynamicRelevance = true;
			return Result;
		}
	};

	if (PathDataPack.PathCorridor.Num())
	{
		TArray<FDebugRenderSceneProxy::FMesh> Meshes;
		TArray<FDebugRenderSceneProxy::FDebugLine> Lines;

		for (int32 Idx = 0; Idx < PathDataPack.PathCorridor.Num(); Idx++)
		{
			const FRepDataPath::FPoly& Poly = PathDataPack.PathCorridor[Idx];
			if (Poly.Points.Num() > 2)
			{
				FDebugRenderSceneProxy::FMesh PolyMesh;
				PolyMesh.Vertices.Add(FDynamicMeshVertex((FVector3f)Poly.Points[0]));
				PolyMesh.Color = Poly.Color;

				for (int32 VertIdx = 2; VertIdx < Poly.Points.Num(); VertIdx++)
				{
					PolyMesh.Vertices.Add(FDynamicMeshVertex((FVector3f)Poly.Points[VertIdx - 1]));
					PolyMesh.Vertices.Add(FDynamicMeshVertex((FVector3f)Poly.Points[VertIdx]));

					PolyMesh.Indices.Add(0);
					PolyMesh.Indices.Add(PolyMesh.Vertices.Num() - 2);
					PolyMesh.Indices.Add(PolyMesh.Vertices.Num() - 1);
				}

				Meshes.Add(PolyMesh);
			}

			for (int32 VertIdx = 0; VertIdx < Poly.Points.Num(); VertIdx++)
			{
				Lines.Add(FDebugRenderSceneProxy::FDebugLine(
					Poly.Points[VertIdx], Poly.Points[(VertIdx + 1) % Poly.Points.Num()],
					Poly.Color, 2.0f));
			}
		}

		const FString ViewFlagName = GetSceneProxyViewFlag();
		FPathDebugRenderSceneProxy* DebugSceneProxy = new FPathDebugRenderSceneProxy(InComponent, ViewFlagName);
		DebugSceneProxy->Lines = Lines;
		DebugSceneProxy->Meshes = Meshes;

		auto* OutDelegateHelper2 = new FDebugDrawDelegateHelper();
		OutDelegateHelper2->InitDelegateHelper(DebugSceneProxy);
		OutDelegateHelper = OutDelegateHelper2;

		return DebugSceneProxy;
	}

	OutDelegateHelper = nullptr;
	return nullptr;
}

void FGameplayDebuggerCategory_AI::DrawPath(UWorld* World)
{
	static const FColor InactiveColor(100, 100, 100);
	static const FColor PathColor(192, 192, 192);
	static const FColor PathGoalColor(255, 255, 255);

	const int32 NumPathVerts = PathDataPack.PathPoints.Num();
	for (int32 Idx = 0; Idx < NumPathVerts; Idx++)
	{
		const FVector PathPoint = PathDataPack.PathPoints[Idx] + NavigationDebugDrawing::PathOffset;
		DrawDebugSolidBox(World, PathPoint, NavigationDebugDrawing::PathNodeBoxExtent, Idx < DataPack.NextPathPointIndex ? InactiveColor : PathColor);
	}

	for (int32 Idx = 1; Idx < NumPathVerts; Idx++)
	{
		const FVector P0 = PathDataPack.PathPoints[Idx - 1] + NavigationDebugDrawing::PathOffset;
		const FVector P1 = PathDataPack.PathPoints[Idx] + NavigationDebugDrawing::PathOffset;

		DrawDebugLine(World, P0, P1, Idx < DataPack.NextPathPointIndex ? InactiveColor : PathColor, false, -1.0f, 0, NavigationDebugDrawing::PathLineThickness);
	}

	if (NumPathVerts && DataPack.bPathHasGoalActor)
	{
		const FVector P0 = PathDataPack.PathPoints.Last() + NavigationDebugDrawing::PathOffset;
		const FVector P1 = DataPack.PathGoalLocation + NavigationDebugDrawing::PathOffset;

		DrawDebugLine(World, P0, P1, PathGoalColor, false, -1.0f, 0, NavigationDebugDrawing::PathLineThickness);
	}
}

void FGameplayDebuggerCategory_AI::DrawOverheadInfo(AActor& DebugActor, FGameplayDebuggerCanvasContext& CanvasContext)
{
	const FVector OverheadLocation = DebugActor.GetActorLocation() + FVector(0, 0, DebugActor.GetSimpleCollisionHalfHeight());
	if (CanvasContext.IsLocationVisible(OverheadLocation))
	{
		FGameplayDebuggerCanvasContext OverheadContext(CanvasContext);
		OverheadContext.Font = GEngine->GetSmallFont();
		OverheadContext.FontRenderInfo.bEnableShadow = true;

		const FVector2D ScreenLoc = OverheadContext.ProjectLocation(OverheadLocation);
		FString ActorDesc = FString::Printf(TEXT("{yellow}%s {white}%s"), *DataPack.ControllerName, *DataPack.PawnName);

		float SizeX = 0.0f, SizeY = 0.0f;
		OverheadContext.MeasureString(ActorDesc, SizeX, SizeY);
		OverheadContext.PrintAt(static_cast<float>(ScreenLoc.X - (SizeX * 0.5f)), static_cast<float>(ScreenLoc.Y - (SizeY * 1.2f)), ActorDesc);
	}
}

void FGameplayDebuggerCategory_AI::DrawPawnIcons(UWorld* World, AActor* DebugActor, APawn* SkipPawn, FGameplayDebuggerCanvasContext& CanvasContext)
{
	FString FailsafeIcon = "/Engine/EngineResources/AICON-Green.AICON-Green";
	for (const APawn* ItPawn : TActorRange<APawn>(World))
	{
		if (SkipPawn != ItPawn)
		{
			const FVector IconLocation = ItPawn->GetActorLocation() + FVector(0, 0, ItPawn->GetSimpleCollisionHalfHeight());
			const AAIController* ItAI = Cast<const AAIController>(ItPawn->GetController());

			FString DebugIconPath = IsValid(ItAI) ? ItAI->GetDebugIcon() : FailsafeIcon;
			if (CanvasContext.IsLocationVisible(IconLocation) && DebugIconPath.Len())
			{
				UTexture2D* IconTexture = (UTexture2D*)StaticLoadObject(UTexture2D::StaticClass(), NULL, *DebugIconPath, NULL, LOAD_NoWarn | LOAD_Quiet, NULL);
				FCanvasIcon CanvasIcon = UCanvas::MakeIcon(IconTexture);
				if (CanvasIcon.Texture)
				{
					const FVector2D ScreenLoc = CanvasContext.ProjectLocation(IconLocation);
					const float IconSize = (DebugActor == ItPawn) ? 32.0f : 16.0f;

					CanvasContext.DrawIcon(FColor::White, CanvasIcon, static_cast<float>(ScreenLoc.X), static_cast<float>(ScreenLoc.Y - IconSize), IconSize / CanvasIcon.Texture->GetSurfaceWidth());
				}
			}
		}
	}
}

#endif // WITH_GAMEPLAY_DEBUGGER_MENU
