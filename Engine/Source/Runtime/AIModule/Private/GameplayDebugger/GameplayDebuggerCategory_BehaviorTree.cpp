// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebugger/GameplayDebuggerCategory_BehaviorTree.h"

#if WITH_GAMEPLAY_DEBUGGER_MENU

#include "GameFramework/Pawn.h"
#include "BrainComponent.h"
#include "AIController.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "GameplayDebuggerTypes.h"
#include "InputCoreTypes.h"

FGameplayDebuggerCategory_BehaviorTree::FGameplayDebuggerCategory_BehaviorTree()
{
	SetDataPackReplication<FRepData>(&DataPack);

	const FGameplayDebuggerInputHandlerConfig ScrollUpKeyConfig(TEXT("ScrollUp"), EKeys::Subtract.GetFName(), FGameplayDebuggerInputModifier::Shift);
	BindKeyPress(ScrollUpKeyConfig, this, &FGameplayDebuggerCategory_BehaviorTree::ScrollUp);

	const FGameplayDebuggerInputHandlerConfig ScrollDownKeyConfig(TEXT("ScrollDown"), EKeys::Add.GetFName(), FGameplayDebuggerInputModifier::Shift);
	BindKeyPress(ScrollDownKeyConfig, this, &FGameplayDebuggerCategory_BehaviorTree::ScrollDown);
}

TSharedRef<FGameplayDebuggerCategory> FGameplayDebuggerCategory_BehaviorTree::MakeInstance()
{
	return MakeShareable(new FGameplayDebuggerCategory_BehaviorTree());
}

void FGameplayDebuggerCategory_BehaviorTree::FRepData::Serialize(FArchive& Ar)
{
	Ar << CompName;
	Ar << TreeDesc;
	Ar << BlackboardDesc;
}

void FGameplayDebuggerCategory_BehaviorTree::CollectData(APlayerController* OwnerPC, AActor* DebugActor)
{
	APawn* MyPawn = Cast<APawn>(DebugActor);
	AAIController* MyController = MyPawn ? Cast<AAIController>(MyPawn->GetController()) : nullptr;
	UBrainComponent* BrainComp = GetValid(MyController ? MyController->GetBrainComponent() : nullptr);
	
	if (BrainComp)
	{
		DataPack.CompName = BrainComp->GetName();
		DataPack.TreeDesc = BrainComp->GetDebugInfoString();

		if (BrainComp->GetBlackboardComponent())
		{
			DataPack.BlackboardDesc = BrainComp->GetBlackboardComponent()->GetDebugInfoString(EBlackboardDescription::KeyWithValue);
		}
	}
}

void FGameplayDebuggerCategory_BehaviorTree::DrawData(APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext)
{
	if (!DataPack.CompName.IsEmpty())
	{
		// Display options
		CanvasContext.Printf(TEXT("Display: Use {yellow}[Shift+]{white} and {yellow}[Shift-]{white} to scroll the blackboard.\n"));

		CanvasContext.Printf(TEXT("Brain Component: {yellow}%s"), *DataPack.CompName);
		CanvasContext.Print(DataPack.TreeDesc);

		TArray<FString> BlackboardLines;
		DataPack.BlackboardDesc.ParseIntoArrayLines(BlackboardLines, true);

		const float SavedDefX = CanvasContext.DefaultX;
		const float SavedPosY = CanvasContext.CursorY;
		CanvasContext.DefaultX = CanvasContext.CursorX = 600.0f;
		CanvasContext.CursorY = CanvasContext.DefaultY;

		for (int32 Idx = DisplayStartingIndex; Idx < BlackboardLines.Num(); Idx++)
		{
			int32 SeparatorIndex = INDEX_NONE;
			BlackboardLines[Idx].FindChar(TEXT(':'), SeparatorIndex);

			if (SeparatorIndex != INDEX_NONE && Idx)
			{
				FString ColoredLine = BlackboardLines[Idx].Left(SeparatorIndex + 1) + FString("{yellow}") + BlackboardLines[Idx].Mid(SeparatorIndex + 1);
				CanvasContext.Print(ColoredLine);
			}
			else
			{
				CanvasContext.Print(BlackboardLines[Idx]);
			}
		}

		CanvasContext.DefaultX = CanvasContext.CursorX = SavedDefX;
		CanvasContext.CursorY = SavedPosY;
	}
}

void FGameplayDebuggerCategory_BehaviorTree::ScrollUp()
{
	DisplayStartingIndex = FMath::Max(DisplayStartingIndex - PagingOffset, 0);
}

void FGameplayDebuggerCategory_BehaviorTree::ScrollDown()
{
	if (!DataPack.CompName.IsEmpty())
	{
		TArray<FString> BlackboardLines;
		DataPack.BlackboardDesc.ParseIntoArrayLines(BlackboardLines, /* Empty strings are not added to the array */ true);

		DisplayStartingIndex = FMath::Min(DisplayStartingIndex + PagingOffset, BlackboardLines.Num());
	}
}

#endif // WITH_GAMEPLAY_DEBUGGER_MENU
