// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebugger/GameplayDebuggerCategory_EQS.h"
#include "GameFramework/Pawn.h"

#if WITH_GAMEPLAY_DEBUGGER_MENU

#include "GameFramework/PlayerController.h"
#include "GlobalRenderResources.h"
#include "CanvasItem.h"
#include "Engine/Canvas.h"
#include "EnvironmentQuery/EnvQueryManager.h"
#include "EnvironmentQuery/EQSRenderingComponent.h"
#include "DrawDebugHelpers.h"

namespace UE::GameplayDebuggerCategory_EQS::Tweakables
{
	float RefreshInterval = 2.0f;

	FAutoConsoleVariableRef CVars[] =
	{
		FAutoConsoleVariableRef(TEXT("ai.debug.EQS.RefreshInterval"), Tweakables::RefreshInterval,
			TEXT("Interval (in seconds) at which data will be collected.")),
	};
}

FGameplayDebuggerCategory_EQS::FGameplayDebuggerCategory_EQS()
{
	MaxQueries = 5;
	MaxItemTableRows = 10;
	ShownQueryIndex = 0;
	CollectDataInterval = UE::GameplayDebuggerCategory_EQS::Tweakables::RefreshInterval;

	const FGameplayDebuggerInputHandlerConfig CycleConfig(TEXT("CycleQueries"), TEXT("Multiply"));
	const FGameplayDebuggerInputHandlerConfig DetailsConfig(TEXT("ToggleDetails"), TEXT("Divide"));

	BindKeyPress(CycleConfig, this, &FGameplayDebuggerCategory_EQS::CycleShownQueries);
	BindKeyPress(DetailsConfig, this, &FGameplayDebuggerCategory_EQS::ToggleDetailView);

#if USE_EQS_DEBUGGER
	SetDataPackReplication<FRepData>(&DataPack);
#endif
}

TSharedRef<FGameplayDebuggerCategory> FGameplayDebuggerCategory_EQS::MakeInstance()
{
	return MakeShareable(new FGameplayDebuggerCategory_EQS());
}

#if USE_EQS_DEBUGGER
void FGameplayDebuggerCategory_EQS::FRepData::Serialize(FArchive& Ar)
{
	Ar << QueryDebugData;
}
#endif // USE_EQS_DEBUGGER

void FGameplayDebuggerCategory_EQS::CollectData(APlayerController* OwnerPC, AActor* DebugActor)
{
#if USE_EQS_DEBUGGER
	UWorld* World = GetDataWorld(OwnerPC, DebugActor);
	check(World);

	UEnvQueryManager* QueryManager = UEnvQueryManager::GetCurrent(World);
	if (QueryManager == nullptr)
	{
		return;
	}

	TArray<FEQSDebugger::FEnvQueryInfo> AuthQueryData = QueryManager->GetDebugger().GetAllQueriesForOwner(DebugActor);

	APawn* DebugPawn = Cast<APawn>(DebugActor);
	if (DebugPawn && DebugPawn->GetController())
	{
		const TArray<FEQSDebugger::FEnvQueryInfo>& AuthControllerQueryData = QueryManager->GetDebugger().GetAllQueriesForOwner(DebugPawn->GetController());
		AuthQueryData.Append(AuthControllerQueryData);
	}

	struct FEnvQuerySortNewFirst
	{
		bool operator()(const FEQSDebugger::FEnvQueryInfo& A, const FEQSDebugger::FEnvQueryInfo& B) const
		{
			return (A.Timestamp > B.Timestamp);
		}
	};

	AuthQueryData.Sort(FEnvQuerySortNewFirst());

	for (int32 Idx = 0; Idx < AuthQueryData.Num() && DataPack.QueryDebugData.Num() < MaxQueries; Idx++)
	{
		FEnvQueryInstance* QueryInstance = AuthQueryData[Idx].Instance.Get();
		if (QueryInstance)
		{
			EQSDebug::FQueryData DebugItem;
			UEnvQueryDebugHelpers::QueryToDebugData(*QueryInstance, DebugItem, MAX_int32);
			DebugItem.Timestamp = AuthQueryData[Idx].Timestamp;

			DataPack.QueryDebugData.Add(DebugItem);
		}
	}
#endif // USE_EQS_DEBUGGER
}

void FGameplayDebuggerCategory_EQS::OnDataPackReplicated(int32 DataPackId)
{
	MarkRenderStateDirty();

#if USE_EQS_DEBUGGER
	if (!DataPack.QueryDebugData.IsValidIndex(ShownQueryIndex))
	{
		ShownQueryIndex = 0;
	}
#endif
}

FDebugRenderSceneProxy* FGameplayDebuggerCategory_EQS::CreateDebugSceneProxy(const UPrimitiveComponent* InComponent, FDebugDrawDelegateHelper*& OutDelegateHelper)
{
#if USE_EQS_DEBUGGER
	if (DataPack.QueryDebugData.IsValidIndex(ShownQueryIndex) && (DataPack.QueryDebugData[ShownQueryIndex].SolidSpheres.Num() > 0 || DataPack.QueryDebugData[ShownQueryIndex].Texts.Num() > 0) && InComponent)
	{
		const EQSDebug::FQueryData& QueryData = DataPack.QueryDebugData[ShownQueryIndex];
		const FString ViewFlagName = GetSceneProxyViewFlag();
		FEQSSceneProxy* EQSSceneProxy = new FEQSSceneProxy(*InComponent, ViewFlagName, QueryData.SolidSpheres, QueryData.Texts);

		auto* OutDelegateHelper2 = new FEQSRenderingDebugDrawDelegateHelper();
		OutDelegateHelper2->SetupFromProxy(EQSSceneProxy);
		OutDelegateHelper2->InitDelegateHelper(EQSSceneProxy);
		OutDelegateHelper = OutDelegateHelper2;

		return EQSSceneProxy;
	}
#endif // USE_EQS_DEBUGGER

	OutDelegateHelper = nullptr;
	return nullptr;
}

void FGameplayDebuggerCategory_EQS::DrawData(APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext)
{
#if USE_EQS_DEBUGGER
	UWorld* World = CanvasContext.GetWorld();
	if (World == nullptr)
	{
		return;
	}

	const FString HeaderDesc = (DataPack.QueryDebugData.Num() > 1) ?
		FString::Printf(TEXT("Queries: {yellow}%d{white}, press {yellow}[%s]{white} to cycle through"), DataPack.QueryDebugData.Num(), *GetInputHandlerDescription(0)) :
		FString::Printf(TEXT("Queries: {yellow}%d"), DataPack.QueryDebugData.Num());

	CanvasContext.Print(HeaderDesc);
	if (DataPack.QueryDebugData.Num() == 0)
	{
		return;
	}

	for (int32 Idx = 0; Idx < DataPack.QueryDebugData.Num(); Idx++)
	{
		const EQSDebug::FQueryData& QueryData = DataPack.QueryDebugData[Idx];

		CanvasContext.Printf(TEXT("{%s}[%d] %s"), 
			(Idx == ShownQueryIndex) ? *FGameplayDebuggerCanvasStrings::ColorNameEnabled : *FGameplayDebuggerCanvasStrings::ColorNameDisabled,
			QueryData.Id, *QueryData.Name);
	}

	if (DataPack.QueryDebugData.IsValidIndex(ShownQueryIndex))
	{
		const EQSDebug::FQueryData& ShownQueryData = DataPack.QueryDebugData[ShownQueryIndex];

		CanvasContext.MoveToNewLine();
		CanvasContext.Printf(TEXT("Timestamp: {yellow}%.3f (~ %.2fs ago)"), ShownQueryData.Timestamp, World->TimeSince(ShownQueryData.Timestamp));
		
		FString OptionsDesc("Options: ");
		for (int32 Idx = 0; Idx < ShownQueryData.Options.Num(); Idx++)
		{
			OptionsDesc += (Idx == ShownQueryData.UsedOption) ? TEXT("[{green}") : TEXT("[");
			OptionsDesc += ShownQueryData.Options[Idx];
			OptionsDesc += (Idx == ShownQueryData.UsedOption) ? TEXT("{white}] ") : TEXT("] ");
		}
		CanvasContext.Print(OptionsDesc);

		const int32 DebugItemIdx = DrawLookedAtItem(ShownQueryData, OwnerPC, CanvasContext);
		DrawDetailedItemTable(ShownQueryData, DebugItemIdx, CanvasContext);
	}
#else // USE_EQS_DEBUGGER
	CanvasContext.Print(FColor::Red, TEXT("Unable to gather EQS debug data, use build with USE_EQS_DEBUGGER enabled."));
#endif // USE_EQS_DEBUGGER
}

#if USE_EQS_DEBUGGER
int32 FGameplayDebuggerCategory_EQS::DrawLookedAtItem(const EQSDebug::FQueryData& QueryData, APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext) const
{
	UWorld* World = CanvasContext.GetWorld();
	if (CanvasContext.Canvas == nullptr || World == nullptr)
	{
		return INDEX_NONE;
	}

	const FVector CameraLocation = CanvasContext.Canvas->SceneView->ViewMatrices.GetViewOrigin();
	const FVector CameraDirection = CanvasContext.Canvas->SceneView->GetViewDirection();

	int32 BestItemIndex = INDEX_NONE;
	FVector::FReal BestScore = TNumericLimits<FVector::FReal>::Min();

	for (int32 Idx = 0; Idx < QueryData.RenderDebugHelpers.Num(); Idx++)
	{
		const EQSDebug::FDebugHelper& ItemInfo = QueryData.RenderDebugHelpers[Idx];
		const FVector DirToItem = ItemInfo.Location - CameraLocation;
		FVector::FReal DistToItem = DirToItem.Size();
		if (FMath::IsNearlyZero(DistToItem))
		{
			DistToItem = 1.;
		}

		const FVector::FReal ItemScore = FVector::DotProduct(DirToItem, CameraDirection) / DistToItem;
		if (ItemScore > BestScore)
		{
			BestItemIndex = Idx;
			BestScore = ItemScore;
		}
	}

	if (BestItemIndex != INDEX_NONE)
	{
		const EQSDebug::FDebugHelper& DebugHelper = QueryData.RenderDebugHelpers[BestItemIndex];
		DrawDebugSphere(World, DebugHelper.Location, DebugHelper.Radius, 8, FColor::Red);
		DrawDebugCone(World, DebugHelper.Location, FVector(0, 0, 1), 100.0f, 0.1f, 0.1f, 8, FColor::Red);

		const int32 FailedTestIndex = DebugHelper.FailedTestIndex;
		if (FailedTestIndex != INDEX_NONE)
		{
			const float BackgroundPadding = 1.0f;
			FCanvasTileItem DescTileItem(FVector2D(0, 0), GWhiteTexture, FVector2D(CanvasContext.Canvas->SizeX, (CanvasContext.GetLineHeight() * 2) + (BackgroundPadding * 2)), FLinearColor(0, 0, 0, 0.6f));
			DescTileItem.BlendMode = SE_BLEND_Translucent;
			CanvasContext.DrawItem(DescTileItem, 0, CanvasContext.CursorY - BackgroundPadding);

			CanvasContext.Printf(FColor::Red, TEXT("Selected item (#%d, %s) failed test [%d]: {yellow}%s {LightBlue}(%s)"),
				BestItemIndex,
				QueryData.Items.IsValidIndex(BestItemIndex) ? *QueryData.Items[BestItemIndex].Desc : TEXT("INVALID"),
				FailedTestIndex,
				*QueryData.Tests[FailedTestIndex].ShortName,
				*QueryData.Tests[FailedTestIndex].Detailed);

			CanvasContext.Printf(TEXT("\t'%s' with score: %3.3f"), *DebugHelper.AdditionalInformation, DebugHelper.FailedScore);
		}
		else
		{
			CanvasContext.Printf(TEXT("Selected item: {yellow}%d"), BestItemIndex);
			CanvasContext.MoveToNewLine();
		}
	}

	return BestItemIndex;
}

namespace FEQSDebugTable
{
	const float RowHeight = 20.0f;
	const float ItemDescriptionWidth = 312.0f;
	const float ItemScoreWidth = 50.0f;
	const float TestScoreWidth = 100.0f;
}

void FGameplayDebuggerCategory_EQS::DrawDetailedItemTable(const EQSDebug::FQueryData& QueryData, int32 LookedAtItemIndex, FGameplayDebuggerCanvasContext& CanvasContext) const
{
	CanvasContext.Printf(TEXT("Detailed table view: {%s}%s{white}, press {yellow}[%s]{white} to toggle"),
		bShowDetails ? *FGameplayDebuggerCanvasStrings::ColorNameEnabled : *FGameplayDebuggerCanvasStrings::ColorNameDisabled,
		bShowDetails ? TEXT("active") : TEXT("disabled"),
		*GetInputHandlerDescription(1));

	if (!bShowDetails)
	{
		return;
	}

	const float BackgroundPadding = 5.0f;
	FCanvasTileItem TileItem(FVector2D(0, 0), GWhiteTexture, FVector2D(CanvasContext.Canvas->SizeX, FEQSDebugTable::RowHeight), FLinearColor::Black);
	FLinearColor ColorOdd(0, 0, 0, 0.6f);
	FLinearColor ColorEven(0, 0, 0.4f, 0.4f);
	FLinearColor ColorHighlighted(0.2f, 0.2f, 0, 0.4f);
	TileItem.BlendMode = SE_BLEND_Translucent;

	const int32 MaxShownItems = FMath::Min(MaxItemTableRows, QueryData.Items.Num());
	if (!MaxShownItems)
	{
		CanvasContext.CursorY += BackgroundPadding;
		TileItem.SetColor(ColorOdd);
		CanvasContext.DrawItem(TileItem, 0, CanvasContext.CursorY);
		CanvasContext.CursorY += 3.0f;

		CanvasContext.Printf(FColor::Yellow, TEXT("Num items: %d"), QueryData.NumValidItems);
		return;
	}

	// find shown items
	TArray<int32> ShownItems;
	for (int32 ItemIdx = 0; ItemIdx < MaxShownItems; ItemIdx++)
	{
		ShownItems.Add(ItemIdx);
	}

	if (LookedAtItemIndex != INDEX_NONE)
	{
		for (int32 ItemIdx = 0; ItemIdx < QueryData.Items.Num(); ItemIdx++)
		{
			if (QueryData.Items[ItemIdx].ItemIdx == LookedAtItemIndex)
			{
				if (ItemIdx >= MaxShownItems)
				{
					ShownItems[MaxShownItems - 1] = ItemIdx;
				}

				break;
			}
		}
	}

	// find relevant tests (at least one item with non zero score)
	TArray<uint8> TestRelevancy;
	int32 NumRelevantTests = 0;

	for (int32 TestIdx = 0; TestIdx < QueryData.Tests.Num(); TestIdx++)
	{
		TestRelevancy.Add(0);
		for (int32 ItemIdx = 0; ItemIdx < MaxShownItems; ItemIdx++)
		{
			const EQSDebug::FItemData& ItemData = QueryData.Items[ShownItems[ItemIdx]];
			const float TestScore = ItemData.TestValues.IsValidIndex(TestIdx) ? ItemData.TestScores[TestIdx] : 0.0f;

			const bool bIsRelevant = (TestScore != 0.0f);
			if (bIsRelevant)
			{
				NumRelevantTests++;
				TestRelevancy[TestIdx] = 1;
				break;
			}
		}
	}

	// find max not normalized score, add up values for first item
	float MaxScoreNotNormalized = 0.0f;
	for (int32 TestIdx = 0; TestIdx < QueryData.Tests.Num(); TestIdx++)
	{
		const EQSDebug::FItemData& ItemData = QueryData.Items[0];
		const float TestScore = ItemData.TestValues.IsValidIndex(TestIdx) ? ItemData.TestScores[TestIdx] : 0.0f;
		MaxScoreNotNormalized += TestScore;
	}

	// table header
	CanvasContext.CursorY += BackgroundPadding;
	const float HeaderY = CanvasContext.CursorY + 3.0f;
	TileItem.SetColor(ColorOdd);
	CanvasContext.DrawItem(TileItem, 0, CanvasContext.CursorY);

	float HeaderX = CanvasContext.CursorX;
	CanvasContext.PrintfAt(HeaderX, HeaderY, FColor::Yellow, TEXT("Num items: %d"), QueryData.NumValidItems);
	HeaderX += FEQSDebugTable::ItemDescriptionWidth;

	CanvasContext.PrintAt(HeaderX, HeaderY, FColor::White, TEXT("Score"));
	HeaderX += FEQSDebugTable::ItemScoreWidth;

	for (int32 Idx = 0; Idx < QueryData.Tests.Num(); Idx++)
	{
		if (TestRelevancy[Idx])
		{
			CanvasContext.PrintfAt(HeaderX, HeaderY, FColor::White, TEXT("Test %d"), Idx);
			HeaderX += FEQSDebugTable::TestScoreWidth;
		}
	}

	CanvasContext.CursorY += FEQSDebugTable::RowHeight;

	// item rows
	for (int32 Idx = 0; Idx < MaxShownItems; Idx++)
	{
		const int32 ItemIdx = ShownItems[Idx];
		const bool bIsHighlighted = QueryData.Items[ItemIdx].ItemIdx == LookedAtItemIndex;

		TileItem.SetColor(bIsHighlighted ? ColorHighlighted : ((Idx % 2) ? ColorOdd : ColorEven));
		CanvasContext.DrawItem(TileItem, 0, CanvasContext.CursorY);

		DrawDetailedItemRow(QueryData.Items[ShownItems[Idx]], TestRelevancy, MaxScoreNotNormalized, CanvasContext);
		CanvasContext.CursorY += FEQSDebugTable::RowHeight;
	}

	// test description
	FCanvasTileItem DescTileItem(FVector2D(0, 0), GWhiteTexture, FVector2D(CanvasContext.Canvas->SizeX, (CanvasContext.GetLineHeight() * (NumRelevantTests + 1)) + (2 * BackgroundPadding)), FLinearColor(0, 0, 0, 0.2f));
	DescTileItem.BlendMode = SE_BLEND_Translucent;
	CanvasContext.DrawItem(DescTileItem, 0, CanvasContext.CursorY);
	CanvasContext.CursorY += BackgroundPadding;

	if (NumRelevantTests)
	{
		CanvasContext.Print(TEXT("Relevant tests from used option:"));
		for (int32 Idx = 0; Idx < QueryData.Tests.Num(); Idx++)
		{
			if (TestRelevancy[Idx])
			{
				CanvasContext.Printf(TEXT("Test %d = {yellow}%s {LightBlue}(%s)"), Idx, *QueryData.Tests[Idx].ShortName, *QueryData.Tests[Idx].Detailed);
			}
		}
	}
	else
	{
		CanvasContext.Print(TEXT("No relevant tests in used option."));
	}
}

void FGameplayDebuggerCategory_EQS::DrawDetailedItemRow(const EQSDebug::FItemData& ItemData, const TArray<uint8>& TestRelevancy, float MaxScore, FGameplayDebuggerCanvasContext& CanvasContext) const
{
	const float PosY = CanvasContext.CursorY + 1.0f;
	float PosX = CanvasContext.CursorX;

	FString ItemDesc = ItemData.Desc;
	float DescSizeX = 0.0f;
	float DescSizeY = 0.0f;
	CanvasContext.MeasureString(ItemData.Desc, DescSizeX, DescSizeY);

	for (int32 DescIdx = ItemData.Desc.Len() - 1; (DescIdx > 0) && (DescSizeX > FEQSDebugTable::ItemDescriptionWidth); DescIdx--)
	{
		ItemDesc = ItemData.Desc.Left(DescIdx) + TEXT("...");
		CanvasContext.MeasureString(ItemDesc, DescSizeX, DescSizeY);
	}

	CanvasContext.PrintAt(PosX, PosY, FColor::White, ItemDesc);
	PosX += FEQSDebugTable::ItemDescriptionWidth;

	const int32 NumTests = ItemData.TestScores.Num();
	float TotalScoreNotNormalized = 0.0f;
	for (int32 Idx = 0; Idx < NumTests; Idx++)
	{
		if (TestRelevancy[Idx])
		{
			TotalScoreNotNormalized += ItemData.TestScores[Idx];
		}
	}

	FCanvasTileItem ActiveTileItem(FVector2D(0, PosY + 15.0f), GWhiteTexture, FVector2D(0, 2.0f), FLinearColor::Yellow);
	FCanvasTileItem BackTileItem(FVector2D(0, PosY + 15.0f), GWhiteTexture, FVector2D(0, 2.0f), FLinearColor(0.1f, 0.1f, 0.1f));
	const float BarWidth = FEQSDebugTable::ItemScoreWidth - 2.0f;

	const float Pct = (MaxScore > KINDA_SMALL_NUMBER) ? (TotalScoreNotNormalized / MaxScore) : 0.0f;
	ActiveTileItem.Position.X = PosX;
	ActiveTileItem.Size.X = BarWidth * Pct;
	BackTileItem.Position.X = PosX + ActiveTileItem.Size.X;
	BackTileItem.Size.X = FMath::Max(BarWidth * (1.0f - Pct), 0.0f);

	CanvasContext.DrawItem(ActiveTileItem, (float)ActiveTileItem.Position.X, (float)ActiveTileItem.Position.Y);
	CanvasContext.DrawItem(BackTileItem, (float)BackTileItem.Position.X, (float)BackTileItem.Position.Y);

	CanvasContext.PrintfAt(PosX, PosY, FColor::Yellow, TEXT("%.2f"), TotalScoreNotNormalized);
	PosX += FEQSDebugTable::ItemScoreWidth;

	const FString RelevantScoreColor = "green";
	const FString IgnoredScoreColor = FColor(0, 96, 0).ToString();
	const FString RelevantValueColor = FColor(192, 192, 192).ToString();
	const FString IgnoredValueColor = FColor(96, 96, 96).ToString();

	for (int32 Idx = 0; Idx < NumTests; Idx++)
	{
		if (TestRelevancy[Idx])
		{
			const float ScoreW = ItemData.TestScores[Idx];
			const float ScoreN = ItemData.TestValues[Idx];
			const bool bIsIgnoredValue = FMath::IsNearlyZero(ScoreW);
			const FString DescScoreN = (ScoreN == UEnvQueryTypes::SkippedItemValue) ? TEXT("SKIP") : FString::Printf(TEXT("%.2f"), ScoreN);

			CanvasContext.PrintfAt(PosX, PosY, TEXT("{%s}%.2f {%s}%s"),
				bIsIgnoredValue ? *IgnoredScoreColor : *RelevantScoreColor, ScoreW,
				bIsIgnoredValue ? *IgnoredValueColor : *RelevantValueColor, *DescScoreN);

			PosX += FEQSDebugTable::TestScoreWidth;
		}
	}
}
#endif // USE_EQS_DEBUGGER

void FGameplayDebuggerCategory_EQS::CycleShownQueries()
{
#if USE_EQS_DEBUGGER
	ShownQueryIndex = (DataPack.QueryDebugData.Num() > 0) ? ((ShownQueryIndex + 1) % DataPack.QueryDebugData.Num()) : 0;
#endif // USE_EQS_DEBUGGER
	MarkRenderStateDirty();
}

void FGameplayDebuggerCategory_EQS::ToggleDetailView()
{
	bShowDetails = !bShowDetails;
}

#endif // WITH_GAMEPLAY_DEBUGGER_MENU
