// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebugger/GameplayDebuggerCategory_NavLocalGrid.h"

#include "DynamicMeshBuilder.h"

#if WITH_GAMEPLAY_DEBUGGER_MENU

#include "GameFramework/PlayerController.h"
#include "AIController.h"
#include "Engine/Engine.h"
#include "MeshElementCollector.h"
#include "Navigation/NavLocalGridManager.h"
#include "Navigation/GridPathFollowingComponent.h"
#include "DebugRenderSceneProxy.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "PrimitiveDrawingUtils.h"

//////////////////////////////////////////////////////////////////////////
// Scene proxy

class FNavLocalGridSceneProxy final : public FDebugRenderSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	FNavLocalGridSceneProxy(const UPrimitiveComponent* InComponent,
		const FGameplayDebuggerCategory_NavLocalGrid::FRepData& RepData,
		const FGameplayDebuggerCategory_NavLocalGrid::FRepAgentData& AgentData) 
		: FDebugRenderSceneProxy(InComponent)
	{
		for (int32 Idx = 0; Idx < RepData.Grids.Num(); Idx++)
		{
			const FGameplayDebuggerCategory_NavLocalGrid::FRepData::FGridData& GridData = RepData.Grids[Idx];
			StoreGridBounds(GridData.Bounds, BoundsMeshVerts, BoundsMeshIndices);

			for (int32 IdxX = 0; IdxX < GridData.NumCols; IdxX++)
			{
				for (int32 IdxY = 0; IdxY < GridData.NumRows; IdxY++)
				{
					const int32 CellIndex = (IdxX * GridData.NumRows) + IdxY;
					const bool bIsMarked = GridData.Cells[CellIndex] != 0;
					const bool bIsOnPath = (AgentData.GridIdx == Idx) && AgentData.PathCells.Contains(CellIndex);

					if (bIsMarked)
					{
						StoreGridCellFull(GridData.Bounds, FIntPoint(IdxX, IdxY), GridData.CellSize, MarkedCellMeshVerts, MarkedCellMeshIndices);
					}
					else if (bIsOnPath)
					{
						StoreGridCellLayer(GridData.Bounds, FIntPoint(IdxX, IdxY), GridData.CellSize, PathMeshVerts, PathMeshIndices);
					}
					else
					{
						StoreGridCellLayer(GridData.Bounds, FIntPoint(IdxX, IdxY), GridData.CellSize, FreeCellMeshVerts, FreeCellMeshIndices);
					}
				}
			}
		}

		if (RepData.Grids.IsValidIndex(AgentData.GridIdx))
		{
			const FGameplayDebuggerCategory_NavLocalGrid::FRepData::FGridData& GridData = RepData.Grids[AgentData.GridIdx];
			for (int32 Idx = 1; Idx < AgentData.PathCells.Num(); Idx++)
			{
				const FIntPoint P0(AgentData.PathCells[Idx - 1] / GridData.NumRows, AgentData.PathCells[Idx - 1] % GridData.NumRows);
				const FIntPoint P1(AgentData.PathCells[Idx] / GridData.NumRows, AgentData.PathCells[Idx] % GridData.NumRows);

				StoreGridConnector(GridData.Bounds, P0, P1, GridData.CellSize, PathMeshVerts, PathMeshIndices);
			}
		}
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
	{
		static FColor BoundsColor(255, 255, 0, 128);
		static FColor FreeCellColor(255, 255, 255, 16);
		static FColor MarkedCellColor(255, 0, 0, 16);
		static FColor PathColor(0, 255, 255, 128);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				const FSceneView* View = Views[ViewIndex];

				if (FreeCellMeshVerts.Num() > 0)
				{
					const FColoredMaterialRenderProxy* MeshColorInstance = &Collector.AllocateOneFrameResource<FColoredMaterialRenderProxy>(GEngine->DebugMeshMaterial->GetRenderProxy(), FLinearColor(FreeCellColor));
					FDynamicMeshBuilder	MeshBuilder(View->GetFeatureLevel());
					MeshBuilder.AddVertices(FreeCellMeshVerts);
					MeshBuilder.AddTriangles(FreeCellMeshIndices);
					MeshBuilder.GetMesh(FMatrix::Identity, MeshColorInstance, IntCastChecked<uint8>((int32)GetDepthPriorityGroup(View)), false, false, ViewIndex, Collector);
				}

				if (MarkedCellMeshVerts.Num() > 0)
				{
					const FColoredMaterialRenderProxy* MeshColorInstance = &Collector.AllocateOneFrameResource<FColoredMaterialRenderProxy>(GEngine->DebugMeshMaterial->GetRenderProxy(), FLinearColor(MarkedCellColor));
					FDynamicMeshBuilder	MeshBuilder(View->GetFeatureLevel());
					MeshBuilder.AddVertices(MarkedCellMeshVerts);
					MeshBuilder.AddTriangles(MarkedCellMeshIndices);
					MeshBuilder.GetMesh(FMatrix::Identity, MeshColorInstance, IntCastChecked<uint8>((int32)GetDepthPriorityGroup(View)), false, false, ViewIndex, Collector);
				}

				if (BoundsMeshVerts.Num() > 0)
				{
					const FColoredMaterialRenderProxy* MeshColorInstance = &Collector.AllocateOneFrameResource<FColoredMaterialRenderProxy>(GEngine->DebugMeshMaterial->GetRenderProxy(), FLinearColor(BoundsColor));
					FDynamicMeshBuilder	MeshBuilder(View->GetFeatureLevel());
					MeshBuilder.AddVertices(BoundsMeshVerts);
					MeshBuilder.AddTriangles(BoundsMeshIndices);
					MeshBuilder.GetMesh(FMatrix::Identity, MeshColorInstance, IntCastChecked<uint8>((int32)GetDepthPriorityGroup(View)), false, false, ViewIndex, Collector);
				}

				if (PathMeshVerts.Num() > 0)
				{
					const FColoredMaterialRenderProxy* MeshColorInstance = &Collector.AllocateOneFrameResource<FColoredMaterialRenderProxy>(GEngine->DebugMeshMaterial->GetRenderProxy(), FLinearColor(PathColor));
					FDynamicMeshBuilder	MeshBuilder(View->GetFeatureLevel());
					MeshBuilder.AddVertices(PathMeshVerts);
					MeshBuilder.AddTriangles(PathMeshIndices);
					MeshBuilder.GetMesh(FMatrix::Identity, MeshColorInstance, IntCastChecked<uint8>((int32)GetDepthPriorityGroup(View)), false, false, ViewIndex, Collector);
				}
			}
		}
	}

protected:

	inline void StoreTriIndices(int32 V0, int32 V1, int32 V2, int32 FirstVertIndex, TArray<uint32>& Indices)
	{
		Indices.Add(FirstVertIndex + V0);
		Indices.Add(FirstVertIndex + V1);
		Indices.Add(FirstVertIndex + V2);
	}

	inline void StoreQuadIndices(int32 V0, int32 V1, int32 V2, int32 V3, int32 FirstVertIndex, TArray<uint32>& Indices)
	{
		StoreTriIndices(V0, V1, V2, FirstVertIndex, Indices);
		StoreTriIndices(V0, V2, V3, FirstVertIndex, Indices);
	}

	void StoreGridConnector(const FBox& GridBounds, const FIntPoint& P0, const FIntPoint& P1, const float CellSize, TArray<FDynamicMeshVertex>& Verts, TArray<uint32>& Indices)
	{
		const float Width = 5.0f;

		const FVector Location0 = FVector(GridBounds.Min.X, GridBounds.Min.Y, GridBounds.Max.Z) + FVector(CellSize * (P0.X + 0.5f), CellSize * (P0.Y + 0.5f), 0);
		const FVector Location1 = FVector(GridBounds.Min.X, GridBounds.Min.Y, GridBounds.Max.Z) + FVector(CellSize * (P1.X + 0.5f), CellSize * (P1.Y + 0.5f), 0);
		const FVector DirFwd = (Location1 - Location0).GetSafeNormal();
		const FVector DirRight = FVector::CrossProduct(DirFwd, FVector::UpVector);

		const int32 FirstVertIdx = Verts.Num();

		Verts.Add(FDynamicMeshVertex((FVector3f)(Location0 - (DirRight * Width))));
		Verts.Add(FDynamicMeshVertex((FVector3f)(Location0 + (DirRight * Width))));
		Verts.Add(FDynamicMeshVertex((FVector3f)(Location1 - (DirRight * Width))));
		Verts.Add(FDynamicMeshVertex((FVector3f)(Location1 + (DirRight * Width))));

		StoreQuadIndices(0, 1, 3, 2, FirstVertIdx, Indices);
	}

	void StoreGridCellLayer(const FBox& GridBounds, const FIntPoint& CellIdx, const float CellSize, TArray<FDynamicMeshVertex>& Verts, TArray<uint32>& Indices)
	{
		const float CellGapSize = 5.0f;
		
		const FVector CellMin = FVector(GridBounds.Min.X, GridBounds.Min.Y, GridBounds.Max.Z) + FVector(CellSize * CellIdx.X, CellSize * CellIdx.Y, 0);
		const FBox CellBox(CellMin, CellMin + FVector(CellSize, CellSize, 0.0f));
		const FVector Center = CellBox.GetCenter();
		const FVector Extent = CellBox.GetExtent() - FVector(CellGapSize, CellGapSize, 0.);

		const int32 FirstVertIdx = Verts.Num();

		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X - Extent.X, Center.Y - Extent.Y, Center.Z - Extent.Z))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X + Extent.X, Center.Y - Extent.Y, Center.Z - Extent.Z))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X - Extent.X, Center.Y + Extent.Y, Center.Z - Extent.Z))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X + Extent.X, Center.Y + Extent.Y, Center.Z - Extent.Z))));

		StoreQuadIndices(0, 1, 3, 2, FirstVertIdx, Indices);
	}

	void StoreGridCellFull(const FBox& GridBounds, const FIntPoint& CellIdx, const float CellSize, TArray<FDynamicMeshVertex>& Verts, TArray<uint32>& Indices)
	{
		const float CellGapSize = 5.0f;

		const FVector CellMin = FVector(GridBounds.Min.X, GridBounds.Min.Y, GridBounds.Min.Z) + FVector(CellSize * CellIdx.X, CellSize * CellIdx.Y, 0);
		const FBox CellBox(CellMin, CellMin + FVector(CellSize, CellSize, GridBounds.Max.Z - GridBounds.Min.Z));
		const FVector Center = CellBox.GetCenter();
		const FVector Extent = CellBox.GetExtent() - FVector(CellGapSize, CellGapSize, 0.f);

		const int32 FirstVertIdx = Verts.Num();

		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X - Extent.X, Center.Y - Extent.Y, Center.Z - Extent.Z))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X + Extent.X, Center.Y - Extent.Y, Center.Z - Extent.Z))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X - Extent.X, Center.Y + Extent.Y, Center.Z - Extent.Z))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X + Extent.X, Center.Y + Extent.Y, Center.Z - Extent.Z))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X - Extent.X, Center.Y - Extent.Y, Center.Z + Extent.Z))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X + Extent.X, Center.Y - Extent.Y, Center.Z + Extent.Z))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X - Extent.X, Center.Y + Extent.Y, Center.Z + Extent.Z))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X + Extent.X, Center.Y + Extent.Y, Center.Z + Extent.Z))));

		StoreQuadIndices(4, 5, 7, 6, FirstVertIdx, Indices);
		StoreQuadIndices(1, 3, 7, 5, FirstVertIdx, Indices);
		StoreQuadIndices(0, 1, 5, 4, FirstVertIdx, Indices);

		StoreQuadIndices(1, 0, 2, 3, FirstVertIdx, Indices);
		StoreQuadIndices(0, 4, 6, 2, FirstVertIdx, Indices);
		StoreQuadIndices(2, 6, 7, 3, FirstVertIdx, Indices);
	}

	void StoreGridBounds(const FBox& GridBounds, TArray<FDynamicMeshVertex>& Verts, TArray<uint32>& Indices)
	{
		const float FaceWidth = 5.0f;

		const int32 FirstVertIdx = Verts.Num();
		const FVector Center = GridBounds.GetCenter();
		const FVector Extent = GridBounds.GetExtent();

		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X - Extent.X, Center.Y - Extent.Y, Center.Z - Extent.Z))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X + Extent.X, Center.Y - Extent.Y, Center.Z - Extent.Z))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X - Extent.X, Center.Y + Extent.Y, Center.Z - Extent.Z))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X + Extent.X, Center.Y + Extent.Y, Center.Z - Extent.Z))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X - Extent.X, Center.Y - Extent.Y, Center.Z + Extent.Z))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X + Extent.X, Center.Y - Extent.Y, Center.Z + Extent.Z))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X - Extent.X, Center.Y + Extent.Y, Center.Z + Extent.Z))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X + Extent.X, Center.Y + Extent.Y, Center.Z + Extent.Z))));

		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X - Extent.X + FaceWidth, Center.Y - Extent.Y, Center.Z - Extent.Z + FaceWidth))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X + Extent.X - FaceWidth, Center.Y - Extent.Y, Center.Z - Extent.Z + FaceWidth))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X - Extent.X + FaceWidth, Center.Y - Extent.Y, Center.Z + Extent.Z - FaceWidth))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X + Extent.X - FaceWidth, Center.Y - Extent.Y, Center.Z + Extent.Z - FaceWidth))));

		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X + Extent.X, Center.Y - Extent.Y + FaceWidth, Center.Z - Extent.Z + FaceWidth))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X + Extent.X, Center.Y + Extent.Y - FaceWidth, Center.Z - Extent.Z + FaceWidth))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X + Extent.X, Center.Y - Extent.Y + FaceWidth, Center.Z + Extent.Z - FaceWidth))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X + Extent.X, Center.Y + Extent.Y - FaceWidth, Center.Z + Extent.Z - FaceWidth))));

		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X - Extent.X + FaceWidth, Center.Y - Extent.Y + FaceWidth, Center.Z + Extent.Z))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X + Extent.X - FaceWidth, Center.Y - Extent.Y + FaceWidth, Center.Z + Extent.Z))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X - Extent.X + FaceWidth, Center.Y + Extent.Y - FaceWidth, Center.Z + Extent.Z))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X + Extent.X - FaceWidth, Center.Y + Extent.Y - FaceWidth, Center.Z + Extent.Z))));

		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X - Extent.X + FaceWidth, Center.Y + Extent.Y, Center.Z - Extent.Z + FaceWidth))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X + Extent.X - FaceWidth, Center.Y + Extent.Y, Center.Z - Extent.Z + FaceWidth))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X - Extent.X + FaceWidth, Center.Y + Extent.Y, Center.Z + Extent.Z - FaceWidth))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X + Extent.X - FaceWidth, Center.Y + Extent.Y, Center.Z + Extent.Z - FaceWidth))));

		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X - Extent.X, Center.Y - Extent.Y + FaceWidth, Center.Z - Extent.Z + FaceWidth))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X - Extent.X, Center.Y + Extent.Y - FaceWidth, Center.Z - Extent.Z + FaceWidth))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X - Extent.X, Center.Y - Extent.Y + FaceWidth, Center.Z + Extent.Z - FaceWidth))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X - Extent.X, Center.Y + Extent.Y - FaceWidth, Center.Z + Extent.Z - FaceWidth))));

		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X - Extent.X + FaceWidth, Center.Y - Extent.Y + FaceWidth, Center.Z - Extent.Z))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X + Extent.X - FaceWidth, Center.Y - Extent.Y + FaceWidth, Center.Z - Extent.Z))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X - Extent.X + FaceWidth, Center.Y + Extent.Y - FaceWidth, Center.Z - Extent.Z))));
		Verts.Add(FDynamicMeshVertex(UE::LWC::NarrowWorldPositionChecked(FVector(Center.X + Extent.X - FaceWidth, Center.Y + Extent.Y - FaceWidth, Center.Z - Extent.Z))));

		StoreQuadIndices(0, 1, 9, 8, FirstVertIdx, Indices);
		StoreQuadIndices(1, 5, 11, 9, FirstVertIdx, Indices);
		StoreQuadIndices(5, 4, 10, 11, FirstVertIdx, Indices);
		StoreQuadIndices(4, 0, 8, 10, FirstVertIdx, Indices);

		StoreQuadIndices(1, 3, 13, 12, FirstVertIdx, Indices);
		StoreQuadIndices(3, 7, 15, 13, FirstVertIdx, Indices);
		StoreQuadIndices(7, 5, 14, 15, FirstVertIdx, Indices);
		StoreQuadIndices(5, 1, 12, 14, FirstVertIdx, Indices);

		StoreQuadIndices(4, 5, 17, 16, FirstVertIdx, Indices);
		StoreQuadIndices(5, 7, 19, 17, FirstVertIdx, Indices);
		StoreQuadIndices(7, 6, 18, 19, FirstVertIdx, Indices);
		StoreQuadIndices(6, 4, 16, 18, FirstVertIdx, Indices);

		StoreQuadIndices(3, 2, 20, 21, FirstVertIdx, Indices);
		StoreQuadIndices(7, 3, 21, 23, FirstVertIdx, Indices);
		StoreQuadIndices(6, 7, 23, 22, FirstVertIdx, Indices);
		StoreQuadIndices(2, 6, 22, 20, FirstVertIdx, Indices);

		StoreQuadIndices(2, 0, 24, 25, FirstVertIdx, Indices);
		StoreQuadIndices(6, 2, 25, 27, FirstVertIdx, Indices);
		StoreQuadIndices(4, 6, 27, 26, FirstVertIdx, Indices);
		StoreQuadIndices(0, 4, 26, 24, FirstVertIdx, Indices);

		StoreQuadIndices(1, 0, 28, 29, FirstVertIdx, Indices);
		StoreQuadIndices(3, 1, 29, 31, FirstVertIdx, Indices);
		StoreQuadIndices(2, 3, 31, 30, FirstVertIdx, Indices);
		StoreQuadIndices(0, 2, 30, 28, FirstVertIdx, Indices);
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View);
		Result.bDynamicRelevance = true;
		Result.bNormalTranslucency = Result.bSeparateTranslucency = IsShown(View);
		return Result;
	}

	virtual uint32 GetMemoryFootprint(void) const override
	{
		return sizeof(*this) + GetAllocatedSize();
	}

	uint32 GetAllocatedSize(void) const
	{
		const SIZE_T AllocatedSize = FDebugRenderSceneProxy::GetAllocatedSize() +
			FreeCellMeshVerts.GetAllocatedSize() + FreeCellMeshIndices.GetAllocatedSize() +
			MarkedCellMeshVerts.GetAllocatedSize() + MarkedCellMeshIndices.GetAllocatedSize() +
			BoundsMeshVerts.GetAllocatedSize() + BoundsMeshIndices.GetAllocatedSize() +
			PathMeshVerts.GetAllocatedSize() + PathMeshIndices.GetAllocatedSize();

		return IntCastChecked<uint32>(AllocatedSize);
	}

private:
	TArray<FDynamicMeshVertex> FreeCellMeshVerts;
	TArray<uint32> FreeCellMeshIndices;
	TArray<FDynamicMeshVertex> MarkedCellMeshVerts;
	TArray<uint32> MarkedCellMeshIndices;
	TArray<FDynamicMeshVertex> BoundsMeshVerts;
	TArray<uint32> BoundsMeshIndices;
	TArray<FDynamicMeshVertex> PathMeshVerts;
	TArray<uint32> PathMeshIndices;
};


//////////////////////////////////////////////////////////////////////////
// Category

FGameplayDebuggerCategory_NavLocalGrid::FGameplayDebuggerCategory_NavLocalGrid()
{
	SetDataPackReplication<FRepData>(&DataPack, EGameplayDebuggerDataPack::Persistent);
	SetDataPackReplication<FRepAgentData>(&AgentDataPack);

	bShowUpdateTimer = false;
	bShowDataPackReplication = true;
	bShowOnlyWithDebugActor = false;
}

TSharedRef<FGameplayDebuggerCategory> FGameplayDebuggerCategory_NavLocalGrid::MakeInstance()
{
	return MakeShareable(new FGameplayDebuggerCategory_NavLocalGrid());
}

void FGameplayDebuggerCategory_NavLocalGrid::FRepData::Serialize(FArchive& Ar)
{
	Ar << NumSources;

	int32 NumGrids = Grids.Num();
	Ar << NumGrids;
	if (Ar.IsLoading())
	{
		Grids.SetNum(NumGrids);
	}

	for (int32 Idx = 0; Idx < NumGrids; Idx++)
	{
		FGridData& Data = Grids[Idx];
		Ar << Data.Bounds;
		Ar << Data.CellSize;
		Ar << Data.NumRows;
		Ar << Data.NumCols;

		Data.Cells.BulkSerialize(Ar);
	}
}

void FGameplayDebuggerCategory_NavLocalGrid::FRepAgentData::Serialize(FArchive& Ar)
{
	Ar << PathCells;
	Ar << GridIdx;
}

void FGameplayDebuggerCategory_NavLocalGrid::CollectData(APlayerController* OwnerPC, AActor* DebugActor)
{
	UNavLocalGridManager* GridManager = UNavLocalGridManager::GetCurrent(OwnerPC);
	if (GridManager && GridManager->GetVersion() != DataPack.VersionNum)
	{
		DataPack.VersionNum = GridManager->GetVersion();
		DataPack.NumSources = GridManager->GetNumSources();
		DataPack.Grids.SetNum(GridManager->GetNumGrids());

		for (int32 Idx = 0; Idx < GridManager->GetNumGrids(); Idx++)
		{
			const FNavLocalGridData& GridData = GridManager->GetGridData(Idx);
			FRepData::FGridData& DebugGrid = DataPack.Grids[Idx];

			DebugGrid.Bounds = GridData.WorldBounds;
			DebugGrid.CellSize = GridData.GridCellSize;
			DebugGrid.NumCols = (int32)GridData.GridSize.Width;
			DebugGrid.NumRows = (int32)GridData.GridSize.Height;

			const uint32 NumGridCells = GridData.GetCellsCount();
			DebugGrid.Cells.SetNum(NumGridCells);
			for (uint32 CellIdx = 0; CellIdx < NumGridCells; CellIdx++)
			{
				DebugGrid.Cells[CellIdx] = GridData.GetCellAtIndexUnsafe(CellIdx);
			}
		}
	}

	const APawn* DebugPawn = Cast<APawn>(DebugActor);
	const AAIController* DebugAI = DebugPawn ? Cast<AAIController>(DebugPawn->GetController()) : nullptr;
	const UGridPathFollowingComponent* GridPathComp = DebugAI ? Cast<UGridPathFollowingComponent>(DebugAI->GetPathFollowingComponent()) : nullptr;
	if (GridPathComp && GridPathComp->HasActiveGrid() && GridManager)
	{
		const int32 CurrentGridIdx = GridPathComp->GetActiveGridIdx();
		if (CurrentGridIdx >= 0 && CurrentGridIdx < GridManager->GetNumGrids())
		{
			AgentDataPack.GridIdx = CurrentGridIdx;

			TArray<FVector> PathPoints = GridPathComp->GetGridPathPoints();
			const FNavLocalGridData& GridData = GridManager->GetGridData(AgentDataPack.GridIdx);

			AgentDataPack.PathCells.SetNum(PathPoints.Num());
			for (int32 Idx = 0; Idx < PathPoints.Num(); Idx++)
			{
				AgentDataPack.PathCells[Idx] = GridData.GetCellIndex(PathPoints[Idx]);
			}
		}
	}
}

void FGameplayDebuggerCategory_NavLocalGrid::DrawData(APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext)
{
	CanvasContext.Printf(TEXT("Num sources: {%s}%d"), DataPack.NumSources ? TEXT("yellow") : TEXT("red"), DataPack.NumSources);
}

FDebugRenderSceneProxy* FGameplayDebuggerCategory_NavLocalGrid::CreateDebugSceneProxy(const UPrimitiveComponent* InComponent, FDebugDrawDelegateHelper*& OutDelegateHelper)
{
	OutDelegateHelper = nullptr;
	return new FNavLocalGridSceneProxy(InComponent, DataPack, AgentDataPack);
}

void FGameplayDebuggerCategory_NavLocalGrid::OnDataPackReplicated(int32 DataPackId)
{
	MarkRenderStateDirty();
}

#endif // WITH_GAMEPLAY_DEBUGGER_MENU
