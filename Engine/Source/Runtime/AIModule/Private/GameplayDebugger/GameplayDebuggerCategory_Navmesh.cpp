// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebugger/GameplayDebuggerCategory_Navmesh.h"
#include "GameFramework/Pawn.h"

#if WITH_GAMEPLAY_DEBUGGER_MENU

#include "NavigationSystem.h"
#include "GameFramework/PlayerController.h"
#include "NavMesh/RecastNavMesh.h"

namespace FGameplayDebuggerCategoryNavmeshTweakables
{
	int32 bDrawExcludedFlags = 0;
	int32 DisplaySize = 3;
	float RefreshInterval = 5.0f;
}

namespace
{
	FAutoConsoleVariableRef CVars_GameplayDebuggerCategory_Navmesh[] = {
		FAutoConsoleVariableRef(TEXT("ai.debug.nav.DrawExcludedFlags"), 
			FGameplayDebuggerCategoryNavmeshTweakables::bDrawExcludedFlags,
			TEXT("If we want to mark \"forbidden\" nav polys while debug-drawing.")),

		FAutoConsoleVariableRef(TEXT("ai.debug.nav.DisplaySize"), 
			FGameplayDebuggerCategoryNavmeshTweakables::DisplaySize,
			TEXT("Area to display in tiles (DisplaySize x DisplaySize) in gameplay debugger."
				 " Size will round up to an odd number of tiles."
				 " Culling distance can be modified using 'ai.debug.nav.DrawDistance'.")),

		FAutoConsoleVariableRef(TEXT("ai.debug.nav.RefreshInterval"),
			FGameplayDebuggerCategoryNavmeshTweakables::RefreshInterval,
			TEXT("Interval (in seconds) at which data will be collected."))
	};
}

FGameplayDebuggerCategory_Navmesh::FGameplayDebuggerCategory_Navmesh()
{
	bShowOnlyWithDebugActor = false;
	bShowDataPackReplication = true;
	CollectDataInterval = FGameplayDebuggerCategoryNavmeshTweakables::RefreshInterval;
	SetDataPackReplication<FNavMeshSceneProxyData>(&NavmeshRenderData);
	SetDataPackReplication<FRepData>(&DataPack);

	const FGameplayDebuggerInputHandlerConfig CycleActorReference(TEXT("Cycle Actor Reference"), TEXT("Subtract"), FGameplayDebuggerInputModifier::Shift);
	const FGameplayDebuggerInputHandlerConfig CycleNavigationData(TEXT("Cycle NavData"), TEXT("Add"), FGameplayDebuggerInputModifier::Shift);
	const FGameplayDebuggerInputHandlerConfig LockReferenceLocation(TEXT("Lock Reference Location"), TEXT("Multiply"), FGameplayDebuggerInputModifier::Shift);

	BindKeyPress(CycleActorReference, this, &FGameplayDebuggerCategory_Navmesh::CycleActorReference, EGameplayDebuggerInputMode::Replicated);
	BindKeyPress(CycleNavigationData, this, &FGameplayDebuggerCategory_Navmesh::CycleNavData, EGameplayDebuggerInputMode::Replicated);
	BindKeyPress(LockReferenceLocation, this, &FGameplayDebuggerCategory_Navmesh::ToggleLockedReferenceLocation, EGameplayDebuggerInputMode::Replicated);
}

void FGameplayDebuggerCategory_Navmesh::CycleNavData()
{
	bSwitchToNextNavigationData = true;
	ForceImmediateCollect();
}

void FGameplayDebuggerCategory_Navmesh::CycleActorReference()
{
	switch (ActorReferenceMode)
	{
	case EActorReferenceMode::PlayerActorOnly:
		// Nothing to do since we don't have a debug actor
		break;
	
	case EActorReferenceMode::DebugActor:
		ActorReferenceMode = EActorReferenceMode::PlayerActor;
		ForceImmediateCollect();
		break;
	
	case EActorReferenceMode::PlayerActor:
		ActorReferenceMode = EActorReferenceMode::DebugActor;
		ForceImmediateCollect();
		break;
	}
}

void FGameplayDebuggerCategory_Navmesh::ToggleLockedReferenceLocation()
{
	bToggleLockedReferenceLocation = true;
}

TSharedRef<FGameplayDebuggerCategory> FGameplayDebuggerCategory_Navmesh::MakeInstance()
{
	return MakeShareable(new FGameplayDebuggerCategory_Navmesh());
}

void FGameplayDebuggerCategory_Navmesh::FRepData::Serialize(FArchive& Ar)
{
	Ar << NumDirtyAreas;
	Ar << NumRunningTasks;
	Ar << NumRemainingTasks;
	Ar << NavDataName;

	Ar << LockedReferenceLocation;

	Ar << NavBuildLockStatusDesc;
	Ar << SupportedAgents;
	Ar << NumSuspendedDirtyAreas;
	Ar << bIsNavBuildLocked;
	Ar << bIsNavOctreeLocked;
	Ar << bIsNavDataRebuildingSuspended;

	uint8 Flags =
		((bCanChangeReference			? 1 : 0) << 0) |
		((bCanCycleNavigationData		? 1 : 0) << 1) |
		((bIsUsingPlayerActor			? 1 : 0) << 2) |
		((bReferenceTooFarFromNavData	? 1 : 0) << 3);

	Ar << Flags;

	bCanChangeReference			= (Flags & (1 << 0)) != 0;
	bCanCycleNavigationData		= (Flags & (1 << 1)) != 0;
	bIsUsingPlayerActor			= (Flags & (1 << 2)) != 0;
	bReferenceTooFarFromNavData = (Flags & (1 << 3)) != 0;
}

void FGameplayDebuggerCategory_Navmesh::CollectData(APlayerController* OwnerPC, AActor* DebugActor)
{
	UNavigationSystemV1* NavSys = nullptr;
	ANavigationData* NavData = nullptr;
	const APawn* RefPawn = nullptr;
	int32 NumNavData = 0;

	if (OwnerPC != nullptr)
	{
		NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(OwnerPC->GetWorld());
		if (NavSys) 
		{
			DataPack.NumDirtyAreas = NavSys->GetNumDirtyAreas();
			DataPack.NumRunningTasks = NavSys->GetNumRunningBuildTasks();
			DataPack.NumRemainingTasks = NavSys->GetNumRemainingBuildTasks();
			DataPack.bIsNavOctreeLocked = NavSys->IsNavigationOctreeLocked();
			DataPack.bIsNavBuildLocked = NavSys->IsNavigationBuildingLocked();
			DataPack.NavBuildLockStatusDesc = FString("Unknown");
			if (NavSys->IsNavigationBuildingLocked(ENavigationBuildLock::InitialLock))
			{
				DataPack.NavBuildLockStatusDesc = FString("Initial Lock");
			}
			else if (NavSys->IsNavigationBuildingLocked(ENavigationBuildLock::Custom))
			{
				DataPack.NavBuildLockStatusDesc = FString("Custom Lock");
			}
			else if (NavSys->IsNavigationBuildingLocked(ENavigationBuildLock::NoUpdateInEditor))
			{
				DataPack.NavBuildLockStatusDesc = FString("NoUpdateInEditor Lock");
			}
			else if (NavSys->IsNavigationBuildingLocked(ENavigationBuildLock::NoUpdateInPIE))
			{
				DataPack.NavBuildLockStatusDesc = FString("NoUpdateInPIE Lock");
			}
			for (const FNavDataConfig& NavigationData : NavSys->GetSupportedAgents())
			{
				if (NavigationData.IsValid())
				{
					DataPack.SupportedAgents = FString::Printf(TEXT("%s%s%s"), *DataPack.SupportedAgents, DataPack.SupportedAgents.IsEmpty() ? TEXT("") : TEXT(" | "), *NavigationData.Name.ToString());
				}
			}
			
			NumNavData = NavSys->NavDataSet.Num();
			
			APawn* DebugActorAsPawn = Cast<APawn>(DebugActor);
			
			// Manage actor reference mode:
			// - As soon as we get a new valid debug actor: use it as reference to preserve legacy behavior 
			// - Debug actor is no longer valid: use player actor
			if (ActorReferenceMode == EActorReferenceMode::PlayerActorOnly && DebugActorAsPawn != nullptr)
			{
				ActorReferenceMode = EActorReferenceMode::DebugActor;
			}
			else if (ActorReferenceMode != EActorReferenceMode::PlayerActorOnly && DebugActorAsPawn == nullptr)
			{
				ActorReferenceMode = EActorReferenceMode::PlayerActorOnly;
			}

			if (bSwitchToNextNavigationData || NavDataIndexToDisplay == INDEX_NONE)
			{
				NavDataIndexToDisplay = (NumNavData > 0) ? FMath::Max(0, ++NavDataIndexToDisplay % NumNavData) : INDEX_NONE;
				bSwitchToNextNavigationData = false;
			}

			if (NavSys->NavDataSet.IsValidIndex(NavDataIndexToDisplay))
			{
				NavData = NavSys->NavDataSet[NavDataIndexToDisplay];
				DataPack.bIsNavDataRebuildingSuspended = NavData->IsRebuildingSuspended();
				DataPack.NumSuspendedDirtyAreas = NavData->GetNumSuspendedDirtyAreas();
			}
			
			if (ActorReferenceMode == EActorReferenceMode::DebugActor)
			{
				RefPawn = DebugActorAsPawn;

				// Switch to new debug actor NavigationData
				if (PrevDebugActorReference != RefPawn)
				{
					const FNavAgentProperties& NavAgentProperties = RefPawn->GetNavAgentPropertiesRef();
					NavData = NavSys->GetNavDataForProps(NavAgentProperties, RefPawn->GetNavAgentLocation());
					NavDataIndexToDisplay = NavSys->NavDataSet.Find(NavData);

					PrevDebugActorReference = RefPawn;
				}
			}
			else
			{
				RefPawn = OwnerPC ? OwnerPC->GetPawnOrSpectator() : nullptr;
			}
		}
	}

	if (bToggleLockedReferenceLocation)
	{
		LockedReferenceLocation = (!LockedReferenceLocation.IsSet() && RefPawn) ? RefPawn->GetActorLocation() : TOptional<FVector>();
		bToggleLockedReferenceLocation = false;
	}
	DataPack.LockedReferenceLocation = LockedReferenceLocation.Get(FNavigationSystem::InvalidLocation);

	if (NavData)
	{
		DataPack.bIsUsingPlayerActor = (ActorReferenceMode != EActorReferenceMode::DebugActor);
		DataPack.bCanChangeReference = (ActorReferenceMode != EActorReferenceMode::PlayerActorOnly);
		DataPack.bCanCycleNavigationData = (NumNavData > 1);

		if (NumNavData > 1)
		{
			DataPack.NavDataName = FString::Printf(TEXT("[%d/%d] %s"), NavDataIndexToDisplay + 1, NumNavData, *NavData->GetFName().ToString());
		}
		else
		{
			DataPack.NavDataName = NavData->GetFName().ToString();
		}
	}

	CollectNavigationData(NavSys, NavData, RefPawn);
}

void FGameplayDebuggerCategory_Navmesh::CollectNavigationData(const UNavigationSystemV1* NavSys, const ANavigationData* NavData, const APawn* RefPawn)
{
	const FVector RefLocation = LockedReferenceLocation.Get(RefPawn ? RefPawn->GetActorLocation() : FNavigationSystem::InvalidLocation);
	CollectNavigationData(NavSys, NavData, RefLocation);
}

void FGameplayDebuggerCategory_Navmesh::CollectNavigationData(const UNavigationSystemV1* NavSys, const ANavigationData* NavData, const FVector& RefLocation)
{
#if WITH_RECAST
	const ARecastNavMesh* RecastNavMesh = Cast<const ARecastNavMesh>(NavData);
	if (RecastNavMesh && RefLocation != FNavigationSystem::InvalidLocation)
	{
		// add NxN neighborhood of target (where N is the number of tiles)
		// Note that we round up to the next odd number to keep the reference position in the middle tile

		TArray<FIntPoint> TileDeltas;
		RetrieveRelativeTilesToDisplay(TileDeltas);

		int32 TargetTileX = 0;
		int32 TargetTileY = 0;
		RecastNavMesh->GetNavMeshTileXY(RefLocation, TargetTileX, TargetTileY);

		TArray<FNavTileRef> TileSet;
		for (const FIntPoint& TileDelta : TileDeltas)
		{
			const int32 NeiX = TargetTileX + TileDelta.X;
			const int32 NeiY = TargetTileY + TileDelta.Y;
			RecastNavMesh->GetNavMeshTilesAt(NeiX, NeiY, TileSet);
		}

		const int32 DetailFlags =
			(1 << static_cast<int32>(ENavMeshDetailFlags::PolyEdges)) |
			(1 << static_cast<int32>(ENavMeshDetailFlags::FilledPolys)) |
			(1 << static_cast<int32>(ENavMeshDetailFlags::NavLinks)) |
			(FGameplayDebuggerCategoryNavmeshTweakables::bDrawExcludedFlags ? (1 << static_cast<int32>(ENavMeshDetailFlags::MarkForbiddenPolys)) : 0);

		// Do not attempt to gather render data when TileSet is empty otherwise the whole nav mesh will be displayed
		DataPack.bReferenceTooFarFromNavData = (TileSet.Num() == 0);
		if (DataPack.bReferenceTooFarFromNavData)
		{
			NavmeshRenderData.Reset();
		}
		else
		{
			NavmeshRenderData.GatherData(RecastNavMesh, DetailFlags, TileSet);
		}
	}
#endif // WITH_RECAST
}

void FGameplayDebuggerCategory_Navmesh::RetrieveRelativeTilesToDisplay(TArray<FIntPoint>& OutTileDelta)
{
	int32 NumTilesPerSide = FMath::Max(FGameplayDebuggerCategoryNavmeshTweakables::DisplaySize, 1);
	NumTilesPerSide += (NumTilesPerSide % 2 == 0) ? 1 : 0;

	const int32 NumTilesToDisplay = NumTilesPerSide * NumTilesPerSide;

	OutTileDelta.AddUninitialized(NumTilesToDisplay);

	const int32 MinIdx = -(NumTilesPerSide >> 1);
	for (int32 i=0; i < NumTilesToDisplay; ++i)
	{
		OutTileDelta[i] = FIntPoint(MinIdx + (i % NumTilesPerSide), MinIdx + (i / NumTilesPerSide));
	}
}

void FGameplayDebuggerCategory_Navmesh::DrawData(APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext)
{
	CanvasContext.Printf(TEXT("Num dirty areas: {%s}%d"), DataPack.NumDirtyAreas > 0 ? TEXT("red") : TEXT("green"), DataPack.NumDirtyAreas);
	CanvasContext.Printf(TEXT("Tile jobs running/remaining: %d / %d"), DataPack.NumRunningTasks, DataPack.NumRemainingTasks);

	if (DataPack.bIsNavBuildLocked)
	{
		CanvasContext.Printf(TEXT("Navigation Update is locked! Reason = '%s'. Navigation changes to the map are discarded."), *DataPack.NavBuildLockStatusDesc);
	}
	
	if (DataPack.bIsNavOctreeLocked)
	{
		CanvasContext.Printf(TEXT("Navigation Octree is locked! Changes to the map are not getting stored."));
	}

	if (DataPack.bIsNavDataRebuildingSuspended)
	{
		CanvasContext.Printf(TEXT("Navigation Data Generation is suspended! New dirty areas are queued (NumSuspendedDirtyAreas=%d)"), DataPack.NumSuspendedDirtyAreas);
	}	

	if (!DataPack.SupportedAgents.IsEmpty())
	{
		CanvasContext.Printf(TEXT("Supported Agents: %s"), *DataPack.SupportedAgents);
	}

	if (!DataPack.NavDataName.IsEmpty())
	{
		CanvasContext.Printf(TEXT("Navigation Data: {silver}%s%s"), *DataPack.NavDataName, DataPack.bReferenceTooFarFromNavData ? TEXT(" (too far from navmesh)") : TEXT(""));
	}

	if (DataPack.bCanCycleNavigationData)
	{
		CanvasContext.Printf(TEXT("[{yellow}%s{white}]: Cycle NavData"), *GetInputHandlerDescription(1));
	}

	if (DataPack.bCanChangeReference)
	{
		CanvasContext.Printf(TEXT("[{yellow}%s{white}]: Display around %s actor"), *GetInputHandlerDescription(0), DataPack.bIsUsingPlayerActor ? TEXT("Debug") : TEXT("Player"));
	}

	CanvasContext.Printf(TEXT("[{yellow}%s{white}]: %s"),
		*GetInputHandlerDescription(2),
		DataPack.LockedReferenceLocation != FNavigationSystem::InvalidLocation ? *FString::Printf(TEXT("Locked at %s"), *DataPack.LockedReferenceLocation.ToString()) : TEXT("Not Locked") );

}

void FGameplayDebuggerCategory_Navmesh::OnDataPackReplicated(int32 DataPackId)
{
	MarkRenderStateDirty();
}

FDebugRenderSceneProxy* FGameplayDebuggerCategory_Navmesh::CreateDebugSceneProxy(const UPrimitiveComponent* InComponent, FDebugDrawDelegateHelper*& OutDelegateHelper)
{
	FNavMeshSceneProxy* NavMeshSceneProxy = new FNavMeshSceneProxy(InComponent, &NavmeshRenderData, true);

#if WITH_RECAST && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	auto* OutDelegateHelper2 = new FNavMeshDebugDrawDelegateHelper();
	OutDelegateHelper2->InitDelegateHelper(NavMeshSceneProxy);
	OutDelegateHelper = OutDelegateHelper2;
#endif // WITH_RECAST && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	return NavMeshSceneProxy;
}

#endif // WITH_GAMEPLAY_DEBUGGER_MENU
