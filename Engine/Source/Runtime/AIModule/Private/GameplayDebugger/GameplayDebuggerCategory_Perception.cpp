// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebugger/GameplayDebuggerCategory_Perception.h"

#if WITH_GAMEPLAY_DEBUGGER_MENU

#include "GameFramework/Pawn.h"
#include "AIController.h"
#include "Perception/AIPerceptionComponent.h"

FGameplayDebuggerCategory_Perception::FGameplayDebuggerCategory_Perception()
{
}

TSharedRef<FGameplayDebuggerCategory> FGameplayDebuggerCategory_Perception::MakeInstance()
{
	return MakeShareable(new FGameplayDebuggerCategory_Perception());
}

void FGameplayDebuggerCategory_Perception::CollectData(APlayerController* OwnerPC, AActor* DebugActor)
{
	UAIPerceptionComponent* PerceptionComponent = nullptr;
	APawn* MyPawn = Cast<APawn>(DebugActor);
	if (MyPawn)
	{
		AController* Controller = MyPawn->GetController();
		if (AAIController* AIC = Cast<AAIController>(Controller))
		{
			PerceptionComponent = AIC->GetPerceptionComponent();
		}
		else
		{
			PerceptionComponent = MyPawn->FindComponentByClass<UAIPerceptionComponent>();
			// try the controller if the Pawn doesn't have it
			if (PerceptionComponent == nullptr && Controller)
			{
				PerceptionComponent = Controller->FindComponentByClass<UAIPerceptionComponent>();
			}
		}
	}

	if (PerceptionComponent == nullptr && DebugActor != nullptr)
	{
		PerceptionComponent = DebugActor->FindComponentByClass<UAIPerceptionComponent>();
	}

	if (PerceptionComponent)
	{
		PerceptionComponent->DescribeSelfToGameplayDebugger(this);
	}
}

#endif // WITH_GAMEPLAY_DEBUGGER_MENU
