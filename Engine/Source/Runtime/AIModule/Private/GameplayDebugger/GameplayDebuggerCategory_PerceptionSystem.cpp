// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebugger/GameplayDebuggerCategory_PerceptionSystem.h"

#if WITH_GAMEPLAY_DEBUGGER_MENU

#include "GameFramework/PlayerController.h"
#include "Perception/AIPerceptionSystem.h"

FGameplayDebuggerCategory_PerceptionSystem::FGameplayDebuggerCategory_PerceptionSystem()
{
	bShowOnlyWithDebugActor = false;
}

TSharedRef<FGameplayDebuggerCategory> FGameplayDebuggerCategory_PerceptionSystem::MakeInstance()
{
	return MakeShareable(new FGameplayDebuggerCategory_PerceptionSystem());
}

void FGameplayDebuggerCategory_PerceptionSystem::CollectData(APlayerController* OwnerPC, AActor* DebugActor)
{
	UWorld* World = GetDataWorld(OwnerPC, DebugActor);
	check(World);

	const UAIPerceptionSystem* PerceptionSystem = UAIPerceptionSystem::GetCurrent(*World);
	if (PerceptionSystem != nullptr)
	{
		PerceptionSystem->DescribeSelfToGameplayDebugger(*this);
	}
}

#endif // WITH_GAMEPLAY_DEBUGGER_MENU
