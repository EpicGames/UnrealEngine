// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeomUtils.h"

namespace UE::AI
{

	bool IntersectSegmentPoly2D(const FVector& Start, const FVector& End, TConstArrayView<FVector> Poly,
								FVector2D::FReal& OutTMin, FVector2D::FReal& OutTMax, int32& OutSegMin, int32& OutSegMax)
	{
		using FReal = FVector::FReal;

		OutTMin = 0.0;
		OutTMax = 1.0;
		OutSegMin = INDEX_NONE;
		OutSegMax = INDEX_NONE;

		const FVector Dir = End - Start;
		const int32 NumVerts = Poly.Num();
		
		for (int NextIndex = 0, Index = NumVerts - 1; NextIndex < NumVerts; Index = NextIndex++)
		{
			const FVector Edge = Poly[NextIndex] - Poly[Index];
			const FVector Diff = Start - Poly[Index];

			// Skip degenerate edges.
			if (Edge.SizeSquared2D() < UE_KINDA_SMALL_NUMBER)
			{
				continue;
			}
			
			const FReal N = Cross2D(Edge, Diff);
			const FReal D = Cross2D(Dir, Edge);
			
			if (FMath::Abs(D) < UE_KINDA_SMALL_NUMBER)
			{
				// S is nearly parallel to this edge
				if (N > 0.0)
				{
					return false;
				}
				continue;
			}
			
			const FReal T = N / D;
			
			if (D > 0.0)
			{
				// segment S is entering across this edge
				if (T > OutTMin)
				{
					OutTMin = T;
					OutSegMin = Index;
					// S enters after leaving polygon
					if (OutTMin > OutTMax)
					{
						return false;
					}
				}
			}
			else
			{
				// segment S is leaving across this edge
				if (T < OutTMax)
				{
					OutTMax = T;
					OutSegMax = Index;
					// S leaves before entering polygon
					if (OutTMax < OutTMin)
					{
						return false;
					}
				}
			}
		}
		
		return true;
	}

	FVector2D InvBilinear2D(const FVector Point, const FVector VertexA, const FVector VertexB, const FVector VertexC, const FVector VertexD)
	{
		// For more info how inverse bilinear works, see:
		//  - https://www.reedbeta.com/blog/quadrilateral-interpolation-part-2/
		//  - https://iquilezles.org/articles/ibilinear/
		
		using FReal = FVector::FReal;
		
		const FVector E = VertexB - VertexA;
		const FVector F = VertexD - VertexA;
		const FVector G = VertexA - VertexB + VertexC - VertexD;
		const FVector H = Point - VertexA;

		// The algorithm is sensitive to the floating point precisions because we're squaring squares.
		// Scaling the coefficients will help with the precision.
		const FReal K = 1.0 / FMath::Max3(1.0, E.SquaredLength(), F.SquaredLength());

		// Coefficient for solving the quadratic equation.
		const FReal A = Cross2D(G, F) * K;
		const FReal B = (Cross2D(E, F) + Cross2D(H, G)) * K;
		const FReal C = Cross2D(H, E) * K; 

		FReal U = 0.0;
		FReal V = 0.0;
		
		// if edges are parallel, this is a linear equation
		if (FMath::Abs(A) > UE_KINDA_SMALL_NUMBER)
		{
			// Quadratic solution
			const FReal W = B * B - 4.0 * A * C;
			if (W < 0)
			{
				return FVector2D(-1, -1);
			}
			V = (-B - FMath::Sqrt(W)) / (2.0 * A);
		}
		else if (FMath::Abs(B) > UE_KINDA_SMALL_NUMBER)
		{
			// Linear solution
			V = -C / B;
		}

		// Calculate U using V, use the larger denominator for better precision.
		const FReal DenomX = E.X + G.X * V;
		const FReal DenomY = E.Y + G.Y * V;
		if (FMath::Abs(DenomX) > FMath::Abs(DenomY))
		{
			U = (H.X - F.X * V) / DenomX;
		}
		else
		{
			U = (H.Y - F.Y * V) / DenomY;
		}
		
		return FVector2D(U, V);
	}

}; // UE::AI