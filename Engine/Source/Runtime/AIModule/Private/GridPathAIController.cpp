// Copyright Epic Games, Inc. All Rights Reserved.

#include "GridPathAIController.h"
#include "Navigation/GridPathFollowingComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GridPathAIController)

AGridPathAIController::AGridPathAIController(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<UGridPathFollowingComponent>(TEXT("PathFollowingComponent")))
{

}

