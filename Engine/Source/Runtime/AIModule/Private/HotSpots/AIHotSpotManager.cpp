// Copyright Epic Games, Inc. All Rights Reserved.

#include "HotSpots/AIHotSpotManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AIHotSpotManager)


UAIHotSpotManager::UAIHotSpotManager(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

