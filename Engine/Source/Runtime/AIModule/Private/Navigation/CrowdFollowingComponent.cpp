// Copyright Epic Games, Inc. All Rights Reserved.

#include "Navigation/CrowdFollowingComponent.h"
#include "NavigationSystem.h"
#include "NavMesh/RecastNavMesh.h"
#include "VisualLogger/VisualLoggerTypes.h"
#include "VisualLogger/VisualLogger.h"
#include "AIModuleLog.h"
#include "Navigation/CrowdManager.h"
#include "NavAreas/NavArea.h"
#include "AbstractNavData.h"
#include "AIConfig.h"
#include "Navigation/MetaNavMeshPath.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CrowdFollowingComponent)


DEFINE_LOG_CATEGORY(LogCrowdFollowing);

UCrowdFollowingComponent::UCrowdFollowingComponent(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	SimulationState = ECrowdSimulationState::Enabled;
		
	bAffectFallingVelocity = false;
	bRotateToVelocity = true;
	bSuspendCrowdSimulation = false;
	bEnableSimulationReplanOnResume = true;
	bIsTraversingLink = false;
	bRegisteredWithCrowdSimulation = false;
	bCanCheckMovingTooFar = true;
	bCanUpdatePathPartInTick = true;

	bEnableAnticipateTurns = false;
	bEnableObstacleAvoidance = true;
	bEnableSeparation = false;
	bEnableOptimizeVisibility = true;
	bEnableOptimizeTopology = true;
	bEnablePathOffset = false;
	bEnableSlowdownAtGoal = true;

	SeparationWeight = 2.0f;
	CollisionQueryRange = 400.0f;		// approx: radius * 12.0f
	PathOptimizationRange = 1000.0f;	// approx: radius * 30.0f
	AvoidanceQuality = ECrowdAvoidanceQuality::Low;
	AvoidanceRangeMultiplier = 1.0f;
}

FVector UCrowdFollowingComponent::GetCrowdAgentLocation() const
{
	return NavMovementInterface.IsValid() ? NavMovementInterface->GetFeetLocation() : FVector::ZeroVector;
}

FVector UCrowdFollowingComponent::GetCrowdAgentVelocity() const
{
	FVector Velocity(NavMovementInterface.IsValid() ? NavMovementInterface->GetVelocityForNavMovement() : FVector::ZeroVector);
	Velocity *= (GetStatus() == EPathFollowingStatus::Moving) ? 1.0f : 0.25f;
	return Velocity;
}

void UCrowdFollowingComponent::GetCrowdAgentCollisions(float& CylinderRadius, float& CylinderHalfHeight) const
{
	if (NavMovementInterface.IsValid() && NavMovementInterface->GetUpdatedObject())
	{
		if (const USceneComponent* UpdatedComponent = Cast<USceneComponent>(NavMovementInterface->GetUpdatedObject()))
		{
			UpdatedComponent->CalcBoundingCylinder(CylinderRadius, CylinderHalfHeight);
		}
	}
}

float UCrowdFollowingComponent::GetCrowdAgentMaxSpeed() const
{
	return NavMovementInterface.IsValid() ? NavMovementInterface->GetMaxSpeedForNavMovement() : 0.0f;
}

int32 UCrowdFollowingComponent::GetCrowdAgentAvoidanceGroup() const
{
	return GetAvoidanceGroup();
}

int32 UCrowdFollowingComponent::GetCrowdAgentGroupsToAvoid() const
{
	return GetGroupsToAvoid();
}

int32 UCrowdFollowingComponent::GetCrowdAgentGroupsToIgnore() const
{
	return GetGroupsToIgnore();
}

void UCrowdFollowingComponent::SetCrowdAnticipateTurns(bool bEnable, bool bUpdateAgent)
{
	if (bEnableAnticipateTurns != bEnable)
	{
		bEnableAnticipateTurns = bEnable;
		
		if (bUpdateAgent)
		{
			UpdateCrowdAgentParams();
		}
	}
}

void UCrowdFollowingComponent::SetCrowdObstacleAvoidance(bool bEnable, bool bUpdateAgent)
{
	if (bEnableObstacleAvoidance != bEnable)
	{
		bEnableObstacleAvoidance = bEnable;

		if (bUpdateAgent)
		{
			UpdateCrowdAgentParams();
		}
	}
}

void UCrowdFollowingComponent::SetCrowdSeparation(bool bEnable, bool bUpdateAgent)
{
	if (bEnableSeparation != bEnable)
	{
		bEnableSeparation = bEnable;

		if (bUpdateAgent)
		{
			UpdateCrowdAgentParams();
		}
	}
}

void UCrowdFollowingComponent::SetCrowdOptimizeVisibility(bool bEnable, bool bUpdateAgent)
{
	if (bEnableOptimizeVisibility != bEnable)
	{
		bEnableOptimizeVisibility = bEnable;

		if (bUpdateAgent)
		{
			UpdateCrowdAgentParams();
		}
	}
}

void UCrowdFollowingComponent::SetCrowdOptimizeTopology(bool bEnable, bool bUpdateAgent)
{
	if (bEnableOptimizeTopology != bEnable)
	{
		bEnableOptimizeTopology = bEnable;

		if (bUpdateAgent)
		{
			UpdateCrowdAgentParams();
		}
	}
}

void UCrowdFollowingComponent::SetCrowdSlowdownAtGoal(bool bEnable, bool bUpdateAgent)
{
	if (bEnableSlowdownAtGoal != bEnable)
	{
		bEnableSlowdownAtGoal = bEnable;

		if (bUpdateAgent)
		{
			UpdateCrowdAgentParams();
		}
	}
}

void UCrowdFollowingComponent::SetCrowdSeparationWeight(float Weight, bool bUpdateAgent)
{
	if (SeparationWeight != Weight)
	{
		SeparationWeight = Weight;

		if (bUpdateAgent)
		{
			UpdateCrowdAgentParams();
		}
	}
}

void UCrowdFollowingComponent::SetCrowdCollisionQueryRange(float Range, bool bUpdateAgent)
{
	if (CollisionQueryRange != Range)
	{
		CollisionQueryRange = Range;

		if (bUpdateAgent)
		{
			UpdateCrowdAgentParams();
		}
	}
}

void UCrowdFollowingComponent::SetCrowdPathOptimizationRange(float Range, bool bUpdateAgent)
{
	if (PathOptimizationRange != Range)
	{
		PathOptimizationRange = Range;

		if (bUpdateAgent)
		{
			UpdateCrowdAgentParams();
		}
	}
}

void UCrowdFollowingComponent::SetCrowdAvoidanceQuality(ECrowdAvoidanceQuality::Type Quality, bool bUpdateAgent)
{
	if (AvoidanceQuality != Quality)
	{
		AvoidanceQuality = Quality;

		if (bUpdateAgent)
		{
			UpdateCrowdAgentParams();
		}
	}
}

void UCrowdFollowingComponent::SetCrowdAvoidanceRangeMultiplier(float Multiplier, bool bUpdateAgent)
{
	if (AvoidanceRangeMultiplier != Multiplier)
	{
		AvoidanceRangeMultiplier = Multiplier;

		if (bUpdateAgent)
		{
			UpdateCrowdAgentParams();
		}
	}
}

void UCrowdFollowingComponent::SetCrowdPathOffset(bool bEnable, bool bUpdateAgent)
{
	if (bEnablePathOffset != bEnable)
	{
		bEnablePathOffset = bEnable;

		if (bUpdateAgent)
		{
			UpdateCrowdAgentParams();
		}
	}
}

void UCrowdFollowingComponent::SetCrowdAffectFallingVelocity(bool bEnable)
{
	bAffectFallingVelocity = bEnable;
}

void UCrowdFollowingComponent::SetCrowdRotateToVelocity(bool bEnable)
{
	bRotateToVelocity = bEnable;
}

void UCrowdFollowingComponent::SetAvoidanceGroup(int32 GroupFlags, bool bUpdateAgent)
{
	if (AvoidanceInterface.IsValid() && AvoidanceInterface->GetAvoidanceGroupMask() != GroupFlags)
	{
		AvoidanceInterface->SetAvoidanceGroupMask(GroupFlags);

		if (bUpdateAgent)
		{
			UpdateCrowdAgentParams();
		}
	}
}

void UCrowdFollowingComponent::SetGroupsToAvoid(int32 GroupFlags, bool bUpdateAgent)
{
	if (AvoidanceInterface.IsValid() && AvoidanceInterface->GetGroupsToAvoidMask() != GroupFlags)
	{
		AvoidanceInterface->SetGroupsToAvoidMask(GroupFlags);

		if (bUpdateAgent)
		{
			UpdateCrowdAgentParams();
		}
	}
}

void UCrowdFollowingComponent::SetGroupsToIgnore(int32 GroupFlags, bool bUpdateAgent)
{
	if (AvoidanceInterface.IsValid() && AvoidanceInterface->GetGroupsToIgnoreMask() != GroupFlags)
	{
		AvoidanceInterface->SetGroupsToIgnoreMask(GroupFlags);

		if (bUpdateAgent)
		{
			UpdateCrowdAgentParams();
		}
	}
}

int32 UCrowdFollowingComponent::GetAvoidanceGroup() const
{
	return AvoidanceInterface.IsValid() ? AvoidanceInterface->GetAvoidanceGroupMask() : 0;
}

int32 UCrowdFollowingComponent::GetGroupsToAvoid() const
{
	return AvoidanceInterface.IsValid() ? AvoidanceInterface->GetGroupsToAvoidMask() : 0;
}

int32 UCrowdFollowingComponent::GetGroupsToIgnore() const
{
	return AvoidanceInterface.IsValid() ? AvoidanceInterface->GetGroupsToIgnoreMask() : 0;
}

void UCrowdFollowingComponent::SuspendCrowdSteering(bool bSuspend)
{
	if (bSuspendCrowdSimulation != bSuspend)
	{
		bSuspendCrowdSimulation = bSuspend;
		UpdateCrowdAgentParams();
	}
}

void UCrowdFollowingComponent::UpdateCrowdAgentParams() const
{
	UCrowdManager* CrowdManager = UCrowdManager::GetCurrent(GetWorld());
	if (CrowdManager)
	{
		const ICrowdAgentInterface* IAgent = Cast<ICrowdAgentInterface>(this);
		CrowdManager->UpdateAgentParams(IAgent);
	}
}

void UCrowdFollowingComponent::UpdateCachedDirections(const FVector& NewVelocity, const FVector& NextPathCorner, bool bTraversingLink)
{
	// MoveSegmentDirection = direction on string pulled path
	const FVector AgentLoc = GetCrowdAgentLocation();
	const FVector ToCorner = NextPathCorner - AgentLoc;
	if (ToCorner.SizeSquared() > FMath::Square(10.0f))
	{
		MoveSegmentDirection = ToCorner.GetSafeNormal();
	}

	// Cache bTraversingLink
	bIsTraversingLink = bTraversingLink;

	// CrowdAgentMoveDirection either direction on path or aligned with current velocity
	const bool bIsNotFalling = (NavMovementInterface == nullptr || !NavMovementInterface->IsFalling());
	if (bIsNotFalling)
	{
		if (bUpdateDirectMoveVelocity)
		{
			const FVector CurrentTargetPt = DestinationActor.IsValid() ? DestinationActor->GetActorLocation() : GetCurrentTargetLocation();
			CrowdAgentMoveDirection = (CurrentTargetPt - AgentLoc).GetSafeNormal();
		}
		else
		{
			CrowdAgentMoveDirection = (bRotateToVelocity || bTraversingLink) && (NewVelocity.SizeSquared() > KINDA_SMALL_NUMBER) ? NewVelocity.GetSafeNormal() : MoveSegmentDirection;
		}
	}
}

bool UCrowdFollowingComponent::ShouldTrackMovingGoal(FVector& OutGoalLocation) const
{
	if (bIsUsingMetaPath)
	{
		FMetaNavMeshPath* MetaPath = Path.IsValid() ? Path->CastPath<FMetaNavMeshPath>() : nullptr;
		if (MetaPath && !MetaPath->IsLastSection())
		{
			return false;
		}
	}

	if (bFinalPathPart && !bUpdateDirectMoveVelocity &&
		Path.IsValid() && !Path->IsPartial() && Path->GetGoalActor())
	{
		OutGoalLocation = Path->GetGoalLocation();
		return true;
	}

	return false;
}

void UCrowdFollowingComponent::UpdateDestinationForMovingGoal(const FVector& NewDestination)
{
	CurrentDestination.Set(Path->GetBaseActor(), NewDestination);
}

void UCrowdFollowingComponent::ApplyCrowdAgentVelocity(const FVector& NewVelocity, const FVector& DestPathCorner, bool bTraversingLink, bool bIsNearEndOfPath)
{
	bCanCheckMovingTooFar = !bTraversingLink && bIsNearEndOfPath;
	if (IsCrowdSimulationEnabled() && GetStatus() == EPathFollowingStatus::Moving && NavMovementInterface.IsValid())
	{
		const bool bIsNotFalling = (NavMovementInterface == nullptr || !NavMovementInterface->IsFalling());
		if (bAffectFallingVelocity || bIsNotFalling)
		{
			UpdateCachedDirections(NewVelocity, DestPathCorner, bTraversingLink);

			const bool bAccelerationBased = NavMovementInterface->UseAccelerationForPathFollowing();
			if (bAccelerationBased)
			{
				const FVector::FReal MaxSpeed = GetCrowdAgentMaxSpeed();
				const FVector::FReal NewSpeed = NewVelocity.Size();
				const FVector::FReal SpeedPct = FMath::Clamp(NewSpeed / MaxSpeed, 0., 1.);
				const FVector MoveInput = FMath::IsNearlyZero(NewSpeed) ? FVector::ZeroVector : ((NewVelocity / NewSpeed) * SpeedPct);

				NavMovementInterface->RequestPathMove(MoveInput);
			}
			else
			{
				NavMovementInterface->RequestDirectMove(NewVelocity, false);
			}
		}
	}

	// call deprecated function in case someone is overriding it
	ApplyCrowdAgentVelocity(NewVelocity, DestPathCorner, bTraversingLink);
}

void UCrowdFollowingComponent::ApplyCrowdAgentPosition(const FVector& NewPosition)
{
	// base implementation does nothing
}

void UCrowdFollowingComponent::SetCrowdSimulationState(ECrowdSimulationState NewState)
{
	static FString CrowdSimulationDesc[] = { TEXT("Enabled"), TEXT("ObstacleOnly"), TEXT("Disabled") };

	if (NewState == SimulationState)
	{
		return;
	}

	if (GetStatus() != EPathFollowingStatus::Idle)
	{
		UE_VLOG(GetOwner(), LogCrowdFollowing, Warning, TEXT("SetCrowdSimulation failed, agent is not in Idle state!"));
		return;
	}

	UCrowdManager* Manager = UCrowdManager::GetCurrent(GetWorld());
	if (Manager == NULL)
	{
		UE_VLOG(GetOwner(), LogCrowdFollowing, Log, TEXT("SetCrowdSimulation: NewState %s: Crowd manager can't be found, disabling simulation."), *CrowdSimulationDesc[static_cast<uint8>(NewState)]);
		SimulationState = ECrowdSimulationState::Disabled;
		return;
	}

	UE_VLOG(GetOwner(), LogCrowdFollowing, Log, TEXT("SetCrowdSimulation: %s"), *CrowdSimulationDesc[static_cast<uint8>(NewState)]);

	const bool bNeedRegistration = (NewState != ECrowdSimulationState::Disabled);
	SimulationState = NewState;

	if (bNeedRegistration != bRegisteredWithCrowdSimulation)
	{
		ICrowdAgentInterface* IAgent = Cast<ICrowdAgentInterface>(this);
		if (bNeedRegistration)
		{
			check(IAgent != nullptr);			
			Manager->RegisterAgent(IAgent);
		}
		else
		{
			Manager->UnregisterAgent(IAgent);
		}

		bRegisteredWithCrowdSimulation = bNeedRegistration;
	}
}

void UCrowdFollowingComponent::Initialize()
{
	Super::Initialize();

	if (SimulationState != ECrowdSimulationState::Disabled)
	{
		RegisterCrowdAgent();

		if (!bRegisteredWithCrowdSimulation)
		{
			// crowd manager might not be created yet if this component was spawned during level's begin play (possessing a pawn placed in level)
			UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
			if (NavSys && !NavSys->IsInitialized())
			{
				NavSys->OnNavigationInitDone.AddUObject(this, &UCrowdFollowingComponent::OnNavigationInitDone);
			}
			else
			{
				SimulationState = ECrowdSimulationState::Disabled;
			}
		}
	}

}

void UCrowdFollowingComponent::Cleanup()
{
	Super::Cleanup();

	if (bRegisteredWithCrowdSimulation)
	{
		UCrowdManager* CrowdManager = UCrowdManager::GetCurrent(GetWorld());
		if (CrowdManager)
		{
			const ICrowdAgentInterface* IAgent = Cast<ICrowdAgentInterface>(this);
			CrowdManager->UnregisterAgent(IAgent);

			SimulationState = ECrowdSimulationState::Disabled;
			bRegisteredWithCrowdSimulation = false;
		}
	}
}

void UCrowdFollowingComponent::BeginDestroy()
{
	Super::BeginDestroy();

	// make sure it's not registered
	Cleanup();
}

void UCrowdFollowingComponent::AbortMove(const UObject& Instigator, FPathFollowingResultFlags::Type AbortFlags, FAIRequestID RequestID, EPathFollowingVelocityMode VelocityMode)
{
	if (IsCrowdSimulationEnabled() && (GetStatus() != EPathFollowingStatus::Idle) && RequestID.IsEquivalent(GetCurrentRequestId()))
	{
		UCrowdManager* CrowdManager = UCrowdManager::GetCurrent(GetWorld());
		if (CrowdManager)
		{
			CrowdManager->ClearAgentMoveTarget(this);
		}
	}

	Super::AbortMove(Instigator, AbortFlags, RequestID, VelocityMode);
}

void UCrowdFollowingComponent::PauseMove(FAIRequestID RequestID, EPathFollowingVelocityMode VelocityMode)
{
	if (IsCrowdSimulationEnabled() && (GetStatus() != EPathFollowingStatus::Paused) && RequestID.IsEquivalent(GetCurrentRequestId()))
	{
		UCrowdManager* CrowdManager = UCrowdManager::GetCurrent(GetWorld());
		if (CrowdManager)
		{
			CrowdManager->PauseAgent(this);
		}
	}

	Super::PauseMove(RequestID, VelocityMode);
}

void UCrowdFollowingComponent::ResumeMove(FAIRequestID RequestID)
{
	if (IsCrowdSimulationEnabled() && (GetStatus() == EPathFollowingStatus::Paused) && RequestID.IsEquivalent(GetCurrentRequestId()))
	{
		UCrowdManager* CrowdManager = UCrowdManager::GetCurrent(GetWorld());
		if (CrowdManager)
		{
			const bool bReplanPath = bEnableSimulationReplanOnResume && HasMovedDuringPause();
			CrowdManager->ResumeAgent(this, bReplanPath);
		}

		// reset cached direction, will be set again after velocity update
		// but before it happens do not change actor's focus point (rotation)
		CrowdAgentMoveDirection = FVector::ZeroVector;
	}

	Super::ResumeMove(RequestID);
}

void UCrowdFollowingComponent::Reset()
{
	Super::Reset();
	PathStartIndex = 0;
	LastPathPolyIndex = 0;

	bFinalPathPart = false;
	bCanCheckMovingTooFar = true;
	bCheckMovementAngle = false;
	bUpdateDirectMoveVelocity = false;
	bIsTraversingLink = false;
}

bool UCrowdFollowingComponent::IsFollowingNavLink() const
{
	if (IsCrowdSimulationEnabled())
	{
		return bIsTraversingLink;
	}
	else
	{
		return Super::IsFollowingNavLink();
	}
}

bool UCrowdFollowingComponent::UpdateMovementComponent(bool bForce)
{
	bool bRet = Super::UpdateMovementComponent(bForce);

	AvoidanceInterface = TWeakInterfacePtr<IRVOAvoidanceInterface>(NavMovementInterface.GetObject());

	return bRet;
}

void UCrowdFollowingComponent::OnLanded()
{
	// don't check overshot in the same frame, AI may require turning back after landing
	bCanCheckMovingTooFar = false;

	if (IsCrowdSimulationEnabled())
	{
		UCrowdManager* CrowdManager = UCrowdManager::GetCurrent(GetWorld());
		if (CrowdManager)
		{
			const ICrowdAgentInterface* IAgent = Cast<ICrowdAgentInterface>(this);
			CrowdManager->UpdateAgentState(IAgent);
		}
	}
}

void UCrowdFollowingComponent::FinishUsingCustomLink(INavLinkCustomInterface* CustomNavLink)
{
	const bool bPrevCustomLink = CurrentCustomLinkOb.IsValid();
	Super::FinishUsingCustomLink(CustomNavLink);

	if (IsCrowdSimulationEnabled())
	{
		const bool bCurrentCustomLink = CurrentCustomLinkOb.IsValid();
		UCrowdManager* CrowdManager = UCrowdManager::GetCurrent(GetWorld());
		if (bPrevCustomLink && !bCurrentCustomLink && CrowdManager)
		{
			const ICrowdAgentInterface* IAgent = Cast<ICrowdAgentInterface>(this);
			CrowdManager->OnAgentFinishedCustomLink(IAgent);
		}
	}
}

void UCrowdFollowingComponent::OnPathFinished(const FPathFollowingResult& Result)
{
	UCrowdManager* CrowdManager = UCrowdManager::GetCurrent(GetWorld());
	if (IsCrowdSimulationEnabled() && CrowdManager)
	{
		CrowdManager->ClearAgentMoveTarget(this);
	}

	Super::OnPathFinished(Result);
}

void UCrowdFollowingComponent::OnPathUpdated()
{
	PathStartIndex = 0;
	LastPathPolyIndex = 0;
}

void UCrowdFollowingComponent::OnPathfindingQuery(FPathFindingQuery& Query)
{
	// disable path post processing (string pulling), crowd simulation needs to handle 
	// large paths by splitting into smaller parts and optimization gets in the way

	if (IsCrowdSimulationEnabled())
	{
		Query.NavDataFlags |= ERecastPathFlags::SkipStringPulling;
	}
}

bool UCrowdFollowingComponent::ShouldCheckPathOnResume() const
{
	if (IsCrowdSimulationEnabled())
	{
		// never call SetMoveSegment on resuming
		return false;
	}

	return HasMovedDuringPause();
}

bool UCrowdFollowingComponent::HasMovedDuringPause() const
{
	return Super::ShouldCheckPathOnResume();
}

bool UCrowdFollowingComponent::IsOnPath() const
{
	if (IsCrowdSimulationEnabled())
	{
		// agent can move off path for steering/avoidance purposes
		// just pretend it's always on path to avoid problems when movement is being resumed
		return true;
	}

	return Super::IsOnPath();
}

int32 UCrowdFollowingComponent::DetermineStartingPathPoint(const FNavigationPath* ConsideredPath) const
{
	int32 StartIdx = 0;

	if (IsCrowdSimulationEnabled())
	{
		StartIdx = PathStartIndex;

		// no path = called from SwitchToNextPathPart
		if (ConsideredPath == NULL && Path.IsValid())
		{
			StartIdx = LastPathPolyIndex;
		}
	}
	else
	{
		StartIdx = Super::DetermineStartingPathPoint(ConsideredPath);
	}

	return StartIdx;
}

void LogPathPartHelper(AActor* LogOwner, FNavMeshPath* NavMeshPath, int32 StartIdx, int32 EndIdx)
{
#if ENABLE_VISUAL_LOG && WITH_RECAST
	ARecastNavMesh* NavMesh = Cast<ARecastNavMesh>(NavMeshPath->GetNavigationDataUsed());
	FVisualLogger& VisualLogger = FVisualLogger::Get();

	if (NavMesh == NULL ||
		!VisualLogger.IsCategoryLogged(LogNavigation) ||
		!NavMeshPath->PathCorridor.IsValidIndex(StartIdx) ||
		!NavMeshPath->PathCorridor.IsValidIndex(EndIdx))
	{
		return;
	}

	FVisualLogShapeElement CorridorPoly(EVisualLoggerShapeElement::Polygon);
	CorridorPoly.SetColor(FColorList::Cyan.WithAlpha(100));
	CorridorPoly.Category = LogNavigation.GetCategoryName();
	CorridorPoly.Points.Reserve((EndIdx - StartIdx) * 6);

	const FVector CorridorOffset = NavigationDebugDrawing::PathOffset * 1.25f;
	int32 NumAreaMark = 1;

	if (FVisualLogEntry* Snapshot = FVisualLogger::GetEntryToWrite(LogOwner, LogCrowdFollowing))
	{
		NavMesh->BeginBatchQuery();

		TArray<FVector> Verts;
		for (int32 Idx = StartIdx; Idx <= EndIdx; Idx++)
		{
			const uint8 AreaID = IntCastChecked<uint8>(NavMesh->GetPolyAreaID(NavMeshPath->PathCorridor[Idx]));
			const UClass* AreaClass = NavMesh->GetAreaClass(AreaID);

			Verts.Reset();
			NavMesh->GetPolyVerts(NavMeshPath->PathCorridor[Idx], Verts);

			FVector CenterPt = FVector::ZeroVector;
			for (int32 VIdx = 0; VIdx < Verts.Num(); VIdx++)
			{
				Verts[VIdx].Z += 5.0f;
				CenterPt += Verts[VIdx];
			}
			CenterPt /= Verts.Num();

			const UNavArea* DefArea = AreaClass ? ((UClass*)AreaClass)->GetDefaultObject<UNavArea>() : NULL;
			const FColor PolygonColor = AreaClass != FNavigationSystem::GetDefaultWalkableArea() ? (DefArea ? DefArea->DrawColor : NavMesh->GetConfig().Color) : FColorList::LightSteelBlue;

			CorridorPoly.SetColor(PolygonColor.WithAlpha(100));
			CorridorPoly.Points.Reset();
			CorridorPoly.Points.Append(Verts);
			Snapshot->ElementsToDraw.Add(CorridorPoly);

			if (AreaClass && AreaClass != FNavigationSystem::GetDefaultWalkableArea())
			{
				FVisualLogShapeElement AreaMarkElem(EVisualLoggerShapeElement::Segment);
				AreaMarkElem.SetColor(FColorList::Orange.WithAlpha(100));
				AreaMarkElem.Category = LogNavigation.GetCategoryName();
				AreaMarkElem.Thickness = 2;
				AreaMarkElem.Description = AreaClass->GetName();

				AreaMarkElem.Points.Add(CenterPt + CorridorOffset);
				AreaMarkElem.Points.Add(CenterPt + CorridorOffset + FVector(0, 0, 100.0f + NumAreaMark * 50.0f));
				Snapshot->ElementsToDraw.Add(AreaMarkElem);

				NumAreaMark = (NumAreaMark + 1) % 5;
			}
		}

		NavMesh->FinishBatchQuery();
	}
#endif // ENABLE_VISUAL_LOG && WITH_RECAST
}

void UCrowdFollowingComponent::SetMoveSegment(int32 SegmentStartIndex)
{
	if (!IsCrowdSimulationEnabled())
	{
		Super::SetMoveSegment(SegmentStartIndex);
		return;
	}

	PathStartIndex = SegmentStartIndex;
	LastPathPolyIndex = PathStartIndex;
	if (Path.IsValid() == false || Path->IsValid() == false || GetOwner() == NULL)
	{
		return;
	}
	
	FVector CurrentTargetPt = Path->GetPathPoints().Last().Location;

	FNavMeshPath* NavMeshPath = Path->CastPath<FNavMeshPath>();
	FAbstractNavigationPath* DirectPath = Path->CastPath<FAbstractNavigationPath>();
	if (NavMeshPath)
	{
#if WITH_RECAST
		if (NavMeshPath->PathCorridor.Num() == 0)
		{
			UE_VLOG(GetOwner(), LogCrowdFollowing, Error, TEXT("Can't switch path segments: empty path corridor!"));
			OnPathFinished(FPathFollowingResult(EPathFollowingResult::Aborted, FPathFollowingResultFlags::InvalidPath));
			return;
		}
		else if (NavMeshPath->PathCorridor.IsValidIndex(PathStartIndex) == false)
		{
			// this should never matter, but just in case
			UE_VLOG(GetOwner(), LogCrowdFollowing, Error, TEXT("SegmentStartIndex in call to UCrowdFollowingComponent::SetMoveSegment is out of path corridor array's bounds (index: %d, array size %d)")
				, PathStartIndex, NavMeshPath->PathCorridor.Num());
			PathStartIndex = FMath::Clamp<int32>(PathStartIndex, 0, NavMeshPath->PathCorridor.Num() - 1);
		}

		// cut paths into parts to avoid problems with crowds getting into local minimum
		// due to using only first 10 steps of A*

		// do NOT use PathPoints here, crowd simulation disables path post processing
		// which means, that PathPoints contains only start and end position 
		// full path is available through PathCorridor array (poly refs)

		UCrowdManager* CrowdManager = UCrowdManager::GetCurrent(GetWorld());
		if (CrowdManager == nullptr)
		{
			UE_VLOG(GetOwner(), LogCrowdFollowing, Error, TEXT("Can't switch path segments: missing crowd manager!"));
			OnPathFinished(FPathFollowingResult(EPathFollowingResult::Aborted, FPathFollowingResultFlags::InvalidPath));
			return;
		}

		ARecastNavMesh* RecastNavData = Cast<ARecastNavMesh>(NavMeshPath->GetNavigationDataUsed());
		if (RecastNavData == nullptr)
		{
			UE_VLOG(GetOwner(), LogCrowdFollowing, Error, TEXT("Invalid navigation data in UCrowdFollowingComponent::SetMoveSegment, expected ARecastNavMesh class, got: %s"), *GetNameSafe(NavMeshPath->GetNavigationDataUsed()));
			OnPathFinished(FPathFollowingResult(EPathFollowingResult::Aborted, FPathFollowingResultFlags::InvalidPath));
			return;
		}
		else if (CrowdManager->GetNavData() != RecastNavData)
		{
			UE_VLOG(GetOwner(), LogCrowdFollowing, Error, TEXT("Invalid navigation data in UCrowdFollowingComponent::SetMoveSegment, expected 0x%X, got: 0x%X"), CrowdManager->GetNavData(), RecastNavData);
			OnPathFinished(FPathFollowingResult(EPathFollowingResult::Aborted, FPathFollowingResultFlags::InvalidPath));
			return;
		}

		const int32 PathPartSize = 15;
		const int32 LastPolyIdx = NavMeshPath->PathCorridor.Num() - 1;
		int32 PathPartEndIdx = FMath::Min(PathStartIndex + PathPartSize, LastPolyIdx);
		bFinalPathPart = (PathPartEndIdx == LastPolyIdx);

		FVector PtA, PtB;
		const bool bStartIsNavLink = RecastNavData->GetLinkEndPoints(NavMeshPath->PathCorridor[PathStartIndex], PtA, PtB);
		if (bStartIsNavLink)
		{
			PathStartIndex = FMath::Max(0, PathStartIndex - 1);
		}

		if (!bFinalPathPart)
		{
			const bool bEndIsNavLink = RecastNavData->GetLinkEndPoints(NavMeshPath->PathCorridor[PathPartEndIdx], PtA, PtB);
			const bool bSwitchIsNavLink = (PathPartEndIdx > 0) ? RecastNavData->GetLinkEndPoints(NavMeshPath->PathCorridor[PathPartEndIdx - 1], PtA, PtB) : false;
			if (bEndIsNavLink)
			{
				PathPartEndIdx = FMath::Max(0, PathPartEndIdx - 1);
			}
			if (bSwitchIsNavLink)
			{
				PathPartEndIdx = FMath::Max(0, PathPartEndIdx - 2);
			}

			RecastNavData->GetPolyCenter(NavMeshPath->PathCorridor[PathPartEndIdx], CurrentTargetPt);
		}
		else if (NavMeshPath->IsPartial())
		{
			RecastNavData->GetClosestPointOnPoly(NavMeshPath->PathCorridor[PathPartEndIdx], Path->GetPathPoints().Last().Location, CurrentTargetPt);
		}

		// not safe to read those directions yet, you have to wait until crowd manager gives you next corner of string pulled path
		CrowdAgentMoveDirection = FVector::ZeroVector;
		MoveSegmentDirection = FVector::ZeroVector;

		CurrentDestination.Set(Path->GetBaseActor(), CurrentTargetPt);

		LogPathPartHelper(GetOwner(), NavMeshPath, PathStartIndex, PathPartEndIdx);
		UE_VLOG_SEGMENT(GetOwner(), LogCrowdFollowing, Log, NavMovementInterface->GetFeetLocation(), CurrentTargetPt, FColor::Red, TEXT("path part"));
		UE_VLOG(GetOwner(), LogCrowdFollowing, Log, TEXT("SetMoveSegment, from:%d segments:%d%s"),
			PathStartIndex, (PathPartEndIdx - PathStartIndex)+1, bFinalPathPart ? TEXT(" (final)") : TEXT(""));

		CrowdManager->SetAgentMovePath(this, NavMeshPath, PathStartIndex, PathPartEndIdx, CurrentTargetPt);
#endif
	}
	else if (DirectPath)
	{
		// direct paths are not using any steering or avoidance
		// pathfinding is replaced with simple velocity request 

		const FVector AgentLoc = NavMovementInterface->GetFeetLocation();

		bFinalPathPart = true;
		bCheckMovementAngle = true;
		bUpdateDirectMoveVelocity = true;
		CurrentDestination.Set(Path->GetBaseActor(), CurrentTargetPt);
		CrowdAgentMoveDirection = (CurrentTargetPt - AgentLoc).GetSafeNormal();
		MoveSegmentDirection = CrowdAgentMoveDirection;

		UE_VLOG(GetOwner(), LogCrowdFollowing, Log, TEXT("SetMoveSegment, direct move"));
		UE_VLOG_SEGMENT(GetOwner(), LogCrowdFollowing, Log, AgentLoc, CurrentTargetPt, FColor::Red, TEXT("path"));

		UCrowdManager* CrowdManager = UCrowdManager::GetCurrent(GetWorld());
		if (CrowdManager)
		{
			CrowdManager->SetAgentMoveDirection(this, CrowdAgentMoveDirection);
		}
	}
	else
	{
		UE_VLOG(GetOwner(), LogCrowdFollowing, Error, TEXT("SetMoveSegment, unknown path type!"));
	}
}

void UCrowdFollowingComponent::UpdatePathSegment()
{
	if (!IsCrowdSimulationEnabled())
	{
		Super::UpdatePathSegment();
		return;
	}

	if (!Path.IsValid() || NavMovementInterface == NULL)
	{
		OnPathFinished(FPathFollowingResult(EPathFollowingResult::Aborted, FPathFollowingResultFlags::InvalidPath));
		return;
	}

	if (!Path->IsValid())
	{
		if (!Path->IsWaitingForRepath())
		{
			UE_VLOG(this, LogPathFollowing, Log, TEXT("Aborting move due to path being invalid and not waiting for repath"));
			OnPathFinished(FPathFollowingResult(EPathFollowingResult::Aborted, FPathFollowingResultFlags::InvalidPath));
			return;
		}
		else
		{
			// continue with execution, if navigation is being rebuild constantly AI will get stuck with current waypoint
			// path updates should be still coming in, even though they get invalidated right away
			UE_VLOG(this, LogPathFollowing, Log, TEXT("Updating path points in invalid & pending path!"));
		}
	}

	// if agent has control over its movement, check finish conditions
	const FVector CurrentLocation = NavMovementInterface->GetFeetLocation();
	const bool bCanReachTarget = NavMovementInterface->CanStopPathFollowing();
	if (bCanReachTarget && GetStatus() == EPathFollowingStatus::Moving)
	{
		const FVector GoalLocation = GetCurrentTargetLocation();

		if (bCollidedWithGoal)
		{
			// check if collided with goal actor
			OnSegmentFinished();
			OnPathFinished(FPathFollowingResult(EPathFollowingResult::Success, FPathFollowingResultFlags::None));
		}
		else if (bFinalPathPart)
		{
			const FVector ToTarget = (GoalLocation - NavMovementInterface->GetFeetLocation());
			const bool bMovedTooFar = (bCheckMovementAngle || bCanCheckMovingTooFar) && !CrowdAgentMoveDirection.IsNearlyZero() && FVector::DotProduct(ToTarget, CrowdAgentMoveDirection) < 0.0;

#if ENABLE_VISUAL_LOG
			if (bMovedTooFar)
			{
				const FVector AgentLoc = NavMovementInterface->GetFeetLocation();
				UE_VLOG_SEGMENT(GetOwner(), LogCrowdFollowing, Log, AgentLoc, AgentLoc + CrowdAgentMoveDirection * 100.0f, FColor::Cyan, TEXT("moveDir"));
				UE_VLOG_SEGMENT(GetOwner(), LogCrowdFollowing, Log, AgentLoc, AgentLoc + ToTarget.GetSafeNormal() * 100.0f, FColor::Cyan, TEXT("toTarget"));
				UE_VLOG(GetOwner(), LogCrowdFollowing, Log, TEXT("Moved too far, dotValue: %.2f (normalized dot: %.2f) velocity:%s (speed:%.0f)"),
					FVector::DotProduct(ToTarget, CrowdAgentMoveDirection),
					FVector::DotProduct(ToTarget.GetSafeNormal(), CrowdAgentMoveDirection),
					*NavMovementInterface->GetVelocityForNavMovement().ToString(),
					NavMovementInterface->GetVelocityForNavMovement().Size()
					);
			}
#endif

			// can't use HasReachedDestination here, because it will use last path point
			// which is not set correctly for partial paths without string pulling
			const float UseAcceptanceRadius = GetFinalAcceptanceRadius(*Path, OriginalMoveRequestGoalLocation, &GoalLocation);
			if (bMovedTooFar || HasReachedInternal(GoalLocation, 0.0f, 0.0f, CurrentLocation, UseAcceptanceRadius, bReachTestIncludesAgentRadius ? MinAgentRadiusPct : 0.0f))
			{
				UE_VLOG(GetOwner(), LogCrowdFollowing, Log, TEXT("Last path segment finished due to \'%s\'"), bMovedTooFar ? TEXT("Missing Last Point") : TEXT("Reaching Destination"));
				OnPathFinished(FPathFollowingResult(EPathFollowingResult::Success, FPathFollowingResultFlags::None));
			}
		}
		else if (bCanUpdatePathPartInTick)
		{
			// override radius multiplier and switch to next path part when closer than 4x agent radius
			const float NextPartMultiplier = 4.0f;
			const bool bHasReached = HasReachedInternal(GoalLocation, 0.0f, 0.0f, CurrentLocation, 0.0f, NextPartMultiplier);

			if (bHasReached)
			{
				SwitchToNextPathPart();
			}
		}
	}

	if (bCanReachTarget && GetStatus() == EPathFollowingStatus::Moving)
	{
		// check waypoint switch condition in meta paths
		FMetaNavMeshPath* MetaNavPath = bIsUsingMetaPath ? Path->CastPath<FMetaNavMeshPath>() : nullptr;
		if (MetaNavPath && GetStatus() == EPathFollowingStatus::Moving)
		{
			MetaNavPath->ConditionalMoveToNextSection(CurrentLocation, EMetaPathUpdateReason::MoveTick);
		}

		// gather location samples to detect if moving agent is blocked
		const bool bHasNewSample = UpdateBlockDetection();
		if (bHasNewSample && IsBlocked())
		{
			OnPathFinished(FPathFollowingResult(EPathFollowingResult::Blocked, FPathFollowingResultFlags::None));
		}
	}
}

void UCrowdFollowingComponent::FollowPathSegment(float DeltaTime)
{
	if (!IsCrowdSimulationEnabled())
	{
		Super::FollowPathSegment(DeltaTime);
	}
	else if (bUpdateDirectMoveVelocity)
	{
		const FVector CurrentTargetPt = DestinationActor.IsValid() ? DestinationActor->GetActorLocation() : GetCurrentTargetLocation();
		const FVector AgentLoc = GetCrowdAgentLocation();
		const FVector NewDirection = (CurrentTargetPt - AgentLoc).GetSafeNormal();

		const bool bDirectionChanged = !NewDirection.Equals(CrowdAgentMoveDirection);
		if (bDirectionChanged)
		{
			CurrentDestination.Set(Path->GetBaseActor(), CurrentTargetPt);
			CrowdAgentMoveDirection = NewDirection;
			MoveSegmentDirection = NewDirection;

			UCrowdManager* Manager = UCrowdManager::GetCurrent(GetWorld());
			Manager->SetAgentMoveDirection(this, NewDirection);

			UE_VLOG(GetOwner(), LogCrowdFollowing, Log, TEXT("Updated direct move direction for crowd agent."));
		}
	}

	UpdateMoveFocus();
}

FVector UCrowdFollowingComponent::GetMoveFocus(bool bAllowStrafe) const
{
	// can't really use CurrentDestination here, as it's pointing at end of path part
	// fallback to looking at point in front of agent

	if (!bAllowStrafe && NavMovementInterface.IsValid() && IsCrowdSimulationEnabled())
	{
		const FVector AgentLoc = NavMovementInterface->GetLocation();

		// if we're not moving, falling, or don't have a crowd agent move direction, set our focus to ahead of the rotation of our owner to keep the same rotation,
		// otherwise use the Crowd Agent Move Direction to move in the direction we're supposed to be going
		const FVector ForwardDir = NavMovementInterface->GetOwnerAsObject() && ((GetStatus() != EPathFollowingStatus::Moving) || (NavMovementInterface->IsFalling()) || CrowdAgentMoveDirection.IsNearlyZero()) ?
			NavMovementInterface->GetForwardVector() :
			CrowdAgentMoveDirection.GetSafeNormal2D();

		return AgentLoc + (ForwardDir * FAIConfig::Navigation::FocalPointDistance);
	}

	return Super::GetMoveFocus(bAllowStrafe);
}

void UCrowdFollowingComponent::OnNavNodeChanged(NavNodeRef NewPolyRef, NavNodeRef PrevPolyRef, int32 CorridorSize)
{
	if (IsCrowdSimulationEnabled() && GetStatus() != EPathFollowingStatus::Idle)
	{
		// update last visited path poly
		FNavMeshPath* NavPath = Path.IsValid() ? Path->CastPath<FNavMeshPath>() : NULL;
		if (NavPath)
		{
			for (int32 Idx = LastPathPolyIndex; Idx < NavPath->PathCorridor.Num(); Idx++)
			{
				if (NavPath->PathCorridor[Idx] == NewPolyRef)
				{
					LastPathPolyIndex = Idx;
					break;
				}
			}
		}

		UE_VLOG(GetOwner(), LogCrowdFollowing, Verbose, TEXT("OnNavNodeChanged, CorridorSize:%d, LastVisitedIndex:%d"), CorridorSize, LastPathPolyIndex);

		const bool bSwitchPart = ShouldSwitchPathPart(CorridorSize);
		if (bSwitchPart && !bFinalPathPart)
		{
			SwitchToNextPathPart();
		}
	}
}

bool UCrowdFollowingComponent::ShouldSwitchPathPart(int32 CorridorSize) const
{
	return CorridorSize <= 2;
}

void UCrowdFollowingComponent::SwitchToNextPathPart()
{
	const int32 NewPartStart = DetermineStartingPathPoint(NULL);
	SetMoveSegment(NewPartStart);
}

void UCrowdFollowingComponent::RegisterCrowdAgent()
{
	UCrowdManager* CrowdManager = UCrowdManager::GetCurrent(GetWorld());
	if (CrowdManager)
	{
		ICrowdAgentInterface* IAgent = Cast<ICrowdAgentInterface>(this);
		check(IAgent != nullptr);
		CrowdManager->RegisterAgent(IAgent);
		bRegisteredWithCrowdSimulation = true;
	}
	else
	{
		bRegisteredWithCrowdSimulation = false;
	}
}

void UCrowdFollowingComponent::OnNavigationInitDone()
{
	Super::OnNavigationInitDone();

	RegisterCrowdAgent();
	
	if (!bRegisteredWithCrowdSimulation)
	{
		SimulationState = ECrowdSimulationState::Disabled;
	}

	// set movement component, it will cache MyNavData from its NavAgentProperties
	SetNavMovementInterface(NavMovementInterface.Get());
}

void UCrowdFollowingComponent::GetDebugStringTokens(TArray<FString>& Tokens, TArray<EPathFollowingDebugTokens::Type>& Flags) const
{
	if (!IsCrowdSimulationEnabled())
	{
		Super::GetDebugStringTokens(Tokens, Flags);
		return;
	}

	Tokens.Add(GetStatusDesc());
	Flags.Add(EPathFollowingDebugTokens::Description);

	UCrowdManager* Manager = UCrowdManager::GetCurrent(GetWorld());
	if (Manager && !Manager->IsAgentValid(this))
	{
		Tokens.Add(TEXT("simulation"));
		Flags.Add(EPathFollowingDebugTokens::ParamName);
		Tokens.Add(TEXT("NOT ACTIVE"));
		Flags.Add(EPathFollowingDebugTokens::FailedValue);
	}

	if (GetStatus() != EPathFollowingStatus::Moving)
	{
		return;
	}

	FString& StatusDesc = Tokens[0];
	if (Path.IsValid())
	{
		FNavMeshPath* NavMeshPath = Path->CastPath<FNavMeshPath>();
		if (NavMeshPath)
		{
			StatusDesc += FString::Printf(TEXT(" (path:%d, visited:%d)"), PathStartIndex, LastPathPolyIndex);
		}
		else if (Path->CastPath<FAbstractNavigationPath>())
		{
			StatusDesc += TEXT(" (direct)");
		}
		else
		{
			StatusDesc += TEXT(" (unknown path)");
		}
	}

	// get cylinder of moving agent
	float AgentRadius = 0.0f;
	float AgentHalfHeight = 0.0f;
	NavMovementInterface->GetSimpleCollisionCylinder(AgentRadius, AgentHalfHeight);

	if (bFinalPathPart)
	{
		float CurrentDot = 0.0f, CurrentDistance = 0.0f, CurrentHeight = 0.0f;
		uint8 bFailedDot = 0, bFailedDistance = 0, bFailedHeight = 0;
		DebugReachTest(CurrentDot, CurrentDistance, CurrentHeight, bFailedHeight, bFailedDistance, bFailedHeight);

		Tokens.Add(TEXT("dist2D"));
		Flags.Add(EPathFollowingDebugTokens::ParamName);
		Tokens.Add(FString::Printf(TEXT("%.0f"), CurrentDistance));
		Flags.Add(bFailedDistance ? EPathFollowingDebugTokens::FailedValue : EPathFollowingDebugTokens::PassedValue);

		Tokens.Add(TEXT("distZ"));
		Flags.Add(EPathFollowingDebugTokens::ParamName);
		Tokens.Add(FString::Printf(TEXT("%.0f"), CurrentHeight));
		Flags.Add(bFailedHeight ? EPathFollowingDebugTokens::FailedValue : EPathFollowingDebugTokens::PassedValue);
	}
	else
	{
		const FVector CurrentLocation = NavMovementInterface->GetFeetLocation();

		// make sure we're not too close to end of path part (poly count can always fail when AI goes off path)
		const FVector::FReal DistSq = (GetCurrentTargetLocation() - CurrentLocation).SizeSquared();
		const float PathSwitchThresSq = FMath::Square(AgentRadius * 5.0f);

		Tokens.Add(TEXT("distance"));
		Flags.Add(EPathFollowingDebugTokens::ParamName);
		Tokens.Add(FString::Printf(TEXT("%.0f"), FMath::Sqrt(DistSq)));
		Flags.Add((DistSq < PathSwitchThresSq) ? EPathFollowingDebugTokens::PassedValue : EPathFollowingDebugTokens::FailedValue);
	}
}

#if ENABLE_VISUAL_LOG

void UCrowdFollowingComponent::DescribeSelfToVisLog(FVisualLogEntry* Snapshot) const
{
	if (!IsCrowdSimulationEnabled())
	{
		Super::DescribeSelfToVisLog(Snapshot);
		return;
	}

	FVisualLogStatusCategory Category;
	Category.Category = TEXT("Path following");

	if (DestinationActor.IsValid())
	{
		Category.Add(TEXT("Goal"), GetNameSafe(DestinationActor.Get()));
	}

	FString StatusDesc = GetStatusDesc();

	FNavMeshPath* NavMeshPath = Path.IsValid() ? Path->CastPath<FNavMeshPath>() : NULL;
	FAbstractNavigationPath* DirectPath = Path.IsValid() ? Path->CastPath<FAbstractNavigationPath>() : NULL;

	if (GetStatus() == EPathFollowingStatus::Moving)
	{
		StatusDesc += FString::Printf(TEXT(" [path:%d, visited:%d]"), PathStartIndex, LastPathPolyIndex);
	}

	Category.Add(TEXT("Status"), StatusDesc);
	Category.Add(TEXT("Path"), !Path.IsValid() ? TEXT("none") : NavMeshPath ? TEXT("navmesh") : DirectPath ? TEXT("direct") : TEXT("unknown"));

	UObject* CustomLinkOb = GetCurrentCustomLinkOb();
	if (CustomLinkOb)
	{
		Category.Add(TEXT("SmartLink"), CustomLinkOb->GetName());
	}

	UCrowdManager* Manager = UCrowdManager::GetCurrent(GetWorld());
	if (Manager && !Manager->IsAgentValid(this))
	{
		Category.Add(TEXT("Simulation"), TEXT("unable to register!"));
	}

	Snapshot->Status.Add(Category);
}

#endif // ENABLE_VISUAL_LOG

