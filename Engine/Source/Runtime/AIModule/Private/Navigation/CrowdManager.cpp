// Copyright Epic Games, Inc. All Rights Reserved.

#include "Navigation/CrowdManager.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/Controller.h"
#include "NavigationSystem.h"
#include "NavFilters/NavigationQueryFilter.h"
#include "NavMesh/RecastNavMesh.h"
#include "VisualLogger/VisualLogger.h"
#include "AIModuleLog.h"
#include "Engine/Engine.h"
#include "Math/UnrealMathUtility.h"

#if WITH_RECAST
#include "NavMesh/RecastHelpers.h"
#include "DetourCrowd/DetourObstacleAvoidance.h"
#include "DetourCrowd/DetourCrowd.h"
#include "Detour/DetourNavMesh.h"
#include "NavMesh/RecastQueryFilter.h"
#endif

#include "Navigation/CrowdFollowingComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CrowdManager)

DECLARE_STATS_GROUP(TEXT("Crowd"), STATGROUP_AICrowd, STATCAT_Advanced);

DECLARE_CYCLE_STAT(TEXT("Nav Tick: crowd simulation"), STAT_AI_Crowd_Tick, STATGROUP_AICrowd);
DECLARE_CYCLE_STAT(TEXT("Step: corridor update"), STAT_AI_Crowd_StepCorridorTime, STATGROUP_AICrowd);
DECLARE_CYCLE_STAT(TEXT("Step: paths"), STAT_AI_Crowd_StepPathsTime, STATGROUP_AICrowd);
DECLARE_CYCLE_STAT(TEXT("Step: proximity"), STAT_AI_Crowd_StepProximityTime, STATGROUP_AICrowd);
DECLARE_CYCLE_STAT(TEXT("Step: next point"), STAT_AI_Crowd_StepNextPointTime, STATGROUP_AICrowd);
DECLARE_CYCLE_STAT(TEXT("Step: steering"), STAT_AI_Crowd_StepSteeringTime, STATGROUP_AICrowd);
DECLARE_CYCLE_STAT(TEXT("Step: avoidance"), STAT_AI_Crowd_StepAvoidanceTime, STATGROUP_AICrowd);
DECLARE_CYCLE_STAT(TEXT("Step: collisions"), STAT_AI_Crowd_StepCollisionsTime, STATGROUP_AICrowd);
DECLARE_CYCLE_STAT(TEXT("Step: components"), STAT_AI_Crowd_StepComponentsTime, STATGROUP_AICrowd);
DECLARE_CYCLE_STAT(TEXT("Step: navlinks"), STAT_AI_Crowd_StepNavLinkTime, STATGROUP_AICrowd);
DECLARE_CYCLE_STAT(TEXT("Step: movement"), STAT_AI_Crowd_StepMovementTime, STATGROUP_AICrowd);
DECLARE_CYCLE_STAT(TEXT("Agent Update Time"), STAT_AI_Crowd_AgentUpdateTime, STATGROUP_AICrowd);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num Agents"), STAT_AI_Crowd_NumAgents, STATGROUP_AICrowd);

namespace FCrowdDebug
{
	/** if set, debug information will be displayed for agent selected in editor */
	int32 DebugSelectedActors = 0;
	FAutoConsoleVariableRef CVarDebugSelectedActors(TEXT("ai.crowd.DebugSelectedActors"), DebugSelectedActors,
		TEXT("Enable debug drawing for selected crowd agent.\n0: Disable, 1: Enable"), ECVF_Default);

	/** if set, basic debug information will be recorded in VisLog for all agents */
	int32 DebugVisLog = 0;
	FAutoConsoleVariableRef CVarDebugVisLog(TEXT("ai.crowd.DebugVisLog"), DebugVisLog,
		TEXT("Enable detailed vislog recording for all crowd agents.\n0: Disable, 1: Enable"), ECVF_Default);

	/** debug flags, works only for selected actor */
	int32 DrawDebugCorners = 1;
	FAutoConsoleVariableRef CVarDrawDebugCorners(TEXT("ai.crowd.DrawDebugCorners"), DrawDebugCorners,
		TEXT("Draw path corners data, requires ai.crowd.DebugSelectedActors.\n0: Disable, 1: Enable"), ECVF_Default);
	
	int32 DrawDebugCollisionSegments = 1;
	FAutoConsoleVariableRef CVarDrawDebugCollisionSegments(TEXT("ai.crowd.DrawDebugCollisionSegments"), DrawDebugCollisionSegments,
		TEXT("Draw colliding navmesh edges, requires ai.crowd.DebugSelectedActors.\n0: Disable, 1: Enable"), ECVF_Default);
	
	int32 DrawDebugPath = 1;
	FAutoConsoleVariableRef CVarDrawDebugPath(TEXT("ai.crowd.DrawDebugPath"), DrawDebugPath,
		TEXT("Draw active paths, requires ai.crowd.DebugSelectedActors.\n0: Disable, 1: Enable"), ECVF_Default);
	
	int32 DrawDebugVelocityObstacles = 1;
	FAutoConsoleVariableRef CVarDrawDebugVelocityObstacles(TEXT("ai.crowd.DrawDebugVelocityObstacles"), DrawDebugVelocityObstacles,
		TEXT("Draw velocity obstacle sampling, requires ai.crowd.DebugSelectedActors.\n0: Disable, 1: Enable"), ECVF_Default);
	
	int32 DrawDebugPathOptimization = 1;
	FAutoConsoleVariableRef CVarDrawDebugPathOptimization(TEXT("ai.crowd.DrawDebugPathOptimization"), DrawDebugPathOptimization,
		TEXT("Draw path optimization data, requires ai.crowd.DebugSelectedActors.\n0: Disable, 1: Enable"), ECVF_Default);
	
	int32 DrawDebugNeighbors = 1;
	FAutoConsoleVariableRef CVarDrawDebugNeighbors(TEXT("ai.crowd.DrawDebugNeighbors"), DrawDebugNeighbors,
		TEXT("Draw current neighbors data, requires ai.crowd.DebugSelectedActors.\n0: Disable, 1: Enable"), ECVF_Default);

	/** debug flags, don't depend on agent */
	int32 DrawDebugBoundaries = 0;
	FAutoConsoleVariableRef CVarDrawDebugBoundaries(TEXT("ai.crowd.DrawDebugBoundaries"), DrawDebugBoundaries,
		TEXT("Draw shared navmesh boundaries used by crowd simulation.\n0: Disable, 1: Enable"), ECVF_Default);

	const FVector Offset(0, 0, 20);

	const FColor Corner(128, 0, 0);
	const FColor CornerLink(192, 0, 0);
	const FColor CornerFixed(192, 192, 0);
	const FColor CollisionRange(192, 0, 128);
	const FColor CollisionSeg0(192, 0, 128);
	const FColor CollisionSeg1(96, 0, 64);
	const FColor CollisionSegIgnored(128, 128, 128);
	const FColor Path(255, 255, 255);
	const FColor PathSpecial(255, 192, 203);
	const FColor PathOpt(0, 128, 0);
	const FColor AvoidanceRange(255, 255, 255);
	const FColor Neighbor(0, 192, 128);

	const float LineThickness = 3.f;
}

void FCrowdTickHelper::Tick(float DeltaTime)
{
#if WITH_EDITOR
	if (Owner.IsValid())
	{
		Owner->DebugTick();
	}
#endif // WITH_EDITOR
}

TStatId FCrowdTickHelper::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FCrowdTickHelper, STATGROUP_Tickables);
}

void FCrowdAgentData::ClearFilter()
{
#if WITH_RECAST
	LinkFilter.Reset();
#endif
}

void FCrowdAvoidanceSamplingPattern::AddSample(float AngleInDegrees, float NormalizedRadius)
{
	Angles.Add(FMath::DegreesToRadians(AngleInDegrees));
	Radii.Add(NormalizedRadius);
}

void FCrowdAvoidanceSamplingPattern::AddSampleWithMirror(float AngleInDegrees, float NormalizedRadius)
{
	AddSample(AngleInDegrees, NormalizedRadius);
	AddSample(-AngleInDegrees, NormalizedRadius);
}

UCrowdManager::UCrowdManager(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	MyNavData = NULL;
#if WITH_RECAST
	DetourCrowd = NULL;

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		DetourAvoidanceDebug = dtAllocObstacleAvoidanceDebugData();
		DetourAvoidanceDebug->init(2048);

		DetourAgentDebug = new dtCrowdAgentDebugInfo();
		FMemory::Memzero(DetourAgentDebug, sizeof(dtCrowdAgentDebugInfo));
		DetourAgentDebug->idx = -1;
		DetourAgentDebug->vod = DetourAvoidanceDebug;
	}
	else
	{
		DetourAgentDebug = NULL;
		DetourAvoidanceDebug = NULL;
	}
#endif
#if WITH_EDITOR
	TickHelper = NULL;
	if (!HasAnyFlags(RF_ClassDefaultObject) && GIsEditor)
	{
		TickHelper = new FCrowdTickHelper();
		TickHelper->Owner = this;
	}
#endif

	MaxAgents = 50;
	MaxAgentRadius = 100.0f;
	MaxAvoidedAgents = 6;
	MaxAvoidedWalls = 8;
	NavmeshCheckInterval = 1.0f;
	PathOptimizationInterval = 0.5f;
	SeparationDirClamp = -1.0f;
	PathOffsetRadiusMultiplier = 1.0f;
	bSingleAreaVisibilityOptimization = true;
	bPruneStartedOffmeshConnections = false;
	bEarlyReachTestOptimization = false;
	bAllowPathReplan = true;
	bResolveCollisions = false;
	
	FCrowdAvoidanceConfig AvoidanceConfig11;		// 11 samples, ECrowdAvoidanceQuality::Low
	AvoidanceConfig11.VelocityBias = 0.5f;
	AvoidanceConfig11.AdaptiveDivisions = 5;
	AvoidanceConfig11.AdaptiveRings = 2;
	AvoidanceConfig11.AdaptiveDepth = 1;
	AvoidanceConfig.Add(AvoidanceConfig11);
	
	FCrowdAvoidanceConfig AvoidanceConfig22;		// 22 samples, ECrowdAvoidanceQuality::Medium
	AvoidanceConfig22.VelocityBias = 0.5f;
	AvoidanceConfig22.AdaptiveDivisions = 5;
	AvoidanceConfig22.AdaptiveRings = 2;
	AvoidanceConfig22.AdaptiveDepth = 2;
	AvoidanceConfig.Add(AvoidanceConfig22);

	FCrowdAvoidanceConfig AvoidanceConfig45;		// 45 samples, ECrowdAvoidanceQuality::Good
	AvoidanceConfig45.VelocityBias = 0.5f;
	AvoidanceConfig45.AdaptiveDivisions = 7;
	AvoidanceConfig45.AdaptiveRings = 2;
	AvoidanceConfig45.AdaptiveDepth = 3;
	AvoidanceConfig.Add(AvoidanceConfig45);

	FCrowdAvoidanceConfig AvoidanceConfig66;		// 66 samples, ECrowdAvoidanceQuality::High
	AvoidanceConfig66.VelocityBias = 0.5f;
	AvoidanceConfig66.AdaptiveDivisions = 7;
	AvoidanceConfig66.AdaptiveRings = 3;
	AvoidanceConfig66.AdaptiveDepth = 3;
	AvoidanceConfig.Add(AvoidanceConfig66);
}

void UCrowdManager::BeginDestroy()
{
#if WITH_RECAST
	// cleanup allocated link filters
	ActiveAgents.Empty();

	dtFreeObstacleAvoidanceDebugData(DetourAvoidanceDebug);
	delete DetourAgentDebug;
#endif

#if WITH_EDITOR
	delete TickHelper;
#endif

#if WITH_RECAST
	DestroyCrowdManager();
#endif // WITH_RECAST
	Super::BeginDestroy();
}

void UCrowdManager::Tick(float DeltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_AI_Crowd_Tick);
	INC_DWORD_STAT_BY(STAT_AI_Crowd_NumAgents, ActiveAgents.Num());

#if WITH_RECAST
	if (DetourCrowd)
	{
		int32 NumActive = DetourCrowd->cacheActiveAgents();
		if (NumActive)
		{
			MyNavData->BeginBatchQuery();

			for (auto It = ActiveAgents.CreateIterator(); It; ++It)
			{
				// collect position and velocity
				FCrowdAgentData& AgentData = It.Value();
				if (AgentData.IsValid())
				{
					PrepareAgentStep(It.Key(), AgentData, DeltaTime);
				}
			}

			// corridor update from previous step
			{
				SCOPE_CYCLE_COUNTER(STAT_AI_Crowd_StepCorridorTime);
				DetourCrowd->updateStepCorridor(DeltaTime, DetourAgentDebug);
			}

			// regular steps
			if (bAllowPathReplan)
			{
				SCOPE_CYCLE_COUNTER(STAT_AI_Crowd_StepPathsTime);
				DetourCrowd->updateStepPaths(DeltaTime, DetourAgentDebug);
			}
			{
				SCOPE_CYCLE_COUNTER(STAT_AI_Crowd_StepProximityTime);
				DetourCrowd->updateStepProximityData(DeltaTime, DetourAgentDebug);
				PostProximityUpdate();
			}
			{
				SCOPE_CYCLE_COUNTER(STAT_AI_Crowd_StepNextPointTime);
				DetourCrowd->updateStepNextMovePoint(DeltaTime, DetourAgentDebug);
				PostMovePointUpdate();
			}
			{
				SCOPE_CYCLE_COUNTER(STAT_AI_Crowd_StepSteeringTime);
				DetourCrowd->updateStepSteering(DeltaTime, DetourAgentDebug);
			}
			{
				SCOPE_CYCLE_COUNTER(STAT_AI_Crowd_StepAvoidanceTime);
				DetourCrowd->updateStepAvoidance(DeltaTime, DetourAgentDebug);
			}
			if (bResolveCollisions)
			{
				SCOPE_CYCLE_COUNTER(STAT_AI_Crowd_StepCollisionsTime);
				DetourCrowd->updateStepMove(DeltaTime, DetourAgentDebug);
			}
			{
				SCOPE_CYCLE_COUNTER(STAT_AI_Crowd_StepComponentsTime);
				UpdateAgentPaths();
			}
			{
				SCOPE_CYCLE_COUNTER(STAT_AI_Crowd_StepNavLinkTime);
				DetourCrowd->updateStepOffMeshVelocity(DeltaTime, DetourAgentDebug);
			}

			// velocity updates
			{
				SCOPE_CYCLE_COUNTER(STAT_AI_Crowd_StepMovementTime);
				for (auto It = ActiveAgents.CreateIterator(); It; ++It)
				{
					const FCrowdAgentData& AgentData = It.Value();
					if (AgentData.bIsSimulated && AgentData.IsValid())
					{
						UCrowdFollowingComponent* CrowdComponent = Cast<UCrowdFollowingComponent>(It.Key());
						if (CrowdComponent && CrowdComponent->IsCrowdSimulationEnabled())
						{
							ApplyVelocity(CrowdComponent, AgentData.AgentIndex);
						}
					}
				}
			}

			MyNavData->FinishBatchQuery();

#if WITH_EDITOR
			// normalize samples only for debug drawing purposes
			DetourAvoidanceDebug->normalizeSamples();
#endif
		}
	}
#endif // WITH_RECAST
}

#if WITH_EDITOR
void UCrowdManager::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
#if WITH_RECAST
	// recreate crowd manger
	DestroyCrowdManager();
	CreateCrowdManager();
#endif // WITH_RECAST
}
#endif // WITH_EDITOR

void UCrowdManager::RegisterAgent(ICrowdAgentInterface* Agent)
{
	if (Agent == nullptr)
	{
		UE_LOGF(LogCrowdFollowing, Warning, "%s, Trying to add an invalid agent interface pointer.", __FUNCTION__);
		return;
	}
	
	UpdateNavData();

	FCrowdAgentData AgentData;

#if WITH_RECAST
	if (DetourCrowd)
	{
		AddAgent(Agent, AgentData);
	}
#endif

	ActiveAgents.Add(Agent, AgentData);
}

void UCrowdManager::UnregisterAgent(const ICrowdAgentInterface* Agent)
{
#if WITH_RECAST
	FCrowdAgentData* AgentData = ActiveAgents.Find(Agent);
	if (DetourCrowd && AgentData)
	{
		RemoveAgent(Agent, AgentData);
	}
#endif

	ActiveAgents.Remove(Agent);
}

bool UCrowdManager::IsAgentValid(const UCrowdFollowingComponent* AgentComponent) const
{
	const FCrowdAgentData* AgentData = ActiveAgents.Find(AgentComponent);
	return AgentData && AgentData->IsValid();
}

bool UCrowdManager::IsAgentValid(const ICrowdAgentInterface* Agent) const
{
	const FCrowdAgentData* AgentData = ActiveAgents.Find(Agent);
	return AgentData && AgentData->IsValid();
}

void UCrowdManager::UpdateAgentParams(const ICrowdAgentInterface* Agent) const
{
	SCOPE_CYCLE_COUNTER(STAT_AI_Crowd_AgentUpdateTime);

#if WITH_RECAST
	const FCrowdAgentData* AgentData = ActiveAgents.Find(Agent);
	if (DetourCrowd && AgentData && AgentData->IsValid())
	{
		dtCrowdAgentParams Params;
		GetAgentParams(Agent, Params);
		Params.linkFilter = AgentData->LinkFilter;

		// store for updating with constant intervals
		((FCrowdAgentData*)AgentData)->bWantsPathOptimization = (Params.updateFlags & DT_CROWD_OPTIMIZE_VIS) != 0;

		DetourCrowd->updateAgentParameters(AgentData->AgentIndex, Params);
	}
#endif
}

void UCrowdManager::UpdateAgentState(const ICrowdAgentInterface* Agent) const
{
	SCOPE_CYCLE_COUNTER(STAT_AI_Crowd_AgentUpdateTime);

#if WITH_RECAST
	const FCrowdAgentData* AgentData = ActiveAgents.Find(Agent);
	if (DetourCrowd && AgentData && AgentData->IsValid())
	{
		DetourCrowd->updateAgentState(AgentData->AgentIndex, false);
	}
#endif
}

void UCrowdManager::OnAgentFinishedCustomLink(const ICrowdAgentInterface* Agent) const
{
#if WITH_RECAST
	const FCrowdAgentData* AgentData = ActiveAgents.Find(Agent);
	if (DetourCrowd && AgentData && AgentData->IsValid())
	{
		DetourCrowd->setAgentBackOnLink(AgentData->AgentIndex);
	}
#endif
}

bool UCrowdManager::SetAgentMoveTarget(const UCrowdFollowingComponent* AgentComponent, const FVector& MoveTarget, FSharedConstNavQueryFilter Filter) const
{
	SCOPE_CYCLE_COUNTER(STAT_AI_Crowd_AgentUpdateTime);

	bool bSuccess = false;

#if WITH_RECAST
	const FCrowdAgentData* AgentData = ActiveAgents.Find(AgentComponent);
	if (AgentData && AgentData->bIsSimulated && AgentData->IsValid() && DetourCrowd)
	{
		FNavLocation ProjectedLoc;
		MyNavData->ProjectPoint(MoveTarget, ProjectedLoc, MyNavData->GetDefaultQueryExtent(), Filter);

		const INavigationQueryFilterInterface* NavFilter = Filter.IsValid() ? Filter->GetImplementation() : MyNavData->GetDefaultQueryFilterImpl();
		const dtQueryFilter* DetourFilter = ((const FRecastQueryFilter*)NavFilter)->GetAsDetourQueryFilter();
		DetourCrowd->updateAgentFilter(AgentData->AgentIndex, DetourFilter);
		DetourCrowd->updateAgentState(AgentData->AgentIndex, false);

		const FVector RcTargetPos = Unreal2RecastPoint(MoveTarget);
		bSuccess = DetourCrowd->requestMoveTarget(AgentData->AgentIndex, ProjectedLoc.NodeRef, &RcTargetPos.X);
	}
#endif

	return bSuccess;
}

bool UCrowdManager::SetAgentMoveDirection(const UCrowdFollowingComponent* AgentComponent, const FVector& MoveDirection) const
{
	SCOPE_CYCLE_COUNTER(STAT_AI_Crowd_AgentUpdateTime);

	bool bSuccess = false;

#if WITH_RECAST
	const FCrowdAgentData* AgentData = ActiveAgents.Find(AgentComponent);
	if (AgentData && AgentData->bIsSimulated && AgentData->IsValid() && DetourCrowd)
	{
		DetourCrowd->updateAgentState(AgentData->AgentIndex, false);

		const FVector RcTargetVelocity = Unreal2RecastPoint(MoveDirection * AgentComponent->GetCrowdAgentMaxSpeed());
		bSuccess = DetourCrowd->requestMoveVelocity(AgentData->AgentIndex, &RcTargetVelocity.X);
	}
#endif

	return bSuccess;
}

bool UCrowdManager::SetAgentMovePath(const UCrowdFollowingComponent* AgentComponent, const FNavMeshPath* Path,
	int32 PathSectionStart, int32 PathSectionEnd, const FVector& PathSectionEndLocation) const
{
	SCOPE_CYCLE_COUNTER(STAT_AI_Crowd_AgentUpdateTime);

	bool bSuccess = false;

#if WITH_RECAST
	const FCrowdAgentData* AgentData = ActiveAgents.Find(AgentComponent);
	ARecastNavMesh* RecastNavData = Cast<ARecastNavMesh>(MyNavData);
	if (AgentData && AgentData->bIsSimulated && AgentData->IsValid() && 
		DetourCrowd && RecastNavData &&
		Path && (Path->GetPathPoints().Num() > 1) &&
		Path->PathCorridor.IsValidIndex(PathSectionStart) && Path->PathCorridor.IsValidIndex(PathSectionEnd))
	{
		FVector TargetPos = PathSectionEndLocation;
		if (PathSectionEnd < (Path->PathCorridor.Num() - 1))
		{
			RecastNavData->GetPolyCenter(Path->PathCorridor[PathSectionEnd], TargetPos);
		}

		TArray<dtPolyRef> PathRefs;
		for (int32 Idx = PathSectionStart; Idx <= PathSectionEnd; Idx++)
		{
			PathRefs.Add(Path->PathCorridor[Idx]);
		}

		const INavigationQueryFilterInterface* NavFilter = Path->GetFilter().IsValid() ? Path->GetFilter()->GetImplementation() : MyNavData->GetDefaultQueryFilterImpl();
		const dtQueryFilter* DetourFilter = ((const FRecastQueryFilter*)NavFilter)->GetAsDetourQueryFilter();
		DetourCrowd->updateAgentFilter(AgentData->AgentIndex, DetourFilter);
		DetourCrowd->updateAgentState(AgentData->AgentIndex, false);

		const FVector RcTargetPos = Unreal2RecastPoint(TargetPos);
		bSuccess = DetourCrowd->requestMoveTarget(AgentData->AgentIndex, PathRefs.Last(), &RcTargetPos.X);
		if (bSuccess)
		{
			bSuccess = DetourCrowd->setAgentCorridor(AgentData->AgentIndex, PathRefs.GetData(), PathRefs.Num());
		}
	}
#endif

	return bSuccess;
}

void UCrowdManager::ClearAgentMoveTarget(const UCrowdFollowingComponent* AgentComponent) const
{
	SCOPE_CYCLE_COUNTER(STAT_AI_Crowd_AgentUpdateTime);

#if WITH_RECAST
	const FCrowdAgentData* AgentData = ActiveAgents.Find(AgentComponent);
	if (AgentData && AgentData->bIsSimulated && AgentData->IsValid() && DetourCrowd)
	{
		DetourCrowd->resetMoveTarget(AgentData->AgentIndex);
		DetourCrowd->resetAgentVelocity(AgentData->AgentIndex);
	}
#endif
}

void UCrowdManager::PauseAgent(const UCrowdFollowingComponent* AgentComponent) const
{
	SCOPE_CYCLE_COUNTER(STAT_AI_Crowd_AgentUpdateTime);

#if WITH_RECAST
	const FCrowdAgentData* AgentData = ActiveAgents.Find(AgentComponent);
	if (AgentData && AgentData->bIsSimulated && AgentData->IsValid() && DetourCrowd)
	{
		DetourCrowd->setAgentWaiting(AgentData->AgentIndex);
		DetourCrowd->resetAgentVelocity(AgentData->AgentIndex);
	}
#endif

}

void UCrowdManager::ResumeAgent(const UCrowdFollowingComponent* AgentComponent, bool bForceReplanPath) const
{
	SCOPE_CYCLE_COUNTER(STAT_AI_Crowd_AgentUpdateTime);

#if WITH_RECAST
	const FCrowdAgentData* AgentData = ActiveAgents.Find(AgentComponent);
	if (AgentData && AgentData->bIsSimulated && AgentData->IsValid() && DetourCrowd)
	{
		DetourCrowd->updateAgentState(AgentData->AgentIndex, bForceReplanPath);
	}
#endif
}

int32 UCrowdManager::GetNumNearbyAgents(const ICrowdAgentInterface* Agent) const
{
	int32 NumNearby = 0;

#if WITH_RECAST
	const FCrowdAgentData* AgentData = ActiveAgents.Find(Agent);
	if (AgentData && AgentData->bIsSimulated && AgentData->IsValid() && DetourCrowd)
	{
		const dtCrowdAgent* ag = DetourCrowd->getAgent(AgentData->AgentIndex);
		NumNearby = ag ? ag->nneis : 0;
	}
#endif

	return NumNearby;
}

int32 UCrowdManager::GetNearbyAgentLocations(const ICrowdAgentInterface* Agent, TArray<FVector>& OutLocations) const
{
	const int32 InitialSize = OutLocations.Num();
#if WITH_RECAST
	const FCrowdAgentData* AgentData = ActiveAgents.Find(Agent);

	if (AgentData && AgentData->bIsSimulated && AgentData->IsValid() && DetourCrowd)
	{
		const dtCrowdAgent* CrowdAgent = DetourCrowd->getAgent(AgentData->AgentIndex);

		if (CrowdAgent)
		{
			OutLocations.Reserve(InitialSize + CrowdAgent->nneis);

			for (int32 NeighbourIndex = 0; NeighbourIndex < CrowdAgent->nneis; NeighbourIndex++)
			{
				const dtCrowdAgent* NeighbourAgent = DetourCrowd->getAgent(CrowdAgent->neis[NeighbourIndex].idx);
				if (NeighbourAgent)
				{
					OutLocations.Add(Recast2UnrealPoint(NeighbourAgent->npos));
				}
			}
		}
	}
#endif

	return OutLocations.Num() - InitialSize;
}

bool UCrowdManager::GetAvoidanceConfig(int32 Idx, FCrowdAvoidanceConfig& Data) const
{
	if (AvoidanceConfig.IsValidIndex(Idx))
	{
		Data = AvoidanceConfig[Idx];
		return true;
	}

	return false;
}

bool UCrowdManager::SetAvoidanceConfig(int32 Idx, const FCrowdAvoidanceConfig& Data)
{
	if (AvoidanceConfig.IsValidIndex(Idx))
	{
		AvoidanceConfig[Idx] = Data;
	}
#if WITH_RECAST
	else if (Idx < DT_CROWD_MAX_OBSTAVOIDANCE_PARAMS)
	{
		AvoidanceConfig.SetNum(Idx + 1);
		AvoidanceConfig[Idx] = Data;
	}
#endif
	else
	{
		return false;
	}

	UpdateAvoidanceConfig();
	return true;
}

void UCrowdManager::AdjustAgentPathStart(const UCrowdFollowingComponent* AgentComponent, const FNavMeshPath* Path, int32& PathStartIdx) const
{
#if WITH_RECAST
	const FCrowdAgentData* AgentData = ActiveAgents.Find(AgentComponent);
	if (AgentData && AgentData->bIsSimulated && AgentData->IsValid() && 
		DetourCrowd && Path && Path->PathCorridor.Num() > PathStartIdx)
	{
		const dtCrowdAgent* ag = DetourCrowd->getAgent(AgentData->AgentIndex);
		const dtPolyRef* agPath = ag->corridor.getPath();

		for (int32 Idx = 0; Idx < ag->corridor.getPathCount(); Idx++)
		{
			const dtPolyRef TestRef = ag->corridor.getFirstPoly();

			for (int32 TestIdx = PathStartIdx; TestIdx < Path->PathCorridor.Num(); TestIdx++)
			{
				if (Path->PathCorridor[TestIdx] == TestRef)
				{
					PathStartIdx = TestIdx;
					return;
				}
			}
		}
	}
#endif
}

void UCrowdManager::SetOffmeshConnectionPruning(bool bRemoveFromCorridor)
{
	bPruneStartedOffmeshConnections = bRemoveFromCorridor;
#if WITH_RECAST
	if (DetourCrowd)
	{
		DetourCrowd->setPruneStartedOffmeshConnections(bRemoveFromCorridor);
	}
#endif
}

void UCrowdManager::SetSingleAreaVisibilityOptimization(bool bEnable)
{
	bSingleAreaVisibilityOptimization = bEnable;
#if WITH_RECAST
	if (DetourCrowd)
	{
		DetourCrowd->setSingleAreaVisibilityOptimization(bEnable);
	}
#endif
}

#if WITH_RECAST

void UCrowdManager::AddAgent(const ICrowdAgentInterface* Agent, FCrowdAgentData& AgentData) const
{
	SCOPE_CYCLE_COUNTER(STAT_AI_Crowd_AgentUpdateTime);

	dtCrowdAgentParams Params;
	GetAgentParams(Agent, Params);

	// store for updating with constant intervals
	AgentData.bWantsPathOptimization = (Params.updateFlags & DT_CROWD_OPTIMIZE_VIS) != 0;

	// create link filter for fully simulated agents
	// (used to determine if agent can traverse smart links)
	TSharedPtr<dtQuerySpecialLinkFilter> MyLinkFilter;
	const UCrowdFollowingComponent* CrowdComponent = Cast<const UCrowdFollowingComponent>(Agent);
	if (CrowdComponent)
	{
		UNavigationSystemV1* NavSys = Cast<UNavigationSystemV1>(GetOuter());
		MyLinkFilter = MakeShareable(new FRecastSpeciaLinkFilter(NavSys, CrowdComponent->GetOuter()));
	}

	Params.linkFilter = MyLinkFilter;

	const FVector RcAgentPos = Unreal2RecastPoint(Agent->GetCrowdAgentLocation());
	const dtQueryFilter* DefaultFilter = ((const FRecastQueryFilter*)MyNavData->GetDefaultQueryFilterImpl())->GetAsDetourQueryFilter();

	AgentData.AgentIndex = DetourCrowd->addAgent(&RcAgentPos.X, Params, DefaultFilter);
	AgentData.bIsSimulated = (Params.collisionQueryRange > 0.0f) && (CrowdComponent == NULL || CrowdComponent->IsCrowdSimulationEnabled());
	AgentData.LinkFilter = MyLinkFilter;
}

void UCrowdManager::RemoveAgent(const ICrowdAgentInterface* Agent, FCrowdAgentData* AgentData) const
{
	SCOPE_CYCLE_COUNTER(STAT_AI_Crowd_AgentUpdateTime);

	DetourCrowd->removeAgent(AgentData->AgentIndex);
	AgentData->ClearFilter();
}

void UCrowdManager::GetAgentParams(const ICrowdAgentInterface* Agent, dtCrowdAgentParams& AgentParams) const
{
	float CylRadius = 0.0f, CylHalfHeight = 0.0f;
	Agent->GetCrowdAgentCollisions(CylRadius, CylHalfHeight);

	// first release the shared pointer
	AgentParams.linkFilter = nullptr;
	// this is actually a bit @hacky if we have non-POD types in dtCrowdAgentParams
	FMemory::Memzero(&AgentParams, sizeof(dtCrowdAgentParams));

	AgentParams.radius = CylRadius;
	AgentParams.height = CylHalfHeight * 2.0f;
	AgentParams.avoidanceQueryMultiplier = 1.0f;
	AgentParams.avoidanceGroup = Agent->GetCrowdAgentAvoidanceGroup();
	AgentParams.groupsToAvoid = Agent->GetCrowdAgentGroupsToAvoid();
	AgentParams.groupsToIgnore = Agent->GetCrowdAgentGroupsToIgnore();

	// skip maxSpeed, it will be constantly updated in every tick
	// skip maxAcceleration, we don't use Detour's movement code

	const UCrowdFollowingComponent* CrowdComponent = Cast<const UCrowdFollowingComponent>(Agent);
	if (CrowdComponent)
	{
		AgentParams.collisionQueryRange = CrowdComponent->GetCrowdCollisionQueryRange();
		AgentParams.pathOptimizationRange = CrowdComponent->GetCrowdPathOptimizationRange();
		AgentParams.separationWeight = CrowdComponent->GetCrowdSeparationWeight();
		AgentParams.obstacleAvoidanceType = IntCastChecked<unsigned char>((int32)CrowdComponent->GetCrowdAvoidanceQuality());
		AgentParams.avoidanceQueryMultiplier = CrowdComponent->GetCrowdAvoidanceRangeMultiplier();

		if (CrowdComponent->IsCrowdSimulationEnabled())
		{
			AgentParams.updateFlags =
				(CrowdComponent->IsCrowdAnticipateTurnsActive() ? DT_CROWD_ANTICIPATE_TURNS : 0) |
				(CrowdComponent->IsCrowdObstacleAvoidanceActive() ? DT_CROWD_OBSTACLE_AVOIDANCE : 0) |
				(CrowdComponent->IsCrowdSeparationActive() ? DT_CROWD_SEPARATION : 0) |
				(CrowdComponent->IsCrowdOptimizeVisibilityEnabled() ? (DT_CROWD_OPTIMIZE_VIS | DT_CROWD_OPTIMIZE_VIS_MULTI) : 0) |
				(CrowdComponent->IsCrowdOptimizeTopologyActive() ? DT_CROWD_OPTIMIZE_TOPO : 0) |
				(CrowdComponent->IsCrowdPathOffsetEnabled() ? DT_CROWD_OFFSET_PATH : 0) |
				(CrowdComponent->IsCrowdSlowdownAtGoalEnabled() ? DT_CROWD_SLOWDOWN_AT_GOAL : 0);
		}
	}
}

void UCrowdManager::PrepareAgentStep(const ICrowdAgentInterface* Agent, FCrowdAgentData& AgentData, float DeltaTime) const
{
	dtCrowdAgent* ag = (dtCrowdAgent*)DetourCrowd->getAgent(AgentData.AgentIndex);
	ag->params.maxSpeed = Agent->GetCrowdAgentMaxSpeed();

	FVector RcLocation = Unreal2RecastPoint(Agent->GetCrowdAgentLocation());
	FVector RcVelocity = Unreal2RecastPoint(Agent->GetCrowdAgentVelocity());

	dtVcopy(ag->npos, &RcLocation.X);
	dtVcopy(ag->vel, &RcVelocity.X);

	if (AgentData.bWantsPathOptimization)
	{
		AgentData.PathOptRemainingTime -= DeltaTime;
		if (AgentData.PathOptRemainingTime > 0)
		{
			ag->params.updateFlags &= ~DT_CROWD_OPTIMIZE_VIS;
		}
		else
		{
			ag->params.updateFlags |= DT_CROWD_OPTIMIZE_VIS;
			AgentData.PathOptRemainingTime = PathOptimizationInterval;
		}
	}
}

void UCrowdManager::ApplyVelocity(UCrowdFollowingComponent* AgentComponent, int32 AgentIndex) const
{
	const dtCrowdAgent* ag = DetourCrowd->getAgent(AgentIndex);
	const dtCrowdAgentAnimation* anims = DetourCrowd->getAgentAnims();

	const FVector NewVelocity = Recast2UnrealPoint(ag->nvel);
	const FVector::FReal* RcDestCorner = anims[AgentIndex].active ? anims[AgentIndex].endPos : 
		ag->ncorners ? &ag->cornerVerts[0] : &ag->npos[0];

	const bool bIsNearEndOfPath = (ag->ncorners == 1) && ((ag->cornerFlags[0] & DT_STRAIGHTPATH_OFFMESH_CONNECTION) == 0);

	const FVector DestPathCorner = Recast2UnrealPoint(RcDestCorner);
	AgentComponent->ApplyCrowdAgentVelocity(NewVelocity, DestPathCorner, anims[AgentIndex].active != 0, bIsNearEndOfPath);

	if (bResolveCollisions)
	{
		const FVector NewPosition = Recast2UnrealPoint(ag->npos);
		AgentComponent->ApplyCrowdAgentPosition(NewPosition);
	}
}

void UCrowdManager::UpdateAgentPaths()
{
	UNavigationSystemV1* NavSys = Cast<UNavigationSystemV1>(GetOuter());
	ARecastNavMesh* RecastNavData = Cast<ARecastNavMesh>(MyNavData);
	if (RecastNavData == NULL)
	{
		return;
	}

	const dtCrowdAgentAnimation* AgentAnims = DetourCrowd->getAgentAnims();
	for (auto It = ActiveAgents.CreateIterator(); It; ++It)
	{
		FCrowdAgentData& AgentData = It.Value();
		if (AgentData.bIsSimulated && AgentData.IsValid())
		{
			UCrowdFollowingComponent* CrowdComponent = nullptr;

			const dtCrowdAgent* Agent = DetourCrowd->getAgent(AgentData.AgentIndex);
			dtPolyRef AgentPolyRef = Agent->corridor.getFirstPoly();

			// look for newly triggered smart links
			const dtCrowdAgentAnimation& AnimInfo = AgentAnims[AgentData.AgentIndex];
			if (AnimInfo.active)
			{
				AgentPolyRef = AnimInfo.polyRef;

				if (AnimInfo.t == 0)
				{
					const FNavLinkId NavLinkId = RecastNavData->GetNavLinkUserId(AnimInfo.polyRef);
					INavLinkCustomInterface* CustomLink = NavSys->GetCustomLink(NavLinkId);

					if (CustomLink)
					{
						FVector EndPt = Recast2UnrealPoint(AnimInfo.endPos);

						// switch to waiting state
						DetourCrowd->setAgentWaiting(AgentData.AgentIndex);
						DetourCrowd->resetAgentVelocity(AgentData.AgentIndex);

						// start using smart link
						CrowdComponent = (CrowdComponent ? CrowdComponent : (UCrowdFollowingComponent*)Cast<const UCrowdFollowingComponent>(It.Key()));
						if (CrowdComponent)
						{
							CrowdComponent->StartUsingCustomLink(CustomLink, EndPt);
						}
					}
				}
			}

			// look for poly updates
			if (AgentPolyRef != AgentData.PrevPoly)
			{
				CrowdComponent = (CrowdComponent ? CrowdComponent : (UCrowdFollowingComponent*)Cast<const UCrowdFollowingComponent>(It.Key()));
				if (CrowdComponent)
				{
					CrowdComponent->OnNavNodeChanged(AgentPolyRef, AgentData.PrevPoly, Agent->corridor.getPathCount());
					AgentData.PrevPoly = AgentPolyRef;
				}
			}
		}
	}
}

void UCrowdManager::UpdateSelectedDebug(const ICrowdAgentInterface* Agent, int32 AgentIndex) const
{
#if WITH_EDITOR
	const UObject* Obj = Cast<const UObject>(Agent);
	if (GIsEditor && Obj)
	{
		const AController* TestController = Cast<const AController>(Obj->GetOuter());
		if (TestController && TestController->GetPawn() && TestController->GetPawn()->IsSelected())
		{
			DetourAgentDebug->idx = AgentIndex;
		}
	}
#endif
}

void UCrowdManager::CreateCrowdManager()
{
	if (MyNavData == nullptr)
	{
		// run update and quit since UpdateNavData will call CreateCrowdManager
		// if navigation mesh is found
		UpdateNavData();

		UE_CLOGF(MyNavData == nullptr, LogCrowdFollowing, Warning, "Unable to find RecastNavMesh instance while trying to create UCrowdManager instance");
		return;
	}

	ARecastNavMesh* RecastNavData = Cast<ARecastNavMesh>(MyNavData);
	dtNavMesh* NavMeshPtr = RecastNavData->GetRecastMesh();

	if (NavMeshPtr)
	{
		DetourCrowd = dtAllocCrowd();
	}
		
	if (DetourCrowd)
	{
		DetourCrowd->init(MaxAgents, MaxAgentRadius, NavMeshPtr);
		DetourCrowd->setAgentCheckInterval(NavmeshCheckInterval);
		DetourCrowd->setSeparationFilter(SeparationDirClamp);
		DetourCrowd->setSingleAreaVisibilityOptimization(bSingleAreaVisibilityOptimization);
		DetourCrowd->setPruneStartedOffmeshConnections(bPruneStartedOffmeshConnections);
		DetourCrowd->setEarlyReachTestOptimization(bEarlyReachTestOptimization);
		DetourCrowd->setPathOffsetRadiusMultiplier(PathOffsetRadiusMultiplier);

		DetourCrowd->initAvoidance(MaxAvoidedAgents, MaxAvoidedWalls, FMath::Max(SamplingPatterns.Num(), 1));

		for (int32 Idx = 0; Idx < SamplingPatterns.Num(); Idx++)
		{
			const FCrowdAvoidanceSamplingPattern& Info = SamplingPatterns[Idx];
			if (Info.Angles.Num() > 0 && Info.Angles.Num() == Info.Radii.Num())
			{
				const TArray<FVector::FReal> Angles = UE::LWC::ConvertArrayType<FVector::FReal>(Info.Angles);
				const TArray<FVector::FReal> Radii = UE::LWC::ConvertArrayType<FVector::FReal>(Info.Radii);

				DetourCrowd->setObstacleAvoidancePattern(Idx, Angles.GetData(), Radii.GetData(), Angles.Num());
			}
		}

		UpdateAvoidanceConfig();

		AgentFlags.Reset();
		AgentFlags.AddZeroed(MaxAgents);

		for (auto It = ActiveAgents.CreateIterator(); It; ++It)
		{
			AddAgent(It.Key(), It.Value());
		}
	}
}

void UCrowdManager::DestroyCrowdManager()
{
	// freeing DetourCrowd with dtFreeCrowd 
	dtFreeCrowd(DetourCrowd);
	DetourCrowd = NULL;
}

#if ENABLE_DRAW_DEBUG
UWorld* UCrowdManager::GetDebugDrawingWorld() const
{
	UWorld* DebugDrawingWorld = GetWorld();

#if WITH_EDITORONLY_DATA
	// note that being ENetMode::NM_DedicatedServer implies DebugDrawingWorld is a game world, which is exactly what we need
	if (DebugDrawingWorld != nullptr && DebugDrawingWorld->GetNetMode() == ENetMode::NM_DedicatedServer)
	{
		// no point in trying to draw on dedicated server. Let's see if there's a client world we can use for drawing!
		const TIndirectArray<FWorldContext>& WorldContexts = GEngine->GetWorldContexts();
		for (const FWorldContext& Context : WorldContexts)
		{
			if (Context.World()->IsGameWorld() && Context.World()->GetNetMode() != ENetMode::NM_DedicatedServer)
			{
				DebugDrawingWorld = Context.World();
				break;
			}
		}
	}
#endif

	return DebugDrawingWorld;
}

void UCrowdManager::DrawDebugCorners(const dtCrowdAgent* CrowdAgent) const
{
	UWorld* DebugDrawingWorld = GetDebugDrawingWorld();

	{
		FVector P0 = Recast2UnrealPoint(CrowdAgent->npos);
		for (int32 Idx = 0; Idx < CrowdAgent->ncorners; Idx++)
		{
			FVector P1 = Recast2UnrealPoint(&CrowdAgent->cornerVerts[Idx * 3]);
			DrawDebugLine(DebugDrawingWorld, P0 + FCrowdDebug::Offset, P1 + FCrowdDebug::Offset, FCrowdDebug::Corner, false, -1.0f, SDPG_World, 2.0f);
			P0 = P1;
		}
	}

	if (CrowdAgent->ncorners > 0 && (CrowdAgent->cornerFlags[CrowdAgent->ncorners - 1] & DT_STRAIGHTPATH_OFFMESH_CONNECTION))
	{
		FVector P0 = Recast2UnrealPoint(&CrowdAgent->cornerVerts[(CrowdAgent->ncorners - 1) * 3]);
		DrawDebugLine(DebugDrawingWorld, P0, P0 + FCrowdDebug::Offset * 2.0f, FCrowdDebug::CornerLink, false, -1.0f, SDPG_World, 2.0f);
	}
}

void UCrowdManager::DrawDebugCollisionSegments(const dtCrowdAgent* CrowdAgent) const
{
	UWorld* DebugDrawingWorld = GetDebugDrawingWorld();

	FVector Center = Recast2UnrealPoint(CrowdAgent->boundary.getCenter()) + FCrowdDebug::Offset;
	DrawDebugCylinder(DebugDrawingWorld, Center - FCrowdDebug::Offset, Center, UE_REAL_TO_FLOAT_CLAMPED_MAX(CrowdAgent->params.collisionQueryRange), 32, FCrowdDebug::CollisionRange);

	for (int32 Idx = 0; Idx < CrowdAgent->boundary.getSegmentCount(); Idx++)
	{
		const FVector::FReal* s = CrowdAgent->boundary.getSegment(Idx);
		const int32 SegFlags = CrowdAgent->boundary.getSegmentFlags(Idx);
		const FColor Color = (SegFlags & DT_CROWD_BOUNDARY_IGNORE) ? FCrowdDebug::CollisionSegIgnored :
			(dtTriArea2D(CrowdAgent->npos, s, s + 3) < 0.0f) ? FCrowdDebug::CollisionSeg1 :
			FCrowdDebug::CollisionSeg0;

		FVector Pt0 = Recast2UnrealPoint(s);
		FVector Pt1 = Recast2UnrealPoint(s + 3);

		DrawDebugLine(DebugDrawingWorld, Pt0 + FCrowdDebug::Offset, Pt1 + FCrowdDebug::Offset, Color, false, -1.0f, SDPG_World, 3.5f);
	}
}

void UCrowdManager::DrawDebugPath(const dtCrowdAgent* CrowdAgent) const
{
	UWorld* DebugDrawingWorld = GetDebugDrawingWorld();

	ARecastNavMesh* NavMesh = Cast<ARecastNavMesh>(MyNavData);
	if (NavMesh == NULL)
	{
		return;
	}

	NavMesh->BeginBatchQuery();
	
	const dtPolyRef* Path = CrowdAgent->corridor.getPath();
	TArray<FVector> Verts;

	for (int32 Idx = 0; Idx < CrowdAgent->corridor.getPathCount(); Idx++)
	{
		Verts.Reset();
		NavMesh->GetPolyVerts(Path[Idx], Verts);

		uint16 PolyFlags = 0;
		uint16 AreaFlags = 0;
		NavMesh->GetPolyFlags(Path[Idx], PolyFlags, AreaFlags);
		const FColor PolyColor = AreaFlags != 1 ? FCrowdDebug::Path : FCrowdDebug::PathSpecial;

		for (int32 VertIdx = 0; VertIdx < Verts.Num(); VertIdx++)
		{
			const FVector Pt0 = Verts[VertIdx];
			const FVector Pt1 = Verts[(VertIdx + 1) % Verts.Num()];

			DrawDebugLine(DebugDrawingWorld, Pt0 + FCrowdDebug::Offset * 0.5f, Pt1 + FCrowdDebug::Offset * 0.5f, PolyColor, false
				, /*LifeTime*/-1.f, /*DepthPriority*/0
				, /*Thickness*/FCrowdDebug::LineThickness);
		}
	}

	NavMesh->FinishBatchQuery();
}

void UCrowdManager::DrawDebugVelocityObstacles(const dtCrowdAgent* CrowdAgent) const
{
	UWorld* DebugDrawingWorld = GetDebugDrawingWorld();

	FVector Center = Recast2UnrealPoint(CrowdAgent->npos) + FCrowdDebug::Offset;
	DrawDebugCylinder(DebugDrawingWorld, Center - FCrowdDebug::Offset, Center, UE_REAL_TO_FLOAT_CLAMPED_MAX(CrowdAgent->params.maxSpeed), 32, FCrowdDebug::AvoidanceRange);

	const FVector::FReal InvQueryMultiplier = 1.0f / (float)CrowdAgent->params.avoidanceQueryMultiplier;
	float BestSampleScore = -1.0f;
	FVector BestSampleLocation = FVector::ZeroVector;

	for (int32 Idx = 0; Idx < DetourAvoidanceDebug->getSampleCount(); Idx++)
	{
		const FVector::FReal* p = DetourAvoidanceDebug->getSampleVelocity(Idx);
		const float sr = UE_REAL_TO_FLOAT_CLAMPED_MAX(DetourAvoidanceDebug->getSampleSize(Idx) * InvQueryMultiplier);
		const float pen = UE_REAL_TO_FLOAT(DetourAvoidanceDebug->getSamplePenalty(Idx));
		const float pen2 = UE_REAL_TO_FLOAT(DetourAvoidanceDebug->getSamplePreferredSidePenalty(Idx));

		FVector SamplePos = Center + Recast2UnrealPoint(p);

		if (BestSampleScore <= -1.0f || pen < BestSampleScore)
		{
			BestSampleScore = pen;
			BestSampleLocation = SamplePos;
		}

		float SamplePenalty = pen * 0.75f + pen2 * 0.25f;
		FColor SampleColor = FColor::MakeRedToGreenColorFromScalar(1.0f - SamplePenalty);
			
		FPlane Plane(0, 0, 1, SamplePos.Z);
		DrawDebugSolidPlane(DebugDrawingWorld, Plane, SamplePos, sr, SampleColor);
	}

	if (BestSampleScore >= 0.0f)
	{
		DrawDebugLine(DebugDrawingWorld, BestSampleLocation + FVector(0, 0, 100), BestSampleLocation + FVector(0, 0, -100), FColor::Green);
	}
}

void UCrowdManager::DrawDebugPathOptimization(const dtCrowdAgent* CrowdAgent) const
{
	UWorld* DebugDrawingWorld = GetDebugDrawingWorld();

	FVector Pt0 = Recast2UnrealPoint(DetourAgentDebug->optStart) + FCrowdDebug::Offset * 1.25f;
	FVector Pt1 = Recast2UnrealPoint(DetourAgentDebug->optEnd) + FCrowdDebug::Offset * 1.25f;

	DrawDebugLine(DebugDrawingWorld, Pt0, Pt1, FCrowdDebug::PathOpt, false, -1.0f, SDPG_World, 2.5f);
}

void UCrowdManager::DrawDebugNeighbors(const dtCrowdAgent* CrowdAgent) const
{
	UWorld* DebugDrawingWorld = GetDebugDrawingWorld();

	FVector Center = Recast2UnrealPoint(CrowdAgent->npos) + FCrowdDebug::Offset;
	DrawDebugCylinder(DebugDrawingWorld, Center - FCrowdDebug::Offset, Center, UE_REAL_TO_FLOAT_CLAMPED_MAX(CrowdAgent->params.collisionQueryRange), 32, FCrowdDebug::CollisionRange);

	for (int32 Idx = 0; Idx < CrowdAgent->nneis; Idx++)
	{
		const dtCrowdAgent* nei = DetourCrowd->getAgent(CrowdAgent->neis[Idx].idx);
		if (nei)
		{
			FVector Pt0 = Recast2UnrealPoint(nei->npos) + FCrowdDebug::Offset;
			DrawDebugLine(DebugDrawingWorld, Center, Pt0, FCrowdDebug::Neighbor);
		}
	}
}

void UCrowdManager::DrawDebugSharedBoundary() const
{
	UWorld* DebugDrawingWorld = GetDebugDrawingWorld();

	FColor Colors[] = { FColorList::Red, FColorList::Orange };

	const dtSharedBoundary* sharedBounds = DetourCrowd->getSharedBoundary();
	for (int32 Idx = 0; Idx < sharedBounds->Data.Num(); Idx++)
	{
		FColor Color = Colors[Idx % UE_ARRAY_COUNT(Colors)];
		const FVector Center = Recast2UnrealPoint(sharedBounds->Data[Idx].Center);
		DrawDebugCylinder(DebugDrawingWorld, Center - FCrowdDebug::Offset, Center, UE_REAL_TO_FLOAT_CLAMPED_MAX(sharedBounds->Data[Idx].Radius), 32, Color);

		for (int32 WallIdx = 0; WallIdx < sharedBounds->Data[Idx].Edges.Num(); WallIdx++)
		{
			const FVector WallV0 = Recast2UnrealPoint(sharedBounds->Data[Idx].Edges[WallIdx].v0) + FCrowdDebug::Offset;
			const FVector WallV1 = Recast2UnrealPoint(sharedBounds->Data[Idx].Edges[WallIdx].v1) + FCrowdDebug::Offset;

			DrawDebugLine(DebugDrawingWorld, WallV0, WallV1, Color);
		}
	}
}
#endif // ENABLE_DRAW_DEBUG

#endif // WITH_RECAST

#if WITH_EDITOR

void UCrowdManager::DebugTick() const
{
#if WITH_RECAST
	if (DetourCrowd == NULL || DetourAgentDebug == NULL)
	{
		return;
	}

	for (auto It = ActiveAgents.CreateConstIterator(); It; ++It)
	{
		const FCrowdAgentData& AgentData = It.Value();
		if (AgentData.IsValid())
		{
			UpdateSelectedDebug(It.Key(), AgentData.AgentIndex);
		}
	}

#if ENABLE_DRAW_DEBUG
	// on screen debugging
	const dtCrowdAgent* SelectedAgent = DetourAgentDebug->idx >= 0 ? DetourCrowd->getAgent(DetourAgentDebug->idx) : NULL;
	if (SelectedAgent && FCrowdDebug::DebugSelectedActors)
	{
		if (FCrowdDebug::DrawDebugCorners)
		{
			DrawDebugCorners(SelectedAgent);
		}

		if (FCrowdDebug::DrawDebugCollisionSegments)
		{
			DrawDebugCollisionSegments(SelectedAgent);
		}

		if (FCrowdDebug::DrawDebugPath)
		{
			DrawDebugPath(SelectedAgent);
		}

		if (FCrowdDebug::DrawDebugVelocityObstacles)
		{
			DrawDebugVelocityObstacles(SelectedAgent);
		}

		if (FCrowdDebug::DrawDebugPathOptimization)
		{
			DrawDebugPathOptimization(SelectedAgent);
		}

		if (FCrowdDebug::DrawDebugNeighbors)
		{
			DrawDebugNeighbors(SelectedAgent);
		}
	}

	if (FCrowdDebug::DrawDebugBoundaries)
	{
		DrawDebugSharedBoundary();
	}
#endif // ENABLE_DRAW_DEBUG

	// vislog debugging
	if (FCrowdDebug::DebugVisLog)
	{
		for (auto It = ActiveAgents.CreateConstIterator(); It; ++It)
		{
			const ICrowdAgentInterface* IAgent = It.Key();
			const UObject* AgentOb = IAgent ?  Cast<const UObject>(IAgent) : NULL;
			const AActor* LogOwner = AgentOb ? Cast<const AActor>(AgentOb->GetOuter()) : NULL;

			const FCrowdAgentData& AgentData = It.Value();
			const dtCrowdAgent* CrowdAgent = AgentData.IsValid() ? DetourCrowd->getAgent(AgentData.AgentIndex) : NULL;

			if (CrowdAgent && LogOwner)
			{
				FString LogData = DetourAgentDebug->agentLog.FindRef(AgentData.AgentIndex);
				if (LogData.Len() > 0)
				{
					UE_VLOG(LogOwner, LogCrowdFollowing, Log, TEXT("%s"), *LogData);
				}

				{
					FVector P0 = Recast2UnrealPoint(CrowdAgent->npos);
					for (int32 Idx = 0; Idx < CrowdAgent->ncorners; Idx++)
					{
						FVector P1 = Recast2UnrealPoint(&CrowdAgent->cornerVerts[Idx * 3]);
						UE_VLOG_SEGMENT(LogOwner, LogCrowdFollowing, Log, P0 + FCrowdDebug::Offset, P1 + FCrowdDebug::Offset, FCrowdDebug::Corner, TEXT(""));
						UE_VLOG_BOX(LogOwner, LogCrowdFollowing, Log, FBox::BuildAABB(P1 + FCrowdDebug::Offset, FVector(2, 2, 2)), FCrowdDebug::Corner, TEXT("%d"), CrowdAgent->cornerFlags[Idx]);
						P0 = P1;
					}
				}

				ARecastNavMesh* RecastNavData = Cast<ARecastNavMesh>(MyNavData);
				if (RecastNavData)
				{
					for (int32 Idx = 0; Idx < CrowdAgent->corridor.getPathCount(); Idx++)
					{
						dtPolyRef PolyRef = CrowdAgent->corridor.getPath()[Idx];
						TArray<FVector> PolyPoints;
						RecastNavData->GetPolyVerts(PolyRef, PolyPoints);

						UE_VLOG_CONVEXPOLY(LogOwner, LogCrowdFollowing, Verbose, PolyPoints, FColor::Cyan, TEXT(""));
					}
				}

				if (CrowdAgent->ncorners && (CrowdAgent->cornerFlags[CrowdAgent->ncorners - 1] & DT_STRAIGHTPATH_OFFMESH_CONNECTION))
				{
					FVector P0 = Recast2UnrealPoint(&CrowdAgent->cornerVerts[(CrowdAgent->ncorners - 1) * 3]);
					UE_VLOG_SEGMENT(LogOwner, LogCrowdFollowing, Log, P0, P0 + FCrowdDebug::Offset * 2.0f, FCrowdDebug::CornerLink, TEXT(""));
				}

				if (CrowdAgent->corridor.hasNextFixedCorner())
				{
					FVector P0 = Recast2UnrealPoint(CrowdAgent->corridor.getNextFixedCorner());
					UE_VLOG_BOX(LogOwner, LogCrowdFollowing, Log, FBox::BuildAABB(P0 + FCrowdDebug::Offset, FVector(10, 10, 10)), FCrowdDebug::CornerFixed, TEXT(""));
				}

				if (CrowdAgent->corridor.hasNextFixedCorner2())
				{
					FVector P0 = Recast2UnrealPoint(CrowdAgent->corridor.getNextFixedCorner2());
					UE_VLOG_BOX(LogOwner, LogCrowdFollowing, Log, FBox::BuildAABB(P0 + FCrowdDebug::Offset, FVector(10, 10, 10)), FCrowdDebug::CornerFixed, TEXT(""));
				}

				for (int32 Idx = 0; Idx < CrowdAgent->boundary.getSegmentCount(); Idx++)
				{
					const FVector::FReal* s = CrowdAgent->boundary.getSegment(Idx);
					const int32 SegFlags = CrowdAgent->boundary.getSegmentFlags(Idx);
					const FColor Color = (SegFlags & DT_CROWD_BOUNDARY_IGNORE) ? FCrowdDebug::CollisionSegIgnored :
						(dtTriArea2D(CrowdAgent->npos, s, s + 3) < 0.0f) ? FCrowdDebug::CollisionSeg1 :
						FCrowdDebug::CollisionSeg0;

					FVector Pt0 = Recast2UnrealPoint(s);
					FVector Pt1 = Recast2UnrealPoint(s + 3);

					UE_VLOG_SEGMENT_THICK(LogOwner, LogCrowdFollowing, Log, Pt0 + FCrowdDebug::Offset, Pt1 + FCrowdDebug::Offset, Color, 3, TEXT(""));
				}
			}
		}
	}

	DetourAgentDebug->agentLog.Reset();
#endif	// WITH_RECAST
}

#endif // WITH_EDITOR

void UCrowdManager::UpdateNavData()
{
	if (MyNavData == NULL)
	{
		UNavigationSystemV1* NavSys = Cast<UNavigationSystemV1>(GetOuter());
		if (NavSys)
		{
			for (ANavigationData* NavData : NavSys->NavDataSet)
			{
				if (NavData && IsSuitableNavData(*NavData))
				{
					SetNavData(NavData);
					break;
				}
			}
		}
	}
}

void UCrowdManager::SetNavData(ANavigationData* NavData, const bool bFindNewNavDataIfNull)
{
	if (NavData != MyNavData && MyNavData != nullptr)
	{
		// clean up
		ARecastNavMesh* AsRecastNavData = Cast<ARecastNavMesh>(MyNavData);
		if (AsRecastNavData)
		{
			AsRecastNavData->OnNavMeshUpdate.RemoveAll(this);
		}
	}

	if (NavData)
	{
		ARecastNavMesh* AsRecastNavData = Cast<ARecastNavMesh>(NavData);
		if (AsRecastNavData)
		{
			AsRecastNavData->OnNavMeshUpdate.AddUObject(this, &UCrowdManager::OnNavMeshUpdate);
		}
	}

	MyNavData = NavData;
	UE_VLOG(this, LogCrowdFollowing, Log, TEXT("Setting cached nav data to %s")
		, MyNavData ? *MyNavData->GetFullName() : TEXT("NONE"));

	if (NavData || bFindNewNavDataIfNull)
	{
		OnNavMeshUpdate();
	}
}

void UCrowdManager::OnNavMeshUpdate()
{
#if WITH_RECAST
	DestroyCrowdManager();
	CreateCrowdManager();
#endif // WITH_RECAST
}

bool UCrowdManager::IsSuitableNavData(const ANavigationData& NavData) const
{
	return NavData.IsSupportingDefaultAgent() && Cast<ARecastNavMesh>(&NavData);
}

void UCrowdManager::OnNavDataRegistered(ANavigationData& NavData)
{
	if (MyNavData == nullptr && IsSuitableNavData(NavData))
	{
		SetNavData(&NavData);
	}
}

void UCrowdManager::OnNavDataUnregistered(ANavigationData& NavData)
{
	if (MyNavData == &NavData)
	{
		UE_VLOG(this, LogCrowdFollowing, Warning, TEXT("%s is being unregistered. Will look for new nav data")
			, *NavData.GetName());
		
		// note that this will cause the manager to look for another nav data instance
		SetNavData(nullptr, /*bFindNewNavDataIfNull=*/true);

		if (MyNavData == nullptr)
		{
			UE_VLOG(this, LogCrowdFollowing, Warning, TEXT("Failed to find suitable navigation data instance (no navmesh). Shutting down."));
		}
	}
}

void UCrowdManager::UpdateAvoidanceConfig()
{
#if WITH_RECAST
	if (DetourCrowd == NULL)
	{
		return;
	}

	for (int32 Idx = 0; Idx < AvoidanceConfig.Num(); Idx++)
	{
		const FCrowdAvoidanceConfig& ConfigInfo = AvoidanceConfig[Idx];
		
		dtObstacleAvoidanceParams params;
		params.velBias = ConfigInfo.VelocityBias;
		params.weightDesVel = ConfigInfo.DesiredVelocityWeight;
		params.weightCurVel = ConfigInfo.CurrentVelocityWeight;
		params.weightSide = ConfigInfo.SideBiasWeight;
		params.weightToi = ConfigInfo.ImpactTimeWeight;
		params.horizTime = ConfigInfo.ImpactTimeRange;
		params.sideBiasThreshold = ConfigInfo.SideBiasThreshold;
		params.weightRightBias = ConfigInfo.RightSideBias;
		params.patternIdx = ConfigInfo.CustomPatternIdx;
		params.adaptiveDivs = ConfigInfo.AdaptiveDivisions;
		params.adaptiveRings = ConfigInfo.AdaptiveRings;
		params.adaptiveDepth = ConfigInfo.AdaptiveDepth;

		DetourCrowd->setObstacleAvoidanceParams(Idx, &params);
	}
#endif // WITH_RECAST
}

void UCrowdManager::PostProximityUpdate()
{
	// empty in base class
}

void UCrowdManager::PostMovePointUpdate()
{
#if WITH_RECAST
	const uint8 UpdateDestinationFlag = 1;

	// special case when following last segment of full path to actor: replace end point with actor's location
	for (auto& AgentTuple : ActiveAgents)
	{
		UCrowdFollowingComponent* PathComp = Cast<UCrowdFollowingComponent>(AgentTuple.Key);
		const FCrowdAgentData& AgentData = AgentTuple.Value;
		FVector NewGoalPosition;

		const bool bUpdateTargetPos = PathComp ? PathComp->ShouldTrackMovingGoal(NewGoalPosition) : false;
		if (bUpdateTargetPos && AgentFlags.IsValidIndex(AgentData.AgentIndex))
		{
			PathComp->UpdateDestinationForMovingGoal(NewGoalPosition);

			const dtCrowdAgent* Agent = DetourCrowd->getAgent(AgentData.AgentIndex);
			dtCrowdAgent* MutableAgent = (dtCrowdAgent*)Agent;
			const FVector RcTargetPos = Unreal2RecastPoint(NewGoalPosition);
		
			dtVcopy(MutableAgent->targetPos, &RcTargetPos.X);
			AgentFlags[AgentData.AgentIndex] |= UpdateDestinationFlag;
		}
	}

	dtCrowdAgent** ActiveDetourAgents = DetourCrowd->getActiveAgents();
	for (int32 Idx = 0; Idx < DetourCrowd->getNumActiveAgents(); Idx++)
	{
		dtCrowdAgent* Agent = ActiveDetourAgents[Idx];
		if (Agent->state == DT_CROWDAGENT_STATE_WALKING &&
			Agent->ncorners == 1 && Agent->corridor.getPathCount() < 5 &&
			(Agent->cornerFlags[0] & DT_STRAIGHTPATH_OFFMESH_CONNECTION) == 0)
		{
			const int32 AgentIndex = DetourCrowd->getAgentIndex(Agent);

			if (AgentFlags.IsValidIndex(AgentIndex) && (AgentFlags[AgentIndex] & UpdateDestinationFlag) != 0)
			{
				dtVcopy(Agent->cornerVerts, Agent->targetPos);
			}
		}
	}

	for (auto& AgentTuple : ActiveAgents)
	{
		const FCrowdAgentData& AgentData = AgentTuple.Value;
		if (AgentFlags.IsValidIndex(AgentData.AgentIndex))
		{
			AgentFlags[AgentData.AgentIndex] &= ~UpdateDestinationFlag;
		}
	}
#endif
}

UWorld* UCrowdManager::GetWorld() const
{
	UNavigationSystemV1* NavSys = Cast<UNavigationSystemV1>(GetOuter());
	return NavSys ? NavSys->GetWorld() : NULL;
}

UCrowdManager* UCrowdManager::GetCurrent(UObject* WorldContextObject)
{
	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(WorldContextObject);
	return NavSys ? Cast<UCrowdManager>(NavSys->GetCrowdManager()) : NULL;
}

UCrowdManager* UCrowdManager::GetCurrent(UWorld* World)
{
	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World);
	return NavSys ? Cast<UCrowdManager>(NavSys->GetCrowdManager()) : NULL;
}

