// Copyright Epic Games, Inc. All Rights Reserved.

#include "Navigation/GeneratedNavLinksProxy.h"

#include "Engine/GameEngine.h"
#include "GameFramework/Controller.h"
#include "Navigation/PathFollowingComponent.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Pawn.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeneratedNavLinksProxy)

UGeneratedNavLinksProxy::UGeneratedNavLinksProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UGeneratedNavLinksProxy::OnLinkMoveStarted(UObject* PathComp, const FVector& DestPoint)
{
	NotifySmartLinkReached(PathComp, DestPoint);
	return true;
}

void UGeneratedNavLinksProxy::NotifySmartLinkReached(UObject* PathingAgent, const FVector DestPoint)
{
	UPathFollowingComponent* PathComp = Cast<UPathFollowingComponent>(PathingAgent);
	if (PathComp)
	{
		AActor* PathOwner = PathComp->GetOwner();
		const AController* ControllerOwner = Cast<AController>(PathOwner);
		if (ControllerOwner)
		{
			PathOwner = ControllerOwner->GetPawn();
		}
	
		ReceiveSmartLinkReached(PathOwner, DestPoint);
		OnSmartLinkReached.Broadcast(PathOwner, DestPoint);
	}
}

UWorld* UGeneratedNavLinksProxy::GetWorld() const
{
#if WITH_EDITOR
	if (GEditor)
	{
		constexpr bool bEnsureIsGWorld = false;
		return GEditor->GetEditorWorldContext(bEnsureIsGWorld).World();
	}
#endif // WITH_EDITOR

	if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
	{
		return GameEngine->GetGameWorld();
	}

	return Super::GetWorld();
}
