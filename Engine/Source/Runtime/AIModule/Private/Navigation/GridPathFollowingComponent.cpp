// Copyright Epic Games, Inc. All Rights Reserved.

#include "Navigation/GridPathFollowingComponent.h"
#include "Navigation/NavLocalGridManager.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GridPathFollowingComponent)

UGridPathFollowingComponent::UGridPathFollowingComponent(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	ActiveGridIdx = INDEX_NONE;
	ActiveGridId = INDEX_NONE;
	GridMoveSegmentEndIndex = 0;
	MoveSegmentStartIndexOffGrid = 0;
	GridManager = nullptr;
	bIsPathEndInsideGrid = false;
	bHasGridPath = false;
}

void UGridPathFollowingComponent::Initialize()
{
	Super::Initialize();

	GridManager = UNavLocalGridManager::GetCurrent(this);
}

void UGridPathFollowingComponent::Reset()
{
	Super::Reset();

	ActiveGridIdx = INDEX_NONE;
	ActiveGridId = INDEX_NONE;
	GridMoveSegmentEndIndex = 0;
	MoveSegmentStartIndexOffGrid = 0;
	bIsPathEndInsideGrid = false;
	bHasGridPath = false;
}

void UGridPathFollowingComponent::OnPathUpdated()
{
	Super::OnPathUpdated();

	// force grid update in next tick
	ActiveGridIdx = INDEX_NONE;
	ActiveGridId = INDEX_NONE;
}

void UGridPathFollowingComponent::UpdatePathSegment()
{
	const FVector CurrentLocation = NavMovementInterface.IsValid() ? NavMovementInterface->GetFeetLocation() : FVector::ZeroVector;
	UpdateActiveGrid(CurrentLocation);

	if (HasActiveGrid() && bHasGridPath)
	{
		if (HasReachedDestination(CurrentLocation))
		{
			// always check for destination, acceptance radius may cause it to pass before reaching last segment
			OnSegmentFinished();
			OnPathFinished(EPathFollowingResult::Success, FPathFollowingResultFlags::None);
		}
		else if (HasReachedCurrentTarget(CurrentLocation))
		{
			GridMoveSegmentEndIndex++;
			if (GridPathPoints.IsValidIndex(GridMoveSegmentEndIndex))
			{
				UE_VLOG(this, LogPathFollowing, Log, TEXT("Switching to next segment in grid path"));
				UE_VLOG_BOX(this, LogPathFollowing, Log, FBox::BuildAABB(GridPathPoints[GridMoveSegmentEndIndex], FVector(5, 5, 5)), FColor::Red, TEXT("next corner"));

				CurrentDestination.Set(nullptr, GridPathPoints[GridMoveSegmentEndIndex]);
				MoveSegmentDirection = (GridPathPoints[GridMoveSegmentEndIndex] - GridPathPoints[GridMoveSegmentEndIndex - 1]).GetSafeNormal();
				UpdateMoveFocus();
			}
			else if (bIsPathEndInsideGrid)
			{
				UE_VLOG(this, LogPathFollowing, Log, TEXT("Last grid segment reached, finishing move"));

				OnSegmentFinished();
				OnPathFinished(EPathFollowingResult::Success, FPathFollowingResultFlags::None);
			}
			else
			{
				UE_VLOG(this, LogPathFollowing, Log, TEXT("Last grid segment reached, resuming path from segment %d"), MoveSegmentStartIndexOffGrid);
				
				GridPathPoints.Empty();
				bHasGridPath = false;

				OnSegmentFinished();
				SetMoveSegment(MoveSegmentStartIndexOffGrid);
			}
		}
	}
	else
	{
		// check if was following a grid path
		if (GridPathPoints.Num())
		{
			if (HasReachedCurrentTarget(GridPathPoints.Last()))
			{
				UE_VLOG(this, LogPathFollowing, Log, TEXT("Leaving grid and resuming path from segment %d"), MoveSegmentStartIndexOffGrid);

				OnSegmentFinished();
				SetMoveSegment(MoveSegmentStartIndexOffGrid);
			}

			bHasGridPath = false;
			GridPathPoints.Empty();
		}

		Super::UpdatePathSegment();
	}
}

void UGridPathFollowingComponent::UpdateActiveGrid(const FVector& CurrentLocation)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("NavGrid: UpdateActive"), STAT_GridUpdate, STATGROUP_AI);
	if (GridManager && Path.IsValid())
	{
		const int32 GridIdx = GridManager->GetGridIndex(CurrentLocation);
		if (GridIdx != INDEX_NONE)
		{
			const FNavLocalGridData& GridData = GridManager->GetGridData(GridIdx);
			if (ActiveGridId != GridData.GetGridId())
			{
				ActiveGridId = GridData.GetGridId();
				ActiveGridIdx = GridIdx;

				GridManager->UpdateAccessTime(GridIdx);
				GridData.FindPathForMovingAgent(*Path.Get(), CurrentLocation, MoveSegmentStartIndex, GridPathPoints, MoveSegmentStartIndexOffGrid);
				
				bHasGridPath = GridPathPoints.Num() > 1;
				if (bHasGridPath)
				{
					GridMoveSegmentEndIndex = 1;
					bIsPathEndInsideGrid = GridData.GetCellIndex(Path->GetEndLocation()) != INDEX_NONE;
					CurrentDestination.Set(nullptr, GridPathPoints[GridMoveSegmentEndIndex]);
					MoveSegmentDirection = (GridPathPoints[1] - GridPathPoints[0]).GetSafeNormal();
					UpdateMoveFocus();

#if ENABLE_VISUAL_LOG
					const FVector DebugDrawOffset(0, 0, 15.0f);
					const FVector DebugPathPointExtent(5, 5, 5);

					UE_VLOG_BOX(this, LogPathFollowing, Verbose, GridData.WorldBounds, FColor::Cyan, TEXT(""));
					UE_VLOG_BOX(this, LogPathFollowing, Log, FBox::BuildAABB(GridPathPoints[0] + DebugDrawOffset, DebugPathPointExtent), FColor::Yellow, TEXT(""));
					for (int32 Idx = 1; Idx < GridPathPoints.Num(); Idx++)
					{
						UE_VLOG_BOX(this, LogPathFollowing, Log, FBox::BuildAABB(GridPathPoints[Idx] + DebugDrawOffset, DebugPathPointExtent), FColor::Yellow, TEXT(""));
						UE_VLOG_SEGMENT_THICK(this, LogPathFollowing, Log, GridPathPoints[Idx - 1] + DebugDrawOffset, GridPathPoints[Idx] + DebugDrawOffset, FColor::Yellow, 3, TEXT(""));
					}

					for (int32 Idx = 0; Idx < GridData.GetCellsCount(); Idx++)
					{
						if (GridData.GetCellAtIndexUnsafe(Idx))
						{
							UE_VLOG_BOX(this, LogPathFollowing, Verbose, GridData.GetWorldCellBox(Idx), FColor::Red, TEXT(""));
						}
					}
#endif
				}
			}
		}
		else
		{
			ActiveGridId = INDEX_NONE;
			ActiveGridIdx = INDEX_NONE;
			bHasGridPath = false;
			// don't reset GridPathPoints
		}
	}
}

void UGridPathFollowingComponent::ResumeMove(FAIRequestID RequestID)
{
	if (RequestID.IsEquivalent(GetCurrentRequestId()) && RequestID.IsValid())
	{
		const FVector CurrentLocation = NavMovementInterface.IsValid() ? NavMovementInterface->GetFeetLocation() : FVector::ZeroVector;
		UpdateActiveGrid(CurrentLocation);

		if (HasActiveGrid())
		{
			UE_VLOG(GetOwner(), LogPathFollowing, Log, TEXT("ResumeMove: RequestID(%u) is on grid"), RequestID.GetID());
			SetStatus(EPathFollowingStatus::Moving);
		}
		else
		{
			Super::ResumeMove();
		}
	}
}

