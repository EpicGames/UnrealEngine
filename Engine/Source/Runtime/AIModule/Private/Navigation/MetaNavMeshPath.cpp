// Copyright Epic Games, Inc. All Rights Reserved.

#include "Navigation/MetaNavMeshPath.h"
#include "GameFramework/Controller.h"
#include "NavigationSystem.h"
#include "VisualLogger/VisualLoggerTypes.h"
#include "VisualLogger/VisualLogger.h"
#include "NavAreas/NavArea.h"
#include "NavigationSystemTypes.h"
#include "DrawDebugHelpers.h"

const FNavPathType FMetaNavMeshPath::Type(&FMetaNavMeshPath::Super::Type);

FMetaNavMeshPath::FMetaNavMeshPath()
{
	PathType = FMetaNavMeshPath::Type;

	// should be high enough to allow easy switch to next waypoint for EMetaPathUpdateReason::NearEnd updates (e.g. crowd simulation)
	// suggested value: 5-10x agent radius
	WaypointSwitchRadius = 200.0f;

	ApproximateLength = 0.;
	PathGoalTetherDistance = 0.0f;

	// initialize to 0, path following will try to update it immediately after receiving request
	TargetWaypointIdx = 0;
}

FMetaNavMeshPath::FMetaNavMeshPath(const TArray<FMetaPathWayPoint>& InWaypoints, const ANavigationData& NavData) 
	: FMetaNavMeshPath()
{
	SetNavigationDataUsed(&NavData);
	SetWaypoints(InWaypoints);
}

FMetaNavMeshPath::FMetaNavMeshPath(const TArray<FMetaPathWayPoint>& InWaypoints, const AController& Owner)
	: FMetaNavMeshPath()
{
	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(Owner.GetWorld());
	const ANavigationData* NavData = NavSys ? NavSys->GetNavDataForProps(Owner.GetNavAgentPropertiesRef(), Owner.GetNavAgentLocation()) : nullptr;

	if (ensure(NavData))
	{
		SetNavigationDataUsed(NavData);
		SetWaypoints(InWaypoints);
	}
	else
	{
		UE_VLOG(&Owner, LogNavigation, Error, TEXT("Unable to assign navigation data to MetaNavMeshPath!"));
	}
}

FMetaNavMeshPath::FMetaNavMeshPath(const TArray<FVector>& InWaypoints, const ANavigationData& NavData) : FMetaNavMeshPath()
{
	SetNavigationDataUsed(&NavData);
	SetWaypoints(InWaypoints);
}

FMetaNavMeshPath::FMetaNavMeshPath(const TArray<FVector>& InWaypoints, const AController& Owner) : FMetaNavMeshPath()
{
	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(Owner.GetWorld());
	const ANavigationData* NavData = NavSys ? NavSys->GetNavDataForProps(Owner.GetNavAgentPropertiesRef(), Owner.GetNavAgentLocation()) : nullptr;

	if (ensure(NavData))
	{
		SetNavigationDataUsed(NavData);
		SetWaypoints(InWaypoints);
	}
	else
	{
		UE_VLOG(&Owner, LogNavigation, Error, TEXT("Unable to assign navigation data to MetaNavMeshPath!"));
	}
}

void FMetaNavMeshPath::Initialize(const FVector& AgentLocation)
{
	if (TargetWaypointIdx == 0)
	{
		UE_VLOG(GetSourceActor(), LogNavigation, Log, TEXT("Initializing meta path, Waypoints:%d GoalActor:%s"),
			Waypoints.Num(), *GetNameSafe(GetGoalActor()));

		PathGoal = MakeWeakObjectPtr(const_cast<AActor*>(GetGoalActor()));
		PathGoalTetherDistance = GetGoalActorTetherDistance();
		DisableGoalActorObservation();

		for (int32 Idx = 1; Idx < Waypoints.Num(); Idx++)
		{
			ApproximateLength = FVector::Dist(Waypoints[Idx - 1], Waypoints[Idx]);
		}

		MoveToNextSection(AgentLocation);
	}
}

bool FMetaNavMeshPath::SetWaypoints(const TArray<FMetaPathWayPoint>& InWaypoints)
{
	if (TargetWaypointIdx == 0)
	{
		Waypoints = InWaypoints;

		if (Waypoints.Num() >= 2)
		{
			PathPoints.SetNum(2);
			PathPoints[0] = Waypoints[0];
			PathPoints[1] = Waypoints.Last();
		}

		MarkReady();
		return true;
	}

	return false;
}

bool FMetaNavMeshPath::SetWaypoints(const TArray<FVector>& InWaypoints)
{
	if (TargetWaypointIdx == 0)
	{
		Waypoints.Reset();
		for (const FVector& Location : InWaypoints)
		{
			Waypoints.Add(Location);
		}

		if (Waypoints.Num() >= 2)
		{
			PathPoints.SetNum(2);
			PathPoints[0] = Waypoints[0];
			PathPoints[1] = Waypoints.Last();
		}

		MarkReady();
		return true;
	}

	return false;
}

bool FMetaNavMeshPath::ConditionalMoveToNextSection(const FVector& AgentLocation, EMetaPathUpdateReason Reason)
{
	if (Waypoints.IsValidIndex(TargetWaypointIdx))
	{
		const FVector::FReal DistSq = FVector::DistSquared(AgentLocation, Waypoints[TargetWaypointIdx]);
		if (((Reason == EMetaPathUpdateReason::PathFinished) || (DistSq < FMath::Square(WaypointSwitchRadius)))
			&& (GetSourceActorAsNavAgent() == nullptr || GetSourceActorAsNavAgent()->ShouldPostponePathUpdates() == false))
		{
			return MoveToNextSection(AgentLocation);
		}
	}

	return false;
}

bool FMetaNavMeshPath::ForceMoveToNextSection(const FVector& AgentLocation)
{
	return MoveToNextSection(AgentLocation);
}

bool FMetaNavMeshPath::MoveToNextSection(const FVector& AgentLocation)
{
	if (Waypoints.IsValidIndex(TargetWaypointIdx + 1))
	{
		TargetWaypointIdx++;
		return UpdatePath(AgentLocation);
	}

	return false;
}

bool FMetaNavMeshPath::UpdatePath(const FVector& AgentLocation)
{
	ANavigationData* NavData = NavigationDataUsed.Get();
	if (Waypoints.IsValidIndex(TargetWaypointIdx) && ensure(NavData))
	{
		PathFindingQueryData.StartLocation = AgentLocation;
		PathFindingQueryData.EndLocation = Waypoints[TargetWaypointIdx];
		bUpdateEndPointOnRepath = false;

		// try restoring goal actor observation for following last section of path
		const bool bIsLastSection = IsLastSection();
		if (bIsLastSection)
		{
			AActor* PathGoalOb = PathGoal.Get();
			if (PathGoalOb)
			{
				SetGoalActorObservation(*PathGoalOb, PathGoalTetherDistance);
				bUpdateEndPointOnRepath = true;
			}
		}

		PathPoints.SetNum(2);
		PathPoints[0] = FNavPathPoint(AgentLocation);
		PathPoints[1] = FNavPathPoint(PathFindingQueryData.EndLocation);

		UE_VLOG(GetSourceActor(), LogNavigation, Log, TEXT("Updating meta path, waypoint:%d/%d"), TargetWaypointIdx, Waypoints.Num() - 1);
		NavData->RequestRePath(AsShared(), ENavPathUpdateType::MetaPathUpdate);
		return true;
	}

	return false;
}

void FMetaNavMeshPath::CopyFrom(const FMetaNavMeshPath& Other)
{
	ResetForRepath();

	TargetWaypointIdx = 0;
	SetWaypoints(Other.Waypoints);

	WaypointSwitchRadius = Other.WaypointSwitchRadius;
	PathGoal = Other.PathGoal;
	PathGoalTetherDistance = Other.PathGoalTetherDistance;
	ApproximateLength = Other.ApproximateLength;
}

FVector::FReal FMetaNavMeshPath::GetLengthFromPosition(FVector SegmentStart, uint32 NextPathPointIndex) const
{
	// return approximation of full path, there's not enough data to give accurate value
	return ApproximateLength;
}

FVector::FReal FMetaNavMeshPath::GetCostFromIndex(int32 PathPointIndex) const
{
	// return approximation of full path * default cost, there's not enough data to give accurate value
	const UNavArea* DefaultAreaOb = static_cast<const UNavArea*>(FNavigationSystem::GetDefaultWalkableArea().GetDefaultObject());
	const float DefaultAreaCost = DefaultAreaOb ? DefaultAreaOb->DefaultCost : 1.0f;
	return ApproximateLength * DefaultAreaCost;
}

#if ENABLE_VISUAL_LOG
void FMetaNavMeshPath::DescribeSelfToVisLog(FVisualLogEntry* Snapshot) const
{
	Super::DescribeSelfToVisLog(Snapshot);

	if (Waypoints.Num())
	{
		FVisualLogShapeElement Element(EVisualLoggerShapeElement::Path);
		Element.Category = LogNavigation.GetCategoryName();
		Element.SetColor(FColorList::Yellow);
		Element.Points.Reserve(Waypoints.Num());
		Element.Thickness = 3;

		const FVector DrawingOffset(0, 0, 50);
		for (int32 Idx = 0; Idx < Waypoints.Num(); Idx++)
		{
			Element.Points.Add(Waypoints[Idx] + DrawingOffset);
		}

		Snapshot->ElementsToDraw.Add(Element);
	}
}
#endif

void FMetaNavMeshPath::DebugDraw(const ANavigationData* NavData, const FColor PathColor, UCanvas* Canvas, const bool bPersistent, const float LifeTime, const uint32 NextPathPointIndex) const
{
#if ENABLE_DRAW_DEBUG
	if (Waypoints.Num() > 0)
	{
		Super::DebugDraw(NavData, PathColor, Canvas, bPersistent, LifeTime, NextPathPointIndex);

		static const FVector DrawingOffset(0, 0, 50);
		const UWorld* World = NavData->GetWorld();
		FVector WaypointLocation = Waypoints[0];

		for (int32 WaypointIndex = 1; WaypointIndex < Waypoints.Num(); ++WaypointIndex)
		{
			const FVector NextWaypoint = Waypoints[WaypointIndex];
			DrawDebugLine(World, WaypointLocation + NavigationDebugDrawing::PathOffset, NextWaypoint + NavigationDebugDrawing::PathOffset
				, FColor::Orange, bPersistent, LifeTime, /*DepthPriority*/0
				, /*Thickness*/NavigationDebugDrawing::PathLineThickness + 1);
			WaypointLocation = NextWaypoint;
		}
	}
#endif // ENABLE_DRAW_DEBUG
}

