// Copyright Epic Games, Inc. All Rights Reserved.

#include "Navigation/NavFilter_AIControllerDefault.h"
#include "GameFramework/Pawn.h"
#include "AIController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavFilter_AIControllerDefault)

UNavFilter_AIControllerDefault::UNavFilter_AIControllerDefault(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsMetaFilter = true;	
}

TSubclassOf<UNavigationQueryFilter> UNavFilter_AIControllerDefault::GetSimpleFilterForAgent(const UObject& Querier) const
{
	const APawn* AsPawn = Cast<const APawn>(&Querier);
	const AAIController* AsAIController = Cast<const AAIController>(AsPawn ? AsPawn->GetController() : &Querier);

	return ensure(AsAIController) ? AsAIController->GetDefaultNavigationFilterClass() : nullptr;
}

