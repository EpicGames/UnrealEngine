// Copyright Epic Games, Inc. All Rights Reserved.

#include "Navigation/NavLinkProxy.h"
#include "UObject/ConstructorHelpers.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/Controller.h"
#include "NavigationSystem.h"
#include "NavigationSystemTypes.h"
#include "Components/BillboardComponent.h"
#include "Engine/Texture2D.h"
#include "Navigation/PathFollowingComponent.h"
#include "NavLinkCustomComponent.h"
#include "NavLinkRenderingComponent.h"
#include "NavAreas/NavArea_Default.h"
#include "AI/NavigationSystemHelpers.h"
#include "VisualLogger/VisualLogger.h"
#include "NavigationOctree.h"
#include "ObjectEditorUtils.h"
#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavLinkProxy)

#define LOCTEXT_NAMESPACE "NavLink"

#if WITH_EDITOR
namespace UE::Navigation::LinkProxy::Private
{
	void OnNavAreaRegistrationChanged(ANavLinkProxy& LinkProxy, const UWorld& World, const UClass* NavAreaClass)
	{
		if (&World != LinkProxy.GetWorld())
		{
			return;
		}

		bool bUpdateActor = false;

		for (const FNavigationLink& NavLink : LinkProxy.PointLinks)
		{
			if (NavLink.GetAreaClass() == NavAreaClass)
			{
				bUpdateActor = true;
				break;
			}
		}

		if (bUpdateActor)
		{
			FNavigationSystem::UpdateActorData(LinkProxy);
		}
	}
} // UE::Navigation::LinkProxy::Private
#endif // WITH_EDITOR

ANavLinkProxy::ANavLinkProxy(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	USceneComponent* SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("PositionComponent"));
	RootComponent = SceneComponent;

	SetHidden(true);

#if WITH_EDITORONLY_DATA
	EdRenderComp = CreateDefaultSubobject<UNavLinkRenderingComponent>(TEXT("EdRenderComp"));
	EdRenderComp->SetupAttachment(RootComponent);
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));
	if (!IsRunningCommandlet() && (SpriteComponent != NULL))
	{
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> SpriteTexture;
			FName ID_Decals;
			FText NAME_Decals;
			FConstructorStatics()
				: SpriteTexture(TEXT("/Engine/EditorResources/AI/S_NavLink"))
				, ID_Decals(TEXT("Navigation"))
				, NAME_Decals(NSLOCTEXT("SpriteCategory", "Navigation", "Navigation"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		SpriteComponent->Sprite = ConstructorStatics.SpriteTexture.Get();
		SpriteComponent->SetRelativeScale3D(FVector(0.5f, 0.5f, 0.5f));
		SpriteComponent->bHiddenInGame = true;
		SpriteComponent->SetVisibleFlag(true);
		SpriteComponent->SpriteInfo.Category = ConstructorStatics.ID_Decals;
		SpriteComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Decals;
		SpriteComponent->SetupAttachment(RootComponent);
		SpriteComponent->SetUsingAbsoluteScale(true);
		SpriteComponent->bIsScreenSizeScaled = true;
	}
#endif

	SmartLinkComp = CreateDefaultSubobject<UNavLinkCustomComponent>(TEXT("SmartLinkComp"));
	SmartLinkComp->SetNavigationRelevancy(false);
	SmartLinkComp->SetMoveReachedLink(this, &ANavLinkProxy::NotifySmartLinkReached);
	bSmartLinkIsRelevant = false;

	FNavigationLink DefLink;
	DefLink.SetAreaClass(UNavArea_Default::StaticClass());

	PointLinks.Add(DefLink);

	SetActorEnableCollision(false);
	SetCanBeDamaged(false);
}

void ANavLinkProxy::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	// During loading, those will be registered to in postload instead for thread-safety
	if (GIsEditor && !HasAnyFlags(RF_ClassDefaultObject) && !IsInAsyncLoadingThread())
	{
		OnNavAreaRegisteredDelegateHandle = UNavigationSystemBase::OnNavAreaRegisteredDelegate().AddUObject(this, &ANavLinkProxy::OnNavAreaRegistered);
		OnNavAreaUnregisteredDelegateHandle = UNavigationSystemBase::OnNavAreaUnregisteredDelegate().AddUObject(this, &ANavLinkProxy::OnNavAreaUnregistered);
	}
#endif // WITH_EDITOR
}

void ANavLinkProxy::BeginDestroy()
{
#if WITH_EDITOR
	if (GIsEditor && !HasAnyFlags(RF_ClassDefaultObject))
	{
		UNavigationSystemBase::OnNavAreaRegisteredDelegate().Remove(OnNavAreaRegisteredDelegateHandle);
		UNavigationSystemBase::OnNavAreaUnregisteredDelegate().Remove(OnNavAreaUnregisteredDelegateHandle);
	}
#endif // WITH_EDITOR

	Super::BeginDestroy();
}

#if WITH_EDITOR
// This function is only called if GIsEditor == true
void ANavLinkProxy::OnNavAreaRegistered(const UWorld& World, const UClass* NavAreaClass)
{
	UE::Navigation::LinkProxy::Private::OnNavAreaRegistrationChanged(*this, World, NavAreaClass);
}

// This function is only called if GIsEditor == true
void ANavLinkProxy::OnNavAreaUnregistered(const UWorld& World, const UClass* NavAreaClass)
{
	UE::Navigation::LinkProxy::Private::OnNavAreaRegistrationChanged(*this, World, NavAreaClass);
}

void ANavLinkProxy::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) 
{
	static const FName NAME_SmartLinkIsRelevant = GET_MEMBER_NAME_CHECKED(ANavLinkProxy, bSmartLinkIsRelevant);
	static const FName NAME_PointLinks = GET_MEMBER_NAME_CHECKED(ANavLinkProxy, PointLinks);
	static const FName NAME_AreaClass = "AreaClass";
	const FName PropertyName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	const FName MemberPropertyName = PropertyChangedEvent.MemberProperty ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;

	bool bUpdateInNavOctree = false;
	if (PropertyName == NAME_SmartLinkIsRelevant)
	{
		SmartLinkComp->SetNavigationRelevancy(bSmartLinkIsRelevant);
		bUpdateInNavOctree = true;
	}

	const FName CategoryName = FObjectEditorUtils::GetCategoryFName(PropertyChangedEvent.Property);
	const FName MemberCategoryName = FObjectEditorUtils::GetCategoryFName(PropertyChangedEvent.MemberProperty);
	if (CategoryName == TEXT("SimpleLink") || MemberCategoryName == TEXT("SimpleLink"))
	{
		bUpdateInNavOctree = true;
		if (PropertyName == NAME_AreaClass && MemberPropertyName == NAME_PointLinks)
		{
			for (FNavigationLink& Link : PointLinks)
			{
				Link.InitializeAreaClass(/*bForceRefresh=*/true);
			}
		}
	}

	if (bUpdateInNavOctree)
	{
		UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
		if (NavSys)
		{
			FNavigationSystem::UpdateActorData(*this);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void ANavLinkProxy::PostEditUndo()
{
	Super::PostEditUndo();

	for (FNavigationLink& Link : PointLinks)
	{
		Link.InitializeAreaClass(/*bForceRefresh=*/true);
	}
}

void ANavLinkProxy::PostEditImport()
{
	Super::PostEditImport();

	for (FNavigationLink& Link : PointLinks)
	{
		Link.InitializeAreaClass(/*bForceRefresh=*/true);
	}
}
#endif // WITH_EDITOR

void ANavLinkProxy::PostRegisterAllComponents()
{
	// Check if the smart link is enabled and valid for determining navigation relevancy
	// A smart link with no simple links will result in warnings for invalid bounds if Super is called prior to setting the navigation relevancy of the smart link.
	if (SmartLinkComp)
	{
		SmartLinkComp->SetNavigationRelevancy(bSmartLinkIsRelevant);
	}

	Super::PostRegisterAllComponents();
}

void ANavLinkProxy::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (GIsEditor && !HasAnyFlags(RF_ClassDefaultObject))
	{
		OnNavAreaRegisteredDelegateHandle = UNavigationSystemBase::OnNavAreaRegisteredDelegate().AddUObject(this, &ANavLinkProxy::OnNavAreaRegistered);
		OnNavAreaUnregisteredDelegateHandle = UNavigationSystemBase::OnNavAreaUnregisteredDelegate().AddUObject(this, &ANavLinkProxy::OnNavAreaUnregistered);
	}
#endif // WITH_EDITOR

	for (FNavigationLink& Link : PointLinks)
	{
		Link.InitializeAreaClass();
	}
}

#if ENABLE_VISUAL_LOG
void ANavLinkProxy::BeginPlay()
{
	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
	if (NavSys)
	{
		REDIRECT_OBJECT_TO_VLOG(this, NavSys);
	}

	Super::BeginPlay();
}
#endif // ENABLE_VISUAL_LOG

void ANavLinkProxy::GetNavigationData(FNavigationRelevantData& Data) const
{
	NavigationHelper::ProcessNavLinkAndAppend(&Data.Modifiers, this, PointLinks);
	NavigationHelper::ProcessNavLinkSegmentAndAppend(&Data.Modifiers, this, SegmentLinks);
}

FBox ANavLinkProxy::GetNavigationBounds() const
{
	return GetComponentsBoundingBox();
}

bool ANavLinkProxy::IsNavigationRelevant() const
{
	return (PointLinks.Num() > 0) || (SegmentLinks.Num() > 0) || bSmartLinkIsRelevant;
}

bool ANavLinkProxy::GetNavigationLinksClasses(TArray<TSubclassOf<UNavLinkDefinition> >& OutClasses) const
{
	return false;
}

bool ANavLinkProxy::GetNavigationLinksArray(TArray<FNavigationLink>& OutLink, TArray<FNavigationSegmentLink>& OutSegments) const
{
	OutLink.Append(PointLinks);

	const bool bIsSmartLinkActive = (SmartLinkComp && SmartLinkComp->IsNavigationRelevant());
	if (bIsSmartLinkActive)
	{
		OutLink.Add(SmartLinkComp->GetLinkModifier());
	}

	OutSegments.Append(SegmentLinks);

	return (PointLinks.Num() > 0) || (SegmentLinks.Num() > 0) || bIsSmartLinkActive;
}

FBox ANavLinkProxy::GetComponentsBoundingBox(bool bNonColliding, bool bIncludeFromChildActors) const
{
	FBox LinksBB(FVector(0.f, 0.f, -10.f), FVector(0.f,0.f,10.f));

	for (int32 i = 0; i < PointLinks.Num(); ++i)
	{
		const FNavigationLink& Link = PointLinks[i];
		LinksBB += Link.Left;
		LinksBB += Link.Right;
	}

	for (int32 i = 0; i < SegmentLinks.Num(); ++i)
	{
		const FNavigationSegmentLink& SegmentLink = SegmentLinks[i];
		LinksBB += SegmentLink.LeftStart;
		LinksBB += SegmentLink.LeftEnd;
		LinksBB += SegmentLink.RightStart;
		LinksBB += SegmentLink.RightEnd;
	}

	if (RootComponent)
	{
		LinksBB = LinksBB.TransformBy(RootComponent->GetComponentTransform());
	}

	if (SmartLinkComp && SmartLinkComp->IsNavigationRelevant())
	{
		LinksBB += SmartLinkComp->GetStartPoint();
		LinksBB += SmartLinkComp->GetEndPoint();
	}

	return LinksBB;
}

void ANavLinkProxy::NotifySmartLinkReached(UNavLinkCustomComponent* LinkComp, UObject* PathingAgent, const FVector& DestPoint)
{
	UPathFollowingComponent* PathComp = Cast<UPathFollowingComponent>(PathingAgent);
	if (PathComp)
	{
		AActor* PathOwner = PathComp->GetOwner();
		AController* ControllerOwner = Cast<AController>(PathOwner);
		if (ControllerOwner)
		{
			PathOwner = ControllerOwner->GetPawn();
		}

		ReceiveSmartLinkReached(PathOwner, DestPoint);
		OnSmartLinkReached.Broadcast(PathOwner, DestPoint);
	}
}

void ANavLinkProxy::ResumePathFollowing(AActor* Agent)
{
	if (Agent)
	{
		UPathFollowingComponent* PathComp = Agent->FindComponentByClass<UPathFollowingComponent>();
		if (PathComp == NULL)
		{
			APawn* PawnOwner = Cast<APawn>(Agent);
			if (PawnOwner && PawnOwner->GetController())
			{
				PathComp = PawnOwner->GetController()->FindComponentByClass<UPathFollowingComponent>();
			}
		}

		if (PathComp)
		{
			PathComp->FinishUsingCustomLink(SmartLinkComp);
		}
	}
}

bool ANavLinkProxy::IsSmartLinkEnabled() const
{
	return SmartLinkComp->IsEnabled();
}

void ANavLinkProxy::SetSmartLinkEnabled(bool bEnabled)
{
	SmartLinkComp->SetEnabled(bEnabled);
}

bool ANavLinkProxy::HasMovingAgents() const
{
	return SmartLinkComp->HasMovingAgents();
}

#if WITH_EDITOR
void ANavLinkProxy::CopyEndPointsFromSimpleLinkToSmartLink()
{
	if (PointLinks.Num() && SmartLinkComp)
	{
		{
			const FScopedTransaction Transaction(LOCTEXT("SetLinkData", "Set Link Data"));
			SmartLinkComp->Modify();
			SmartLinkComp->SetLinkData(PointLinks[0].Left, PointLinks[0].Right, PointLinks[0].Direction);
		}
		if (EdRenderComp)
		{
			EdRenderComp->MarkRenderStateDirty();
		}
	}
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE