// Copyright Epic Games, Inc. All Rights Reserved.

#include "Navigation/NavLocalGridData.h"
#include "Navigation/NavLocalGridManager.h"
#include "GraphAStar.h"
#include "NavigationPath.h"
#include "GeomTools.h"


FNavLocalGridData::FNavLocalGridData(const FVector& Center, float Extent2D) : GridId(0)
{
	const float UseCellSize = UNavLocalGridManager::GetCellSize();
	const FVector CenterOnGrid(FMath::RoundToInt(Center.X / UseCellSize) * UseCellSize, FMath::RoundToInt(Center.Y / UseCellSize) * UseCellSize, FMath::RoundToInt(Center.Z / UseCellSize) * UseCellSize);
	const FBox UseBounds = FBox::BuildAABB(CenterOnGrid, FVector(Extent2D, Extent2D, 0.0f));
	Init(UseCellSize, UseBounds);
	
	const int32 OriginX = IntCastChecked<int32>(FMath::TruncToInt(Origin.X / UseCellSize));
	const int32 OriginY = IntCastChecked<int32>(FMath::TruncToInt(Origin.Y / UseCellSize));
	const int32 OriginZ = IntCastChecked<int32>(FMath::TruncToInt(Origin.Z / UseCellSize));

	OriginWorldCoord = FIntVector(OriginX, OriginY, OriginZ);
}

FNavLocalGridData::FNavLocalGridData(const FVector& Center, const FVector2D& Extent2D) : GridId(0)
{
	const float UseCellSize = UNavLocalGridManager::GetCellSize();
	const FVector CenterOnGrid(FMath::RoundToInt(Center.X / UseCellSize) * UseCellSize, FMath::RoundToInt(Center.Y / UseCellSize) * UseCellSize, FMath::RoundToInt(Center.Z / UseCellSize) * UseCellSize);
	const FBox UseBounds = FBox::BuildAABB(CenterOnGrid, FVector(Extent2D.X, Extent2D.Y, 0.0f));
	Init(UseCellSize, UseBounds);

	const int32 OriginX = IntCastChecked<int32>(FMath::TruncToInt(Origin.X / UseCellSize));
	const int32 OriginY = IntCastChecked<int32>(FMath::TruncToInt(Origin.Y / UseCellSize));
	const int32 OriginZ = IntCastChecked<int32>(FMath::TruncToInt(Origin.Z / UseCellSize));

	OriginWorldCoord = FIntVector(OriginX, OriginY, OriginZ);
}

FNavLocalGridData::FNavLocalGridData(const TArray<FNavLocalGridData>& SourceGrids) : GridId(0)
{
	if (SourceGrids.Num() == 0)
	{
		return;
	}

	FBox CombinedBounds(ForceInit);
	for (int32 Idx = 0; Idx < SourceGrids.Num(); Idx++)
	{
		CombinedBounds += SourceGrids[Idx].WorldBounds;
	}

	const float UseCellSize = UNavLocalGridManager::GetCellSize();
	Init(UseCellSize, CombinedBounds);

	const int32 OriginX = IntCastChecked<int32>(FMath::TruncToInt(Origin.X / UseCellSize));
	const int32 OriginY = IntCastChecked<int32>(FMath::TruncToInt(Origin.Y / UseCellSize));
	const int32 OriginZ = IntCastChecked<int32>(FMath::TruncToInt(Origin.Z / UseCellSize));

	for (int32 Idx = 0; Idx < Cells.Num(); Idx++)
	{
		const FIntVector WorldPos = GetGlobalCoords(Idx);
		bool bIsSet = false;

		for (int32 SourceIdx = 0; SourceIdx < SourceGrids.Num() && !bIsSet; SourceIdx++)
		{
			const FNavLocalGridData& SourceData = SourceGrids[SourceIdx];
			const int32 CellIdx = SourceData.GetCellIndexFromGlobalCoords2D(WorldPos);

			if (CellIdx != INDEX_NONE)
			{
				bIsSet = SourceData[CellIdx] != 0;
			}
		}

		Cells[Idx] = bIsSet ? 1 : 0;
	}
}

int32 FNavLocalGridData::GetCellIndexFromGlobalCoords2D(const FIntVector& WorldCoords) const
{
	const int32 PosX = WorldCoords.X - OriginWorldCoord.X;
	const int32 PosY = WorldCoords.Y - OriginWorldCoord.Y;

	if (PosX >= 0 && PosX < (int32)GridSize.Width &&
		PosY >= 0 && PosY < (int32)GridSize.Height)
	{
		return GetCellIndexUnsafe(PosX, PosY);
	}

	return INDEX_NONE;
}

void FNavLocalGridData::MarkPointObstacle(const FVector& Center)
{
	const int32 CellIndex = GetCellIndex(Center);
	if (CellIndex != INDEX_NONE)
	{
		Cells[CellIndex] = 1;
	}
}

void FNavLocalGridData::MarkBoxObstacle(const FVector& Center, const FVector& Extent, const FQuat& Quat)
{
	if (Quat.IsIdentity())
	{
		const FIntVector ClampedBoxMin = GetCellCoords(Center - Extent);
		const FIntVector ClampedBoxMax = GetCellCoords(Center + Extent);

		for (int32 IdxX = ClampedBoxMin.X; IdxX <= ClampedBoxMax.X; IdxX++)
		{
			for (int32 IdxY = ClampedBoxMin.Y; IdxY <= ClampedBoxMax.Y; IdxY++)
			{
				const int32 CellIndex = GetCellIndexUnsafe(IdxX, IdxY);
				Cells[CellIndex] = 1;
			}
		}
	}
	else
	{
		TArray<FVector2D> Verts2D;
		Verts2D.Add(FVector2D(Center + Quat.RotateVector(FVector(Extent.X, Extent.Y, Extent.Z))));
		Verts2D.Add(FVector2D(Center + Quat.RotateVector(FVector(-Extent.X, Extent.Y, Extent.Z))));
		Verts2D.Add(FVector2D(Center + Quat.RotateVector(FVector(-Extent.X, -Extent.Y, Extent.Z))));
		Verts2D.Add(FVector2D(Center + Quat.RotateVector(FVector(Extent.X, -Extent.Y, Extent.Z))));

		for (int32 IdxX = 0; IdxX < (int32)GridSize.Width; IdxX++)
		{
			for (int32 IdxY = 0; IdxY < (int32)GridSize.Height; IdxY++)
			{
				const FVector GridCellLocation = GetWorldCellCenter(IdxX, IdxY);
				const bool bIsInside = FGeomTools2D::IsPointInPolygon(FVector2D(GridCellLocation), Verts2D);
				if (bIsInside)
				{
					const int32 CellIndex = GetCellIndexUnsafe(IdxX, IdxY);
					Cells[CellIndex] = 1;
				}
			}
		}
	}
}

void FNavLocalGridData::MarkCapsuleObstacle(const FVector& Center, float Radius, float HalfHeight)
{
	const float RadiusSq = FMath::Square(Radius);
	for (int32 IdxX = 0; IdxX < (int32)GridSize.Width; IdxX++)
	{
		for (int32 IdxY = 0; IdxY < (int32)GridSize.Height; IdxY++)
		{
			const FVector GridCellLocation = GetWorldCellCenter(IdxX, IdxY);
			const FVector::FReal Dist2DSq = (GridCellLocation - Center).SizeSquared2D();
			const FVector::FReal DistZ = FMath::Abs(GridCellLocation.Z - Center.Z);

			const bool bIsInside = (Dist2DSq < RadiusSq) && (DistZ < HalfHeight);
			if (bIsInside)
			{
				const int32 CellIndex = GetCellIndexUnsafe(IdxX, IdxY);
				Cells[CellIndex] = 1;
			}
		}
	}
}

void FNavLocalGridData::SetHeight(float ExtentZ)
{
	const FVector::FReal CenterZ = (WorldBounds.Max.Z + WorldBounds.Min.Z) * 0.5;
	WorldBounds.Min.Z = CenterZ - ExtentZ;
	WorldBounds.Max.Z = CenterZ + ExtentZ;
}

void FNavLocalGridData::SetGridId(int32 NewId)
{
	GridId = NewId;
}

void FNavLocalGridData::FindPathForMovingAgent(const FNavigationPath& SourcePath, const FVector& EntryLocation, int32 EntrySegmentStart, TArray<FVector>& PathPointsInside, int32& NextSegmentStart) const
{
	// prepare Start & End points for grid pathfinding
	FIntVector StartPos = GetCellCoords(EntryLocation);
	FIntVector EndPos = GetCellCoords(SourcePath.GetEndLocation());

	int32 NextSegmentEnd = EntrySegmentStart + 1;
	for (; NextSegmentEnd < SourcePath.GetPathPoints().Num(); NextSegmentEnd++)
	{
		const bool bIsSegmentEndInside = WorldBounds.IsInside(SourcePath.GetPathPoints()[NextSegmentEnd].Location);
		if (!bIsSegmentEndInside)
		{
			FVector HitLocation, DummyNormal;
			float DummyTime;

			FMath::LineExtentBoxIntersection(WorldBounds, SourcePath.GetPathPoints()[NextSegmentEnd].Location, SourcePath.GetPathPoints()[NextSegmentEnd - 1].Location, FVector::ZeroVector, HitLocation, DummyNormal, DummyTime);
			EndPos = GetCellCoords(HitLocation);

			break;
		}
	}

	const FIntVector OrgStartPos = StartPos;
	int32 StartCellIdx = GetCellIndex(StartPos.X, StartPos.Y);
	bool bHasAdjustedStart = false;

	if (StartCellIdx == INDEX_NONE || GetCellAtIndexUnsafe(StartCellIdx) != 0)
	{
		const FVector AdjustDirection = (EntryLocation - SourcePath.GetEndLocation()).GetSafeNormal();
		const float MaxAdjustDistance = 150.0f;
		const int32 MaxAdjustSteps = 15;

		FIntVector PrevTestPos = StartPos;
		for (int32 StepIdx = 1; StepIdx < MaxAdjustSteps; StepIdx++)
		{
			FIntVector TestPos = GetCellCoords(EntryLocation + (AdjustDirection * MaxAdjustDistance * StepIdx / MaxAdjustSteps));
			if (TestPos != PrevTestPos)
			{
				PrevTestPos = TestPos;

				StartCellIdx = GetCellIndex(TestPos.X, TestPos.Y);
				if (StartCellIdx != INDEX_NONE && GetCellAtIndexUnsafe(StartCellIdx) == 0)
				{
					StartPos = TestPos;
					bHasAdjustedStart = true;
					break;
				}
			}
		}
	}

	NextSegmentStart = NextSegmentEnd - 1;
	PathPointsInside.Reset();

	// find path
	TArray<FIntVector> PathCoords;

	const bool bResult = FindPath(StartPos, EndPos, PathCoords);
	if (bResult)
	{
		if (bHasAdjustedStart)
		{
			PathPointsInside.Add(GetProjectedCellCenter(OrgStartPos.X, OrgStartPos.Y));
		}

		for (int32 Idx = 0; Idx < PathCoords.Num(); Idx++)
		{
			PathPointsInside.Add(GetProjectedCellCenter(PathCoords[Idx].X, PathCoords[Idx].Y));
		}
	}
}

namespace FGridHelpers
{
	static FIntPoint Directions[] = { FIntPoint(1, 0), FIntPoint(1, -1), FIntPoint(0, -1), FIntPoint(-1, -1), FIntPoint(-1, 0), FIntPoint(-1, 1), FIntPoint(0, 1), FIntPoint(1, 1) };

	struct FGridPathFilter
	{
		FGridPathFilter(const FNavLocalGridData& InGridRef) : GridRef(InGridRef) {}

		FVector::FReal GetHeuristicScale() const
		{
			return 1.;
		}

		FVector::FReal GetHeuristicCost(const int32 StartNodeRef, const int32 EndNodeRef) const
		{
			return GetTraversalCost(StartNodeRef, EndNodeRef);
		}

		FVector::FReal GetTraversalCost(const int32 StartNodeRef, const int32 EndNodeRef) const
		{
			// manhattan distance
			return FMath::Abs(GridRef.GetCellCoordX(StartNodeRef) - GridRef.GetCellCoordX(EndNodeRef)) + FMath::Abs(GridRef.GetCellCoordY(StartNodeRef) - GridRef.GetCellCoordY(EndNodeRef));
		}

		bool IsTraversalAllowed(const int32 NodeA, const int32 NodeB) const
		{
			return GridRef[NodeB] == 0;
		}

		bool WantsPartialSolution() const
		{
			return true;
		}

	protected:
		const FNavLocalGridData& GridRef;
	};

}

bool FNavLocalGridData::FindPath(const FIntVector& StartCoords, const FIntVector& EndCoords, TArray<FIntVector>& PathCoords) const
{
	const int32 StartIdx = GetCellIndex(StartCoords.X, StartCoords.Y);
	const int32 EndIdx = GetCellIndex(EndCoords.X, EndCoords.Y);

	TArray<int32> PathIndices;
	FGraphAStar<FNavLocalGridData> Pathfinder(*this);
	Pathfinder.FindPath(StartIdx, EndIdx, FGridHelpers::FGridPathFilter(*this), PathIndices);

	if (PathIndices.Num())
	{
		PathCoords.Reset();
		PostProcessPath(StartCoords, EndCoords, PathIndices, PathCoords);
		return true;
	}

	return false;
}

void FNavLocalGridData::PostProcessPath(const FIntVector& StartCoords, const FIntVector& EndCoords, const TArray<int32>& PathIndices, TArray<FIntVector>& PathCoords) const
{
	// string pulling
	PathCoords.Reserve(PathIndices.Num());
	PathCoords.Add(StartCoords);

	if (PathIndices.Num() > 2)
	{
		FIntVector PrevCoords(StartCoords);
		for (int32 Idx = 2; Idx < PathIndices.Num(); Idx++)
		{
			const FIntVector TestCoords(GetCellCoords(PathIndices[Idx]));
			const bool bHit = IsLineObstructed(PrevCoords, TestCoords);
			if (bHit)
			{
				PrevCoords = GetCellCoords(PathIndices[Idx - 1]);
				PathCoords.Add(PrevCoords);
			}
		}
	}

	PathCoords.Add(EndCoords);
	PathCoords.Shrink();
}

bool FNavLocalGridData::IsLineObstructed(const FIntVector& StartCoords, const FIntVector& EndCoords) const
{
	if (HasObstacleUnsafe(StartCoords.X, StartCoords.Y))
	{
		return true;
	}

	// Bresenham's line algorithm
	int32 DX = FMath::Abs(EndCoords.X - StartCoords.X);
	int32 DY = FMath::Abs(EndCoords.Y - StartCoords.Y);
	const int32 StepX = (EndCoords.X > StartCoords.X) ? 1 : -1;
	const int32 StepY = (EndCoords.Y > StartCoords.Y) ? 1 : -1;
	DX <<= 1;
	DY <<= 1;

	int32 PosX = StartCoords.X;
	int32 PosY = StartCoords.Y;
	if (DX > DY)
	{
		int32 ErrorAcc = DY - (DX >> 1);
		while (PosX != EndCoords.X)
		{
			if (ErrorAcc >= 0)
			{
				PosY += StepY;
				ErrorAcc -= DX;
			}
			PosX += StepX;
			ErrorAcc += DY;

			if (HasObstacleUnsafe(PosX, PosY))
			{
				return true;
			}
		}
	}
	else
	{
		int32 ErrorAcc = DX - (DY >> 1);
		while (PosY != EndCoords.Y)
		{
			if (ErrorAcc >= 0)
			{
				PosX += StepX;
				ErrorAcc -= DY;
			}
			PosY += StepY;
			ErrorAcc += DX;

			if (HasObstacleUnsafe(PosX, PosY))
			{
				return true;
			}
		}
	}

	return false;
}

int32 FNavLocalGridData::GetNeighbour(const int32 NodeRef, const int32 NeiIndex) const
{
	return GetCellIndex(GetCellCoordX(NodeRef) + FGridHelpers::Directions[NeiIndex].X, GetCellCoordY(NodeRef) + FGridHelpers::Directions[NeiIndex].Y);
}

void FNavLocalGridData::ProjectCells(const ANavigationData& NavData)
{
	const bool bAssignBoundsHeight = FMath::IsNearlyZero(WorldBounds.GetExtent().Z);
	if (bAssignBoundsHeight)
	{
		SetHeight(FloatCastChecked<float>(NavData.GetDefaultQueryExtent().Z, UE::LWC::DefaultFloatPrecision));
	}

	TArray<FNavigationProjectionWork> Workload;
	for (int32 Idx = 0; Idx < Cells.Num(); Idx++)
	{
		if (Cells[Idx] == 0)
		{
			FNavigationProjectionWork CellProjectionInfo(GetWorldCellCenter(Idx));
			CellProjectionInfo.bHintProjection2D = true;

			Workload.Add(CellProjectionInfo);
		}
	}

	const FVector ProjectionExtent(GridCellSize * 0.25f, GridCellSize * 0.25f, (WorldBounds.Max.Z - WorldBounds.Min.Z) * 0.5f);
	const FVector::FReal Extent2DSq = FMath::Square(ProjectionExtent.X);
	NavData.BatchProjectPoints(Workload, ProjectionExtent);
	
	const FVector::FReal BoundsMidZ = (WorldBounds.Max.Z + WorldBounds.Min.Z) * 0.5;
	CellZ.Init(BoundsMidZ, Cells.Num());

	int32 WorkloadIdx = 0;
	for (int32 Idx = 0; Idx < Cells.Num(); Idx++)
	{
		if (Cells[Idx] == 0)
		{
			const FNavigationProjectionWork& ProjectionData = Workload[WorkloadIdx];
			if (!ProjectionData.bResult || FVector::DistSquaredXY(ProjectionData.OutLocation.Location, ProjectionData.Point) > Extent2DSq)
			{
				Cells[Idx] = 1;
			}
			else
			{
				CellZ[Idx] = ProjectionData.OutLocation.Location.Z;
			}

			WorkloadIdx++;
		}
	}
}
