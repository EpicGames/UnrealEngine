// Copyright Epic Games, Inc. All Rights Reserved.

#include "Navigation/NavLocalGridManager.h"
#include "AISystem.h"
#include "NavigationSystem.h"
#include "Engine/Engine.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavLocalGridManager)

float UNavLocalGridManager::GridCellSize = 50.0f;

UNavLocalGridManager::UNavLocalGridManager(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NextGridId = 1;
	VersionNum = 0;
}

int32 UNavLocalGridManager::AddGridData(const FNavLocalGridData& GridData, bool bUpdate)
{
	const double GameTime = HasSourceGridLimit() ? GetWorld()->GetTimeSeconds() : 0.;
	const int32 NewGridIdx = SourceGrids.Add(GridData);

	SourceGrids[NewGridIdx].SetGridId(NextGridId);
	SourceGrids[NewGridIdx].LastAccessTime = GameTime;

	const int32 UsedId = SourceGrids[NewGridIdx].GetGridId();
	NextGridId++;
	
	UpdateSourceGrids();

	bNeedsRebuilds = true;
	if (bUpdate)
	{
		RebuildGrids();
	}

	return UsedId;
}

void UNavLocalGridManager::RemoveGridData(int32 GridId, bool bUpdate)
{
	for (int32 Idx = 0; Idx < SourceGrids.Num(); Idx++)
	{
		if (SourceGrids[Idx].GetGridId() == GridId)
		{
			SourceGrids.RemoveAt(Idx, EAllowShrinking::No);

			bNeedsRebuilds = true;
			if (bUpdate)
			{
				RebuildGrids();
			}
			break;
		}
	}
}

bool UNavLocalGridManager::SetCellSize(float CellSize)
{
	if (SourceGrids.Num() == 0)
	{
		UNavLocalGridManager::GridCellSize = CellSize;
		return true;
	}

	return false;
}

void UNavLocalGridManager::RebuildGrids()
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("NavGrid: Rebuild"), STAT_GridRebuild, STATGROUP_AI);
	if (!bNeedsRebuilds)
	{
		return;
	}

	TArray<FCombinedNavGridData> PrevCombinedGrids = CombinedGrids;
	CombinedGrids.Reset();

	bNeedsRebuilds = false;
	if (SourceGrids.Num() == 0)
	{
		VersionNum++;
		return;
	}

	// maps source grids to index in MergedBounds
	TArray<int32> MergedId;

	// unique, non overlapping regions
	TArray<FBox> MergedBounds;

	// initialize
	const int32 NumGrids = SourceGrids.Num();
	MergedId.SetNum(NumGrids);
	MergedBounds.SetNum(NumGrids);
	for (int32 Idx = 0; Idx < NumGrids; Idx++)
	{
		MergedId[Idx] = Idx;
		MergedBounds[Idx] = SourceGrids[Idx].WorldBounds;
	}

	// merge regions
	const int32 MaxIters = FMath::Square(NumGrids);
	for (int32 IterIdx = 0; IterIdx < MaxIters; IterIdx++)
	{
		int32 NumMerges = 0;
		
		for (int32 IdxOuter = 0; IdxOuter < NumGrids; IdxOuter++)
		{
			if (MergedId[IdxOuter] != IdxOuter)
			{
				// child region, skip
				continue;
			}

			for (int32 IdxInner = 0; IdxInner < NumGrids; IdxInner++)
			{
				// merge to outer region if possible:
				// is not child region, is not outer region, overlaps with outer region

				if (MergedId[IdxInner] == IdxInner &&
					IdxInner != IdxOuter &&
					MergedBounds[IdxOuter].Intersect(MergedBounds[IdxInner]))
				{
					MergedId[IdxInner] = IdxOuter;
					MergedBounds[IdxOuter] += MergedBounds[IdxInner];
					NumMerges++;
				}
			}
		}

		if (NumMerges == 0)
		{
			break;
		}
	}

	// build combined grids
	TArray<FNavLocalGridData> GridsToCombine;
	TArray<int32> SourceIds;
	TArray<int32> ChangedIndices;

	for (int32 IdxOuter = 0; IdxOuter < NumGrids; IdxOuter++)
	{
		GridsToCombine.Reset();
		SourceIds.Reset();

		for (int32 IdxInner = 0; IdxInner < NumGrids; IdxInner++)
		{
			int32 ParentId = MergedId[IdxInner];
			while (ParentId != MergedId[ParentId])
			{
				ParentId = MergedId[ParentId];
			}

			if (ParentId != IdxOuter)
			{
				continue;
			}

			GridsToCombine.Add(SourceGrids[IdxInner]);
			SourceIds.Add(SourceGrids[IdxInner].GetGridId());
		}

		if (GridsToCombine.Num())
		{
			// check if already built
			bool bFound = false;
			for (int32 IdxOld = 0; IdxOld < PrevCombinedGrids.Num(); IdxOld++)
			{
				if (PrevCombinedGrids[IdxOld].SourceIds == SourceIds)
				{
					CombinedGrids.Add(PrevCombinedGrids[IdxOld]);
					bFound = true;
					break;
				}
			}

			if (!bFound)
			{
				FCombinedNavGridData NewData(GridsToCombine);
				NewData.SetGridId(NextGridId);
				NewData.SourceIds = SourceIds;
				NextGridId++;

				ChangedIndices.Add(CombinedGrids.Num());
				CombinedGrids.Add(NewData);
			}
		}
	}

	if (ChangedIndices.Num())
	{
		ProjectGrids(ChangedIndices);
		VersionNum++;
	}
}

void UNavLocalGridManager::ProjectGrids(const TArray<int32>& GridIndices)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("NavGrid: Project"), STAT_GridProject, STATGROUP_AI);

	const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
	const ANavigationData* NavData = NavSys ? NavSys->GetDefaultNavDataInstance() : nullptr;
	
	if (NavData)
	{
		for (int32 Idx = 0; Idx < GridIndices.Num(); Idx++)
		{
			CombinedGrids[GridIndices[Idx]].ProjectCells(*NavData);
		}
	}
}

void UNavLocalGridManager::UpdateAccessTime(int32 CombinedGridIdx)
{
	if (CombinedGrids.IsValidIndex(CombinedGridIdx))
	{
		const double GameTime = GetWorld()->GetTimeSeconds();

		for (int32 Idx = 0; Idx < CombinedGrids[CombinedGridIdx].SourceIds.Num(); Idx++)
		{
			const int32 SourceGridIdx = CombinedGrids[CombinedGridIdx].SourceIds[Idx];
			if (SourceGrids.IsValidIndex(SourceGridIdx))
			{
				SourceGrids[SourceGridIdx].LastAccessTime = GameTime;
			}
		}
	}
}

void UNavLocalGridManager::SetMaxActiveSources(int32 NumActiveSources)
{
	MaxActiveSourceGrids = NumActiveSources;
	
	const bool bUpdated = UpdateSourceGrids();
	if (bUpdated)
	{
		RebuildGrids();
	}
}

bool UNavLocalGridManager::UpdateSourceGrids()
{
	if (SourceGrids.Num() < MaxActiveSourceGrids || !HasSourceGridLimit())
	{
		return false;
	}

	while (SourceGrids.Num() > MaxActiveSourceGrids)
	{
		double BestScore = DBL_MAX;
		int32 BestIdx = 0;

		for (int32 Idx = 0; Idx < SourceGrids.Num(); Idx++)
		{
			const FNavLocalGridData& GridData = SourceGrids[Idx];
			if (BestScore > GridData.LastAccessTime)
			{
				BestScore = GridData.LastAccessTime;
				BestIdx = Idx;
			}
		}

		SourceGrids.RemoveAt(BestIdx, EAllowShrinking::No);
	}

	return true;
}

int32 UNavLocalGridManager::GetGridIndex(const FVector& WorldLocation) const
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("NavGrid: GetIndex"), STAT_GridFind, STATGROUP_AI);

	for (int32 Idx = 0; Idx < CombinedGrids.Num(); Idx++)
	{
		if (CombinedGrids[Idx].WorldBounds.IsInside(WorldLocation))
		{
			return Idx;
		}
	}

	return INDEX_NONE;
}

uint8 UNavLocalGridManager::GetGridValueAt(const FVector& WorldLocation) const
{
	const int32 GridIndex = GetGridIndex(WorldLocation);
	return (GridIndex != INDEX_NONE) ? CombinedGrids[GridIndex].GetCellAtWorldLocation(WorldLocation) : 0;
}

bool UNavLocalGridManager::FindPath(const FVector& Start, const FVector& End, TArray<FVector>& PathPoints) const
{
	const int32 StartGridIdx = GetGridIndex(Start);
	const int32 EndGridIdx = GetGridIndex(End);

	if (StartGridIdx == EndGridIdx && StartGridIdx != INDEX_NONE)
	{
		const FNavLocalGridData& GridData = CombinedGrids[StartGridIdx];
		const FIntVector StartCoords = GridData.GetCellCoords(Start);
		const FIntVector EndCoords = GridData.GetCellCoords(End);
		TArray<FIntVector> PathCoords;

		const bool bHasPath = CombinedGrids[StartGridIdx].FindPath(StartCoords, EndCoords, PathCoords);
		if (bHasPath)
		{
			PathPoints.SetNum(PathCoords.Num());
			for (int32 Idx = 0; Idx < PathCoords.Num(); Idx++)
			{
				PathPoints[Idx] = GridData.GetProjectedCellCenter(PathCoords[Idx].X, PathCoords[Idx].Y);
			}

			return true;
		}
	}

	return false;
}

int32 UNavLocalGridManager::AddLocalNavigationGridForPoint(UObject* WorldContextObject, const FVector& Location, const int32 Radius2D, const float Height, bool bRebuildGrids)
{
	int32 GridId = 0;

	UNavLocalGridManager* GridManager = UNavLocalGridManager::GetCurrent(WorldContextObject);
	UE_CVLOG_UELOG(Radius2D <= 0, WorldContextObject, LogNavigation, Warning, TEXT("%hs must be called with a positive non-zero Radius2D but received %d."), __FUNCTION__, Radius2D);
	if (GridManager && Radius2D > 0)
	{
		FNavLocalGridData GridData(Location, UNavLocalGridManager::GridCellSize * Radius2D);
		GridData.SetHeight(Height);
		GridData.MarkPointObstacle(Location);

		GridId = GridManager->AddGridData(GridData, bRebuildGrids);
	}

	return GridId;
}

int32 UNavLocalGridManager::AddLocalNavigationGridForPoints(UObject* WorldContextObject, const TArray<FVector>& Locations, const int32 Radius2D, const float Height, bool bRebuildGrids)
{
	int32 GridId = 0;

	UNavLocalGridManager* GridManager = UNavLocalGridManager::GetCurrent(WorldContextObject);
	UE_CVLOG_UELOG(Locations.IsEmpty(), WorldContextObject, LogNavigation, Warning, TEXT("%hs must be called with a non-empty list of locations."), __FUNCTION__);
	UE_CVLOG_UELOG(Radius2D <= 0, WorldContextObject, LogNavigation, Warning, TEXT("%hs must be called with a positive non-zero Radius2D but received %d."), __FUNCTION__, Radius2D);
	if (GridManager && Locations.Num() > 0 && Radius2D > 0)
	{
		const FBox Bounds(Locations);
		const float BoundsSize2D = FloatCastChecked<float>(FMath::Max(Bounds.Max.X - Bounds.Min.X, Bounds.Max.Y - Bounds.Min.Y), UE::LWC::DefaultFloatPrecision);

		FNavLocalGridData GridData(Bounds.GetCenter(), (UNavLocalGridManager::GridCellSize * Radius2D) + BoundsSize2D);
		GridData.SetHeight(Height);

		for (int32 Idx = 0; Idx < Locations.Num(); Idx++)
		{
			GridData.MarkPointObstacle(Locations[Idx]);
		}

		GridId = GridManager->AddGridData(GridData, bRebuildGrids);
	}

	return GridId;
}

int32 UNavLocalGridManager::AddLocalNavigationGridForBox(UObject* WorldContextObject, const FVector& Location, FVector Extent, FRotator Rotation, const int32 Radius2D, const float Height, bool bRebuildGrids)
{
	int32 GridId = 0;

	UNavLocalGridManager* GridManager = UNavLocalGridManager::GetCurrent(WorldContextObject);
	UE_CVLOG_UELOG(Radius2D <= 0, WorldContextObject, LogNavigation, Warning, TEXT("%hs must be called with a positive non-zero Radius2D but received %d."), __FUNCTION__, Radius2D);
	if (GridManager && Radius2D > 0)
	{
		FNavLocalGridData GridData(Location, FVector2D(Extent.X + UNavLocalGridManager::GridCellSize * Radius2D, Extent.Y + UNavLocalGridManager::GridCellSize * Radius2D));
		GridData.SetHeight(FloatCastChecked<float>(Height + Extent.Z, UE::LWC::DefaultFloatPrecision));
		GridData.MarkBoxObstacle(Location, Extent, Rotation.Quaternion());

		GridId = GridManager->AddGridData(GridData, bRebuildGrids);
	}

	return GridId;
}

int32 UNavLocalGridManager::AddLocalNavigationGridForCapsule(UObject* WorldContextObject, const FVector& Location, float CapsuleRadius, float CapsuleHalfHeight, const int32 Radius2D, const float Height, bool bRebuildGrids)
{
	int32 GridId = 0;

	UNavLocalGridManager* GridManager = UNavLocalGridManager::GetCurrent(WorldContextObject);
	UE_CVLOG_UELOG(Radius2D <= 0, WorldContextObject, LogNavigation, Warning, TEXT("%hs must be called with a positive non-zero Radius2 Dbut received %d."), __FUNCTION__, Radius2D);
	if (GridManager && Radius2D > 0)
	{
		FNavLocalGridData GridData(Location, FVector2D(CapsuleRadius + UNavLocalGridManager::GridCellSize * Radius2D, CapsuleRadius + UNavLocalGridManager::GridCellSize * Radius2D));
		GridData.SetHeight(Height + CapsuleHalfHeight);
		GridData.MarkCapsuleObstacle(Location, CapsuleRadius, CapsuleHalfHeight);

		GridId = GridManager->AddGridData(GridData, bRebuildGrids);
	}

	return GridId;
}

void UNavLocalGridManager::RemoveLocalNavigationGrid(UObject* WorldContextObject, int32 GridId, bool bRebuildGrids)
{
	UNavLocalGridManager* GridManager = UNavLocalGridManager::GetCurrent(WorldContextObject);
	if (GridManager)
	{
		GridManager->RemoveGridData(GridId, bRebuildGrids);
	}
}

bool UNavLocalGridManager::FindLocalNavigationGridPath(UObject* WorldContextObject, const FVector& Start, const FVector& End, TArray<FVector>& PathPoints)
{
	UNavLocalGridManager* GridManager = UNavLocalGridManager::GetCurrent(WorldContextObject);
	if (GridManager)
	{
		return GridManager->FindPath(Start, End, PathPoints);
	}

	return false;
}

bool UNavLocalGridManager::SetLocalNavigationGridDensity(UObject* WorldContextObject, float CellSize)
{
	UNavLocalGridManager* GridManager = UNavLocalGridManager::GetCurrent(WorldContextObject);
	if (GridManager)
	{
		return GridManager->SetCellSize(CellSize);
	}

	return false;
}

UNavLocalGridManager* UNavLocalGridManager::GetCurrent(UWorld* World)
{
	UAISystem* AISys = World ? UAISystem::GetCurrent(*World) : nullptr;
	return AISys ? AISys->GetNavLocalGridManager() : nullptr;
}

UNavLocalGridManager* UNavLocalGridManager::GetCurrent(const UObject* WorldContextObject)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	UAISystem* AISys = World ? UAISystem::GetCurrent(*World) : nullptr;
	return AISys ? AISys->GetNavLocalGridManager() : nullptr;
}

#if WITH_ENGINE
UWorld* UNavLocalGridManager::GetWorld() const
{
	return GetOuter()->GetWorld();
}
#endif // WITH_ENGINE

