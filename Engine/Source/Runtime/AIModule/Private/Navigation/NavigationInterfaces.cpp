// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Navigation/CrowdAgentInterface.h"

UCrowdAgentInterface::UCrowdAgentInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
