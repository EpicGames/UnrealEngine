// Copyright Epic Games, Inc. All Rights Reserved.

#include "Navigation/PathFollowingComponent.h"

#include "AbstractNavData.h"
#include "AIConfig.h"
#include "AIController.h"
#include "AISystem.h"
#include "BrainComponent.h"
#include "Engine/Canvas.h"
#include "GameFramework/Controller.h"
#include "GameFramework/Pawn.h"
#include "NavFilters/NavigationQueryFilter.h"
#include "Navigation/MetaNavMeshPath.h"
#include "NavigationSystem.h"
#include "NavLinkCustomInterface.h"
#include "NavMesh/RecastNavMesh.h"
#include "TimerManager.h"
#include "UObject/Package.h"
#include "VisualLogger/VisualLogger.h"
#include "VisualLogger/VisualLoggerTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PathFollowingComponent)

#if UE_BUILD_TEST || UE_BUILD_SHIPPING
#define SHIPPING_STATIC static
#else
#define SHIPPING_STATIC
#endif // UE_BUILD_TEST || UE_BUILD_SHIPPING

DEFINE_LOG_CATEGORY(LogPathFollowing);

namespace UE::Navigation::Private
{
	FORCEINLINE FVector FindGoalLocation(const UPathFollowingComponent& Component, const AActor& GoalActor, const INavAgentInterface* GoalNavAgent, float& GoalRadius, float& GoalHalfHeight)
	{
		if (GoalNavAgent)
		{
			const AActor* OwnerActor = Component.GetOwner();
			FVector GoalOffset;
			GoalNavAgent->GetMoveGoalReachTest(OwnerActor, Component.GetMoveGoalLocationOffset(), GoalOffset, GoalRadius, GoalHalfHeight);

			return FQuatRotationTranslationMatrix(GoalActor.GetActorQuat(), GoalNavAgent->GetNavAgentLocation()).TransformPosition(GoalOffset);
		}
		else
		{
			return GoalActor.GetActorLocation();
		}
	}
}

FPathFollowingResult::FPathFollowingResult(uint16 InFlags) : Flags(InFlags)
{
	Code =
		HasFlag(FPathFollowingResultFlags::Success) && !HasFlag(FPathFollowingResultFlags::Blocked | FPathFollowingResultFlags::OffPath | FPathFollowingResultFlags::UserAbort) ? EPathFollowingResult::Success :
		HasFlag(FPathFollowingResultFlags::Blocked) && !HasFlag(FPathFollowingResultFlags::Success | FPathFollowingResultFlags::OffPath | FPathFollowingResultFlags::UserAbort) ? EPathFollowingResult::Blocked :
		HasFlag(FPathFollowingResultFlags::OffPath) && !HasFlag(FPathFollowingResultFlags::Success | FPathFollowingResultFlags::Blocked | FPathFollowingResultFlags::UserAbort) ? EPathFollowingResult::OffPath :
		HasFlag(FPathFollowingResultFlags::UserAbort) && !HasFlag(FPathFollowingResultFlags::Success | FPathFollowingResultFlags::Blocked | FPathFollowingResultFlags::OffPath) ? EPathFollowingResult::Aborted :
		EPathFollowingResult::Invalid;
}

FPathFollowingResult::FPathFollowingResult(EPathFollowingResult::Type ResultCode, uint16 ExtraFlags) : Code(ResultCode)
{
	static uint16 CodeLookup[6] = {
		FPathFollowingResultFlags::Success,			// EPathFollowingResult::Success
		FPathFollowingResultFlags::Blocked,			// EPathFollowingResult::Blocked
		FPathFollowingResultFlags::OffPath,			// EPathFollowingResult::OffPath
		FPathFollowingResultFlags::UserAbort,		// EPathFollowingResult::Aborted
		0,											// EPathFollowingResult::Skipped_DEPRECATED
		0,											// EPathFollowingResult::Invalid
	};

	Flags = CodeLookup[ResultCode] | ExtraFlags;
}

FString FPathFollowingResult::ToString() const
{
	static UEnum* ResultEnum = StaticEnum<EPathFollowingResult::Type>();
	return FString::Printf(TEXT("%s[%s]"), ResultEnum ? *ResultEnum->GetNameStringByValue(Code) : TEXT("??"), *FPathFollowingResultFlags::ToString(Flags));
}

FString FPathFollowingResultFlags::ToString(uint16 Value)
{
	static FString FlagDesc[] = {
		GET_FUNCTION_NAME_STRING_CHECKED(FPathFollowingResultFlags, Success),
		GET_FUNCTION_NAME_STRING_CHECKED(FPathFollowingResultFlags, Blocked),
		GET_FUNCTION_NAME_STRING_CHECKED(FPathFollowingResultFlags, OffPath),
		GET_FUNCTION_NAME_STRING_CHECKED(FPathFollowingResultFlags, UserAbort),
		GET_FUNCTION_NAME_STRING_CHECKED(FPathFollowingResultFlags, OwnerFinished),
		GET_FUNCTION_NAME_STRING_CHECKED(FPathFollowingResultFlags, InvalidPath),
		GET_FUNCTION_NAME_STRING_CHECKED(FPathFollowingResultFlags, MovementStop),
		GET_FUNCTION_NAME_STRING_CHECKED(FPathFollowingResultFlags, NewRequest),
		GET_FUNCTION_NAME_STRING_CHECKED(FPathFollowingResultFlags, ForcedScript),
		GET_FUNCTION_NAME_STRING_CHECKED(FPathFollowingResultFlags, AlreadyAtGoal),
	};

	FString CombinedDesc;
	for (int32 Idx = 0; Idx < FirstGameplayFlagShift; Idx++)
	{
		if (Value & (1 << Idx))
		{
			CombinedDesc += FlagDesc[Idx];
			CombinedDesc += TEXT(' ');
		}
	}

	Value = Value >> FirstGameplayFlagShift;
	for (int32 Idx = 0; Value; Idx++, Value /= 2)
	{
		if (Value & 1)
		{
			CombinedDesc += TEXT("Game");
			CombinedDesc += TTypeToString<int32>::ToString(Idx);
			CombinedDesc += TEXT(' ');
		}
	}

	return CombinedDesc.LeftChop(1);
}

//----------------------------------------------------------------------//
// Life cycle                                                        
//----------------------------------------------------------------------//
uint32 UPathFollowingComponent::NextRequestId = 1;
const float UPathFollowingComponent::DefaultAcceptanceRadius = -1.f;

UPathFollowingComponent::UPathFollowingComponent(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;

	// Activation/Ticking is controlled by Status if this instance is of type UPathFollowingComponent (not a derived class)
	bTickComponentOnlyWhenMoving = StaticClass() == GetClass();
	PrimaryComponentTick.bStartWithTickEnabled = !bTickComponentOnlyWhenMoving;
	bAutoActivate = !bTickComponentOnlyWhenMoving;

	MinAgentRadiusPct = 1.1f;
	MinAgentHalfHeightPct = 1.05f;
	BlockDetectionDistance = 10.0f;
	BlockDetectionInterval = 0.5f;
	BlockDetectionSampleCount = 10;
	WaitingTimeout = 1.0f;
	LastSampleTime = 0.0f;
	NextSampleIdx = 0;
	bUseBlockDetection = true;
	bLastMoveReachedGoal = false;
	bStopMovementOnFinish = true;
	bIsUsingMetaPath = false;

	MoveSegmentStartIndex = 0;
	MoveSegmentEndIndex = 1;
	MoveSegmentStartRef = INVALID_NAVNODEREF;
	MoveSegmentEndRef = INVALID_NAVNODEREF;
	bMoveSegmentIsUsingCustomLinkReachCondition = false;

	CachedBrakingDistance = 100.0f;
	CachedBrakingMaxSpeed = 0.0f;
	DecelerationSegmentIndex = INDEX_NONE;

	bReachTestIncludesAgentRadius = true;
	bReachTestIncludesGoalRadius = true;
	bMoveToGoalOnLastSegment = true;
	bMoveToGoalClampedToNavigation = false;

	Status = EPathFollowingStatus::Idle;

	CurrentMoveInput = FVector::ZeroVector;
#if !UE_BUILD_SHIPPING
	DEBUG_bMovingDirectlyToGoal = false;
#endif // !UE_BUILD_SHIPPING
}

void UPathFollowingComponent::SetStatus(const EPathFollowingStatus::Type InStatus)
{
	Status = InStatus;

	if (bTickComponentOnlyWhenMoving)
	{
		if (Status == EPathFollowingStatus::Moving)
		{
			if (!IsActive())
			{
				Activate();
			}
		}
		else
		{
			if (IsActive())
			{
				Deactivate();
			}
		}
	}
}

void UPathFollowingComponent::LogPathHelper(const AActor* LogOwner, FNavigationPath* InLogPath, const AActor* LogGoalActor)
{
#if ENABLE_VISUAL_LOG
	FVisualLogger& Vlog = FVisualLogger::Get();
	if (Vlog.IsRecording() &&
		InLogPath && InLogPath->IsValid() && InLogPath->GetPathPoints().Num())
	{
		const FVector PathEnd = *InLogPath->GetPathPointLocation(InLogPath->GetPathPoints().Num() - 1);

		if (FVisualLogEntry* Entry = FVisualLogger::GetEntryToWrite(LogOwner, LogPathFollowing))
		{
			InLogPath->DescribeSelfToVisLog(Entry);
			if (LogGoalActor)
			{
				const FVector GoalLoc = LogGoalActor->GetActorLocation();
				if (FVector::DistSquared(GoalLoc, PathEnd) > 1.0f)
				{
					UE_VLOG_LOCATION(LogOwner, LogPathFollowing, Verbose, GoalLoc, 30, FColor::Green, TEXT("GoalActor"));
					UE_VLOG_SEGMENT(LogOwner, LogPathFollowing, Verbose, GoalLoc, PathEnd, FColor::Green, TEXT_EMPTY);
				}
			}
		}

		UE_VLOG_BOX(LogOwner, LogPathFollowing, Verbose, FBox(PathEnd - FVector(30.0f), PathEnd + FVector(30.0f)), FColor::Green, TEXT("PathEnd"));
	}
#endif // ENABLE_VISUAL_LOG
}

void UPathFollowingComponent::LogPathHelper(const AActor* LogOwner, FNavPathSharedPtr InLogPath, const AActor* LogGoalActor)
{
#if ENABLE_VISUAL_LOG
	if (InLogPath.IsValid())
	{
		LogPathHelper(LogOwner, InLogPath.Get(), LogGoalActor);
	}
#endif // ENABLE_VISUAL_LOG
}

void LogBlockHelper(AActor* LogOwner, INavMovementInterface* NavMoveInterface, float RadiusPct, float HeightPct, const FVector& SegmentStart, const FVector& SegmentEnd)
{
#if ENABLE_VISUAL_LOG
	if (NavMoveInterface && LogOwner)
	{
		const FVector AgentLocation = NavMoveInterface->GetFeetLocation();
		const FVector ToTarget = (SegmentEnd - AgentLocation);
		const FVector::FReal SegmentDot = FVector::DotProduct(ToTarget.GetSafeNormal(), (SegmentEnd - SegmentStart).GetSafeNormal());
		UE_VLOG(LogOwner, LogPathFollowing, Verbose, TEXT("[agent to segment end] dot [segment dir]: %f"), SegmentDot);
		
		float AgentRadius = 0.0f;
		float AgentHalfHeight = 0.0f;
		NavMoveInterface->GetSimpleCollisionCylinder(AgentRadius, AgentHalfHeight);

		const FVector::FReal Dist2D = ToTarget.Size2D();
		UE_VLOG(LogOwner, LogPathFollowing, Verbose, TEXT("dist 2d: %f (agent radius: %f [%f])"), Dist2D, AgentRadius, AgentRadius * (1 + RadiusPct));

		const FVector::FReal ZDiff = FMath::Abs(ToTarget.Z);
		UE_VLOG(LogOwner, LogPathFollowing, Verbose, TEXT("Z diff: %f (agent halfZ: %f [%f])"), ZDiff, AgentHalfHeight, AgentHalfHeight * (1 + HeightPct));
	}
#endif // ENABLE_VISUAL_LOG
}

FString GetPathDescHelper(FNavPathSharedPtr Path)
{
	return !Path.IsValid() ? TEXT("missing") :
		!Path->IsValid() ? TEXT("invalid") :
		FString::Printf(TEXT("%s:%d"), Path->IsPartial() ? TEXT("partial") : TEXT("complete"), Path->GetPathPoints().Num());
}

void UPathFollowingComponent::OnPathEvent(FNavigationPath* InPath, ENavPathEvent::Type Event)
{
	const static UEnum* NavPathEventEnum = StaticEnum<ENavPathEvent::Type>();
	UE_VLOG(GetOwner(), LogPathFollowing, Log, TEXT("OnPathEvent: %s"), *NavPathEventEnum->GetNameStringByValue(Event));

	if (InPath && Path.Get() == InPath)
	{
		switch (Event)
		{
		case ENavPathEvent::UpdatedDueToGoalMoved:
		case ENavPathEvent::UpdatedDueToNavigationChanged:
		case ENavPathEvent::MetaPathUpdate:
			{
				const bool bIsPathAccepted = HandlePathUpdateEvent();
				if (!bIsPathAccepted)
				{
					UE_VLOG(GetOwner(), LogPathFollowing, Log, TEXT(">> updated path was not accepcted, aborting move"));
					OnPathFinished(EPathFollowingResult::Aborted, FPathFollowingResultFlags::InvalidPath);
				}
				else if (InPath->IsPartial() && InPath->GetGoalActor())
				{
					float IgnoreGoalRadius, IgnoreGoalHalfHeight;
					OriginalMoveRequestGoalLocation = UE::Navigation::Private::FindGoalLocation(*this, *InPath->GetGoalActor(), InPath->GetGoalActorAsNavAgent(), IgnoreGoalRadius, IgnoreGoalHalfHeight);
				}
			}
			break;

		default:
			break;
		}
	}
}

bool UPathFollowingComponent::HandlePathUpdateEvent()
{
	UE_VLOG(GetOwner(), LogPathFollowing, Log, TEXT("HandlePathUpdateEvent Path(%s) Status(%s)"), *GetPathDescHelper(Path), *GetStatusDesc());
	LogPathHelper(GetOwner(), Path, DestinationActor.Get());

	if (Status == EPathFollowingStatus::Waiting && Path.IsValid() && !Path->IsValid())
	{
		UE_VLOG(GetOwner(), LogPathFollowing, Error, TEXT("Received unusable path in Waiting state! (ready:%d upToDate:%d pathPoints:%d)"),
			Path->IsReady() ? 1 : 0,
			Path->IsUpToDate() ? 1 : 0,
			Path->GetPathPoints().Num());
	}

	if (Status == EPathFollowingStatus::Idle || !Path.IsValid() || !Path->IsValid())
	{
		UE_VLOG(GetOwner(), LogPathFollowing, Log, TEXT(">> invalid request"));
		return false;
	}

	const AActor* PathGoalActor = Path->GetGoalActor();
	if (PathGoalActor)
	{
		float IgnoreGoalRadius, IgnoreGoalHalfHeight;
		OriginalMoveRequestGoalLocation = UE::Navigation::Private::FindGoalLocation(*this, *PathGoalActor, Path->GetGoalActorAsNavAgent(), IgnoreGoalRadius, IgnoreGoalHalfHeight);
	}
	if (FAISystem::IsValidLocation(OriginalMoveRequestGoalLocation))
	{
		PreciseAcceptanceRadiusCheckStartNodeIndex = FindPreciseAcceptanceRadiusTestsStartNodeIndex(*Path, OriginalMoveRequestGoalLocation);
	}

	OnPathUpdated();
	GetWorld()->GetTimerManager().ClearTimer(WaitingForPathTimer);
	UpdateDecelerationData();

	if (Status == EPathFollowingStatus::Waiting || Status == EPathFollowingStatus::Moving)
	{
		SetStatus(EPathFollowingStatus::Moving);

		const int32 CurrentSegment = DetermineStartingPathPoint(Path.Get());
		SetMoveSegment(CurrentSegment);
	}

	return true;
}

void UPathFollowingComponent::SetAcceptanceRadius(const float InAcceptanceRadius)
{
	AcceptanceRadius = InAcceptanceRadius;
	if (Path.IsValid() && Path->IsValid() && FAISystem::IsValidLocation(OriginalMoveRequestGoalLocation))
	{
		// figure out when we need to start doing precise end-condition tests
		PreciseAcceptanceRadiusCheckStartNodeIndex = FindPreciseAcceptanceRadiusTestsStartNodeIndex(*Path, OriginalMoveRequestGoalLocation);
	}
}

FAIRequestID UPathFollowingComponent::RequestMove(const FAIMoveRequest& RequestData, FNavPathSharedPtr InPath)
{
	UE_VLOG(GetOwner(), LogPathFollowing, Log, TEXT("RequestMove: Path(%s) %s"), *GetPathDescHelper(InPath), *RequestData.ToString());
	LogPathHelper(GetOwner(), InPath, RequestData.GetGoalActor());

	if (ResourceLock.IsLocked())
	{
		UE_VLOG(GetOwner(), LogPathFollowing, Log, TEXT("Rejecting move request due to resource lock by %s"), *ResourceLock.GetLockPriorityName());
		return FAIRequestID::InvalidRequest;
	}

	const float UseAcceptanceRadius = (RequestData.GetAcceptanceRadius() == UPathFollowingComponent::DefaultAcceptanceRadius) ? MyDefaultAcceptanceRadius : RequestData.GetAcceptanceRadius();
 	if (!InPath.IsValid() || UseAcceptanceRadius < 0.0f)
	{
		UE_VLOG(GetOwner(), LogPathFollowing, Log, TEXT("RequestMove: invalid request"));
		return FAIRequestID::InvalidRequest;
	}

	// try to grab movement component
	if (!UpdateMovementComponent())
	{
		UE_VLOG(GetOwner(), LogPathFollowing, Warning, TEXT("RequestMove: missing movement component"));
		return FAIRequestID::InvalidRequest;
	}

	FAIRequestID MoveId = CurrentRequestId;

	// abort previous movement
	if (Status == EPathFollowingStatus::Paused && Path.IsValid() && InPath.Get() == Path.Get() && DestinationActor == RequestData.GetGoalActor())
	{
		ResumeMove();
	}
	else
	{
		if (Status != EPathFollowingStatus::Idle)
		{
			// setting to false to make sure OnPathFinished won't result in stoping 
			bStopMovementOnFinish = false;
			OnPathFinished(EPathFollowingResult::Aborted, FPathFollowingResultFlags::NewRequest);
		}
		
		// force-setting this to true since this is the default state, and gets modified only 
		// when aborting movement.
		bStopMovementOnFinish = true;
		
		Reset();
		StoreRequestId();
		
		// store new Id, using CurrentRequestId directly at the end of function is not safe with chained moves
		MoveId = CurrentRequestId;

		// store new data
		Path = InPath;
		Path->AddObserver(FNavigationPath::FPathObserverDelegate::FDelegate::CreateUObject(this, &UPathFollowingComponent::OnPathEvent));
		if (NavMovementInterface.IsValid() && NavMovementInterface->GetOwnerAsObject())
		{
			if (AActor* OwnerAsActor = Cast<AActor>(NavMovementInterface->GetOwnerAsObject()))
			{
				Path->SetSourceActor(*(OwnerAsActor));
			}
		}

		OriginalMoveRequestGoalLocation = RequestData.GetGoalActor() ? RequestData.GetGoalActor()->GetActorLocation() : RequestData.GetGoalLocation();
		
		// update meta path data
		FMetaNavMeshPath* MetaNavPath = Path->CastPath<FMetaNavMeshPath>();
		if (MetaNavPath)
		{
			bIsUsingMetaPath = true;

			const FVector CurrentLocation = NavMovementInterface.IsValid() ? NavMovementInterface->GetFeetLocation() : FAISystem::InvalidLocation;
			MetaNavPath->Initialize(CurrentLocation);
		}

		NavigationFilter = ExtractNavigationFilter(RequestData, Path);

		PathTimeWhenPaused = 0.;
		OnPathUpdated();

		// make sure that OnPathUpdated didn't change current move request
		// otherwise there's no point in starting it
		if (CurrentRequestId == MoveId)
		{
			AcceptanceRadius = UseAcceptanceRadius;
			bReachTestIncludesAgentRadius = RequestData.IsReachTestIncludingAgentRadius();
			bReachTestIncludesGoalRadius = RequestData.IsReachTestIncludingGoalRadius();
			GameData = RequestData.GetUserData();
			SetDestinationActor(RequestData.GetGoalActor());
			SetAcceptanceRadius(UseAcceptanceRadius);
			UpdateDecelerationData();

#if ENABLE_VISUAL_LOG
			const FVector CurrentLocation = NavMovementInterface.IsValid() ? NavMovementInterface->GetFeetLocation() : FVector::ZeroVector;
			const FVector DestLocation = InPath->GetDestinationLocation();
			const FVector ToDest = DestLocation - CurrentLocation;
			UE_VLOG(GetOwner(), LogPathFollowing, Log, TEXT("RequestMove: accepted, ID(%u) dist2D(%.0f) distZ(%.0f)"), MoveId.GetID(), ToDest.Size2D(), FMath::Abs(ToDest.Z));
#endif // ENABLE_VISUAL_LOG

			// with async pathfinding paths can be incomplete, movement will start after receiving path event 
			if (!bIsUsingMetaPath && Path.IsValid() && Path->IsValid())
			{
				SetStatus(EPathFollowingStatus::Moving);

				// determine with path segment should be followed
				const uint32 CurrentSegment = DetermineStartingPathPoint(InPath.Get());
				SetMoveSegment(CurrentSegment);
			}
			else
			{
				SetStatus(EPathFollowingStatus::Waiting);
				GetWorld()->GetTimerManager().SetTimer(WaitingForPathTimer, this, &UPathFollowingComponent::OnWaitingPathTimeout, WaitingTimeout);
			}
		}
	}

	return MoveId;
}

void UPathFollowingComponent::AbortMove(const UObject& Instigator, FPathFollowingResultFlags::Type AbortFlags, FAIRequestID RequestID, EPathFollowingVelocityMode VelocityMode)
{
	UE_VLOG(GetOwner(), LogPathFollowing, Log, TEXT("AbortMove: RequestID(%u) Instigator(%s)"), RequestID.GetID(), *Instigator.GetName());

	if ((Status != EPathFollowingStatus::Idle) && RequestID.IsEquivalent(GetCurrentRequestId()))
	{
		bStopMovementOnFinish = (VelocityMode == EPathFollowingVelocityMode::Reset);
		OnPathFinished(FPathFollowingResult(EPathFollowingResult::Aborted, AbortFlags & FPathFollowingResultFlags::UserAbortFlagMask));
	}
}

FAIRequestID UPathFollowingComponent::RequestMoveWithImmediateFinish(EPathFollowingResult::Type Result, EPathFollowingVelocityMode VelocityMode)
{
	// make sure we stop (or not) depending on VelocityMode's value
	bStopMovementOnFinish = (VelocityMode == EPathFollowingVelocityMode::Reset);

	if (Status != EPathFollowingStatus::Idle)
	{
		OnPathFinished(EPathFollowingResult::Aborted, FPathFollowingResultFlags::NewRequest);
	}
	StoreRequestId();
	const FAIRequestID MoveId = CurrentRequestId;
	OnPathFinished(Result, FPathFollowingResultFlags::None);
	return MoveId;
}

void UPathFollowingComponent::PauseMove(FAIRequestID RequestID, EPathFollowingVelocityMode VelocityMode)
{
	UE_VLOG(GetOwner(), LogPathFollowing, Log, TEXT("PauseMove: RequestID(%u)"), RequestID.GetID());
	if (Status == EPathFollowingStatus::Paused)
	{
		return;
	}

	if (RequestID.IsEquivalent(GetCurrentRequestId()))
	{
		if ((VelocityMode == EPathFollowingVelocityMode::Reset) && NavMovementInterface.IsValid() && HasMovementAuthority())
		{
			NavMovementInterface->StopMovementKeepPathing();
		}

		LocationWhenPaused = NavMovementInterface.IsValid() ? NavMovementInterface->GetFeetLocation() : FVector::ZeroVector;
		PathTimeWhenPaused = Path.IsValid() ? Path->GetTimeStamp() : 0.;
		Status = EPathFollowingStatus::Paused;

		UpdateMoveFocus();
	}
}

void UPathFollowingComponent::ResumeMove(FAIRequestID RequestID)
{
	if (RequestID.IsEquivalent(CurrentRequestId) && RequestID.IsValid())
	{
		UE_VLOG(GetOwner(), LogPathFollowing, Log, TEXT("ResumeMove: RequestID(%u)"), RequestID.GetID());

		const bool bMovedDuringPause = ShouldCheckPathOnResume();
		const bool bIsOnPath = IsOnPath();
		if (bIsOnPath)
		{
			SetStatus(EPathFollowingStatus::Moving);

			const bool bWasPathUpdatedRecently = Path.IsValid() ? (Path->GetTimeStamp() > PathTimeWhenPaused) : false;
			if (bMovedDuringPause || bWasPathUpdatedRecently)
			{
				const int32 CurrentSegment = DetermineStartingPathPoint(Path.Get());
				SetMoveSegment(CurrentSegment);
			}
			else
			{
				UpdateMoveFocus();
			}
		}
		else if (Path.IsValid() && Path->IsValid() && Path->GetNavigationDataUsed() == NULL)
		{
			// this means it's a scripted path, just resume
			SetStatus(EPathFollowingStatus::Moving);
			UpdateMoveFocus();
		}
		else
		{
			OnPathFinished(EPathFollowingResult::OffPath, FPathFollowingResultFlags::None);
		}
	}
	else
	{
		UE_VLOG(GetOwner(), LogPathFollowing, Log, TEXT("ResumeMove: RequestID(%u) is neither \'AnyRequest\' not CurrentRequestId(%u). Ignoring."), RequestID.GetID(), CurrentRequestId.GetID());
	}
}

bool UPathFollowingComponent::ShouldCheckPathOnResume() const
{
	bool bCheckPath = true;
	if (NavMovementInterface != nullptr)
	{
		float AgentRadius = 0.0f, AgentHalfHeight = 0.0f;
		NavMovementInterface->GetSimpleCollisionCylinder(AgentRadius, AgentHalfHeight);

		const FVector CurrentLocation = NavMovementInterface->GetFeetLocation();
		const FVector::FReal DeltaMove2DSq = (CurrentLocation - LocationWhenPaused).SizeSquared2D();
		const FVector::FReal DeltaZ = FMath::Abs(CurrentLocation.Z - LocationWhenPaused.Z);
		if (DeltaMove2DSq < FMath::Square(AgentRadius) && DeltaZ < (AgentHalfHeight * 0.5))
		{
			bCheckPath = false;
		}
	}

	return bCheckPath;
}

void UPathFollowingComponent::OnPathFinished(const FPathFollowingResult& Result)
{
	UE_VLOG(GetOwner(), LogPathFollowing, Log, TEXT("OnPathFinished: %s"), *Result.ToString());
	
	INavLinkCustomInterface* CustomNavLink = Cast<INavLinkCustomInterface>(CurrentCustomLinkOb.Get());
	if (CustomNavLink)
	{
		CustomNavLink->OnLinkMoveFinished(this);
		CurrentCustomLinkOb.Reset();
	}
	
	// update meta path if needed
	if (bIsUsingMetaPath && Result.IsSuccess() && NavMovementInterface.IsValid())
	{
		FMetaNavMeshPath* MetaNavPath = Path->CastPath<FMetaNavMeshPath>();
		const bool bNeedPathUpdate = MetaNavPath && MetaNavPath->ConditionalMoveToNextSection(NavMovementInterface->GetFeetLocation(), EMetaPathUpdateReason::PathFinished);
		if (bNeedPathUpdate)
		{
			// keep move request active
			return;
		}
	}

	// save move status
	bLastMoveReachedGoal = Result.IsSuccess() && !HasPartialPath();

	// save data required for observers before reseting temporary variables
	const FAIRequestID FinishedMoveId = CurrentRequestId;

	Reset();
	UpdateMoveFocus();

	if (ShouldStopMovementOnPathFinished())
	{
		NavMovementInterface->StopMovementKeepPathing();
	}

	// notify observers after state was reset (they can request another move)
	OnRequestFinished.Broadcast(FinishedMoveId, Result);

	FAIMessage Msg(UBrainComponent::AIMessage_MoveFinished, this, FinishedMoveId, Result.IsSuccess());
	Msg.SetFlag(Result.Flags & 0xff);
	FAIMessage::Send(Cast<AController>(GetOwner()), Msg);
}

bool UPathFollowingComponent::ShouldStopMovementOnPathFinished() const
{
	return bStopMovementOnFinish && NavMovementInterface.IsValid() && HasMovementAuthority() && !NavMovementInterface->UseAccelerationForPathFollowing();
}

void UPathFollowingComponent::OnSegmentFinished()
{
	UE_VLOG(GetOwner(), LogPathFollowing, Verbose, TEXT("OnSegmentFinished"));
}

void UPathFollowingComponent::OnPathUpdated()
{
}

void UPathFollowingComponent::SetMovementComponent(UNavMovementComponent* MoveComp)
{
	SetNavMovementInterface(Cast<INavMovementInterface>(MoveComp));
}

void UPathFollowingComponent::Initialize()
{
	UpdateCachedComponents();
	LocationSamples.Reserve(BlockDetectionSampleCount);
}

void UPathFollowingComponent::Cleanup()
{
	SetNavMovementInterface(nullptr);
	Reset();
}

void UPathFollowingComponent::UpdateCachedComponents()
{
	UpdateMovementComponent(/*bForce=*/true);
}

void UPathFollowingComponent::OnNewPawn(APawn* NewPawn)
{
	UpdateCachedComponents();
}

void UPathFollowingComponent::OnRegister()
{
	Super::OnRegister();
	AController* ControllerOwner = Cast<AController>(GetOwner());

	if (ControllerOwner)
	{
		ControllerOwner->GetOnNewPawnNotifier().AddUObject(this, &UPathFollowingComponent::OnNewPawn);
	}
}

void UPathFollowingComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if (Status == EPathFollowingStatus::Moving)
	{
		// check finish conditions, update current segment if needed
		UpdatePathSegment();
	}

	if (Status == EPathFollowingStatus::Moving)
	{
		// follow current path segment
		FollowPathSegment(DeltaTime);
	}
};

void UPathFollowingComponent::SetNavMovementInterface(INavMovementInterface* NavMoveInterface)
{
	if (!NavMoveInterface && NavMovementInterface.IsValid() && NavMovementInterface->GetPathFollowingAgent() == this)
	{
		NavMovementInterface->SetPathFollowingAgent(nullptr);
	}
	
	NavMovementInterface = NavMoveInterface;
	MyNavData = nullptr;

	if (NavMovementInterface.IsValid())
	{
		const FNavAgentProperties& NavAgentProps = NavMovementInterface->GetNavAgentPropertiesRef();
		MyDefaultAcceptanceRadius = NavAgentProps.AgentRadius * 0.1f;
		NavMovementInterface->SetPathFollowingAgent(Cast<IPathFollowingAgentInterface>(this));

		UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
		
		if (NavSys)
		{
			MyNavData = NavSys->GetNavDataForProps(NavAgentProps, NavMovementInterface->GetLocation());
			if (MyNavData == nullptr)
			{
				if (NavSys->IsInitialized() == false)
				{
					NavSys->OnNavigationInitDone.AddUObject(this, &UPathFollowingComponent::OnNavigationInitDone);
				}
				else
				{
					NavSys->OnNavDataRegisteredEvent.AddUniqueDynamic(this, &UPathFollowingComponent::OnNavDataRegistered);
				}
			}
		}
	}
}

void UPathFollowingComponent::OnNavigationInitDone()
{
	UWorld* MyWorld = GetWorld();
	if (NavMovementInterface.IsValid() && MyWorld)
	{
		UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(MyWorld);
		check(NavSys);
		const FNavAgentProperties& NavAgentProps = NavMovementInterface->GetNavAgentPropertiesRef();
		MyNavData = NavSys->GetNavDataForProps(NavAgentProps, NavMovementInterface->GetLocation());
		NavSys->OnNavigationInitDone.RemoveAll(this);
	}
}

void UPathFollowingComponent::OnNavDataRegistered(ANavigationData* NavData)
{
	if (NavData && NavMovementInterface.IsValid())
	{
		const FNavAgentProperties& NavAgentProps = NavMovementInterface->GetNavAgentPropertiesRef();
		if (NavData->DoesSupportAgent(NavAgentProps))
		{
			MyNavData = NavData;
			UWorld* MyWorld = GetWorld();
			UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(MyWorld);
			check(NavSys);
			NavSys->OnNavDataRegisteredEvent.RemoveAll(this);
		}
	}
}

FVector UPathFollowingComponent::GetMoveFocus(bool bAllowStrafe) const
{
	FVector MoveFocus = FVector::ZeroVector;
	if (bAllowStrafe && DestinationActor.IsValid())
	{
		MoveFocus = DestinationActor->GetActorLocation();
	}
	else
	{
		const FVector CurrentMoveDirection = GetCurrentDirection();
		MoveFocus = *CurrentDestination + (CurrentMoveDirection * FAIConfig::Navigation::FocalPointDistance);
	}

	return MoveFocus;
}

FSharedConstNavQueryFilter UPathFollowingComponent::ExtractNavigationFilter(const FAIMoveRequest& RequestData, FNavPathSharedPtr InPath) const
{
	if (InPath.IsValid() && !InPath->CastPath<FAbstractNavigationPath>() && InPath->GetFilter() != nullptr)
	{
		return InPath->GetFilter();
	}

	if (MyNavData)
	{
		if (RequestData.GetNavigationFilter() != nullptr)
		{
			return UNavigationQueryFilter::GetQueryFilter(*MyNavData, GetOwner(), RequestData.GetNavigationFilter());
		}

		if (const AAIController* AIOwner = Cast<AAIController>(GetOwner()))
		{
			if (AIOwner->GetDefaultNavigationFilterClass() != nullptr)
			{
				return UNavigationQueryFilter::GetQueryFilter(*MyNavData, AIOwner->GetDefaultNavigationFilterClass());
			}
		}

		return MyNavData->GetDefaultQueryFilter();
	}

	return FSharedConstNavQueryFilter();
}

void UPathFollowingComponent::Reset()
{
	MoveSegmentStartIndex = 0;
	MoveSegmentStartRef = INVALID_NAVNODEREF;
	MoveSegmentEndRef = INVALID_NAVNODEREF;
	DecelerationSegmentIndex = INDEX_NONE;
	ResetMoveSegmentCustomLinkCache();

	LocationSamples.Reset();
	LastSampleTime = 0.0f;
	NextSampleIdx = 0;

	Path.Reset();
	GameData.Reset();
	DestinationActor.Reset();
	CurrentDestination.Clear();
	AcceptanceRadius = MyDefaultAcceptanceRadius;
	bReachTestIncludesAgentRadius = true;
	bReachTestIncludesGoalRadius = true;
	bCollidedWithGoal = false;
	bIsUsingMetaPath = false;
	bWalkingNavLinkStart = false;
	PreciseAcceptanceRadiusCheckStartNodeIndex = INDEX_NONE;
	OriginalMoveRequestGoalLocation = FAISystem::InvalidLocation;

	CurrentRequestId = FAIRequestID::InvalidRequest;
	SetStatus(EPathFollowingStatus::Idle);

	if (WaitingForPathTimer.IsValid())
	{
		GetWorld()->GetTimerManager().ClearTimer(WaitingForPathTimer);
		WaitingForPathTimer.Invalidate();
	}
}

int32 UPathFollowingComponent::DetermineStartingPathPoint(const FNavigationPath* ConsideredPath) const
{
	int32 PickedPathPoint = INDEX_NONE;

	if (ConsideredPath && ConsideredPath->IsValid())
	{
		// if we already have some info on where we were on previous path
		// we can find out if there's a segment on new path we're currently at
		if (MoveSegmentStartRef != INVALID_NAVNODEREF &&
			MoveSegmentEndRef != INVALID_NAVNODEREF &&
			ConsideredPath->GetNavigationDataUsed() != NULL)
		{
			// iterate every new path node and see if segment match
			for (int32 PathPoint = 0; PathPoint < ConsideredPath->GetPathPoints().Num() - 1; ++PathPoint)
			{
				if (ConsideredPath->GetPathPoints()[PathPoint].NodeRef == MoveSegmentStartRef &&
					ConsideredPath->GetPathPoints()[PathPoint + 1].NodeRef == MoveSegmentEndRef)
				{
					PickedPathPoint = PathPoint;
					break;
				}
			}
		}

		if (NavMovementInterface.IsValid() && PickedPathPoint == INDEX_NONE)
		{
			if (ConsideredPath->GetPathPoints().Num() > 2)
			{
				// check if is closer to first or second path point (don't assume AI's standing)
				const FVector CurrentLocation = NavMovementInterface->GetFeetLocation();
				const FVector PathPt0 = *ConsideredPath->GetPathPointLocation(0);
				const FVector PathPt1 = *ConsideredPath->GetPathPointLocation(1);
				// making this test in 2d to avoid situation where agent's Z location not being in "navmesh plane"
				// would influence the result
				const FVector::FReal SqDistToFirstPoint = (CurrentLocation - PathPt0).SizeSquared2D();
				const FVector::FReal SqDistToSecondPoint = (CurrentLocation - PathPt1).SizeSquared2D();

				PickedPathPoint = FMath::IsNearlyEqual(SqDistToFirstPoint, SqDistToSecondPoint) ?
					((FMath::Abs(CurrentLocation.Z - PathPt0.Z) < FMath::Abs(CurrentLocation.Z - PathPt1.Z)) ? 0 : 1) :
					((SqDistToFirstPoint < SqDistToSecondPoint) ? 0 : 1);
			}
			else
			{
				// If there are only two point we probably should start from the beginning
				PickedPathPoint = 0;
			}
		}
	}

	return PickedPathPoint;
}

void UPathFollowingComponent::SetDestinationActor(const AActor* InDestinationActor)
{
	DestinationActor = MakeWeakObjectPtr(const_cast<AActor*>(InDestinationActor));
	DestinationAgent = Cast<const INavAgentInterface>(InDestinationActor);
	
	const AActor* OwnerActor = GetOwner();
	MoveOffset = DestinationAgent ? DestinationAgent->GetMoveGoalOffset(OwnerActor) : FVector::ZeroVector;
}

void UPathFollowingComponent::SetMoveSegment(int32 SegmentStartIndex)
{
	SHIPPING_STATIC	const float PathPointAcceptanceRadius = GET_AI_CONFIG_VAR(PathfollowingRegularPathPointAcceptanceRadius);
	SHIPPING_STATIC const float NavLinkAcceptanceRadius = GET_AI_CONFIG_VAR(PathfollowingNavLinkAcceptanceRadius);

	int32 EndSegmentIndex = SegmentStartIndex + 1;
	const FNavigationPath* PathInstance = Path.Get();
	if (PathInstance != nullptr && PathInstance->GetPathPoints().IsValidIndex(SegmentStartIndex) && PathInstance->GetPathPoints().IsValidIndex(EndSegmentIndex))
	{
		EndSegmentIndex = DetermineCurrentTargetPathPoint(SegmentStartIndex);

		MoveSegmentStartIndex = SegmentStartIndex;
		MoveSegmentEndIndex = EndSegmentIndex;
		const FNavPathPoint& PathPt0 = PathInstance->GetPathPoints()[MoveSegmentStartIndex];
		const FNavPathPoint& PathPt1 = PathInstance->GetPathPoints()[MoveSegmentEndIndex];

		MoveSegmentStartRef = PathPt0.NodeRef;
		MoveSegmentEndRef = PathPt1.NodeRef;
		
		CurrentDestination = PathInstance->GetPathPointLocation(MoveSegmentEndIndex);
		const FVector SegmentStart = *PathInstance->GetPathPointLocation(MoveSegmentStartIndex);
		FVector SegmentEnd = *CurrentDestination;

		// make sure we have a non-zero direction if still following a valid path
		if (SegmentStart.Equals(SegmentEnd) && PathInstance->GetPathPoints().IsValidIndex(MoveSegmentEndIndex + 1))
		{
			MoveSegmentEndIndex++;

			CurrentDestination = PathInstance->GetPathPointLocation(MoveSegmentEndIndex);
			SegmentEnd = *CurrentDestination;
		}

		// Check if the next segment is a custom link and has the need to use its own custom reach conditions
		if (PathPt1.CustomNavLinkId != FNavLinkId::Invalid)
		{
			const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
			if (const INavLinkCustomInterface* MoveSegmentCustomLink = NavSys->GetCustomLink(PathPt1.CustomNavLinkId))
			{
				// Cache the CustomLinkOb for faster access during the update
				MoveSegmentCustomLinkOb = Cast<const UObject>(MoveSegmentCustomLink);
				bMoveSegmentIsUsingCustomLinkReachCondition = MoveSegmentCustomLink->IsLinkUsingCustomReachCondition(this);
			}
			else
			{
				ResetMoveSegmentCustomLinkCache();
			}
		}
		else
		{
			ResetMoveSegmentCustomLinkCache();
		}

		CurrentAcceptanceRadius = (PathInstance->GetPathPoints().Num() == (MoveSegmentEndIndex + 1))
			? GetFinalAcceptanceRadius(*PathInstance, OriginalMoveRequestGoalLocation)
			// pick appropriate value base on whether we're going to nav link or not
			: (FNavMeshNodeFlags(PathPt1.Flags).IsNavLink() == false ? PathPointAcceptanceRadius : NavLinkAcceptanceRadius);

		MoveSegmentDirection = (SegmentEnd - SegmentStart).GetSafeNormal();

		bWalkingNavLinkStart = FNavMeshNodeFlags(PathPt0.Flags).IsNavLink();

		// handle moving through custom nav links
		if (PathPt0.CustomNavLinkId != FNavLinkId::Invalid)
		{
			UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
			INavLinkCustomInterface* CustomNavLink = NavSys->GetCustomLink(PathPt0.CustomNavLinkId);
			StartUsingCustomLink(CustomNavLink, SegmentEnd);
		}

		// update move focus in owning AI
		UpdateMoveFocus();
		UpdateDecelerationData();
	}
}

int32 UPathFollowingComponent::DetermineCurrentTargetPathPoint(int32 StartIndex)
{
	return StartIndex + 1;
}

void UPathFollowingComponent::UpdatePathSegment()
{
#if !UE_BUILD_SHIPPING
	DEBUG_bMovingDirectlyToGoal = false;
#endif // !UE_BUILD_SHIPPING

	if ((Path.IsValid() == false) || !NavMovementInterface.IsValid())
	{
		UE_CVLOG(Path.IsValid() == false, this, LogPathFollowing, Log, TEXT("Aborting move due to not having a valid path object"));
		OnPathFinished(EPathFollowingResult::Aborted, FPathFollowingResultFlags::InvalidPath);
		return;
	}

	checkf(NavMovementInterface.IsValid() , TEXT("Temporary comment: if you hit this, check other threads to see what could stump the NavMovementInterface."));
	const FVector CurrentLocation = NavMovementInterface->GetFeetLocation();
	const bool bCanUpdateState = HasMovementAuthority();

	if (!Path->IsValid())
	{
		if (!Path->IsWaitingForRepath())
		{
			UE_VLOG(this, LogPathFollowing, Log, TEXT("Aborting move due to path being invalid and not waiting for repath"));
			OnPathFinished(EPathFollowingResult::Aborted, FPathFollowingResultFlags::InvalidPath);
			return;
		}
		else if (HasStartedNavLinkMove() && bCanUpdateState && Status == EPathFollowingStatus::Moving)
		{
			// pawn needs to get off navlink to unlock path updates (AAIController::ShouldPostponePathUpdates)
			if (HasReachedCurrentTarget(CurrentLocation))
			{
				OnSegmentFinished();
				SetNextMoveSegment();
			}
		}
		else
		{
			// continue with execution, if navigation is being rebuild constantly AI will get stuck with current waypoint
			// path updates should be still coming in, even though they get invalidated right away
			UE_VLOG(this, LogPathFollowing, Log, TEXT("Updating path points in invalid & pending path!"));
		}
	}

	FMetaNavMeshPath* MetaNavPath = bIsUsingMetaPath ? Path->CastPath<FMetaNavMeshPath>() : nullptr;

	/** it's possible that finishing this move request will result in another request
	 *	which won't be easily detectable from this function. This simple local
	 *	variable gives us this knowledge. */
	const FAIRequestID MoveRequestId = GetCurrentRequestId();

	// if agent has control over its movement, check finish conditions
	if (bCanUpdateState && Status == EPathFollowingStatus::Moving)
	{
		const int32 LastSegmentEndIndex = Path->GetPathPoints().Num() - 1;
		const bool bFollowingLastSegment = (MoveSegmentEndIndex >= LastSegmentEndIndex);
		const bool bLastPathChunk = (MetaNavPath == nullptr || MetaNavPath->IsLastSection());

		if (bCollidedWithGoal)
		{
			// check if collided with goal actor
			OnSegmentFinished();
			OnPathFinished(EPathFollowingResult::Success, FPathFollowingResultFlags::None);
		}
		else if (MoveSegmentEndIndex > PreciseAcceptanceRadiusCheckStartNodeIndex && HasReachedDestination(CurrentLocation))
		{
			OnSegmentFinished();
			OnPathFinished(EPathFollowingResult::Success, FPathFollowingResultFlags::None);
		}
		else if (bFollowingLastSegment && bMoveToGoalOnLastSegment && bLastPathChunk)
		{
			// use goal actor for end of last path segment
			// UNLESS it's partial path (can't reach goal)
			if (DestinationActor.IsValid() && Path->IsPartial() == false)
			{
				const FVector AgentLocation = DestinationAgent ? DestinationAgent->GetNavAgentLocation() : DestinationActor->GetActorLocation();
				FVector GoalLocation = FQuatRotationTranslationMatrix(DestinationActor->GetActorQuat(), AgentLocation).TransformPosition(MoveOffset);

				if (bMoveToGoalClampedToNavigation && NavigationFilter)
				{
					FVector HitLocation;
					if (MyNavData->Raycast(CurrentLocation, GoalLocation, HitLocation, NavigationFilter))
					{
						GoalLocation = HitLocation;
					}
				}

				CurrentDestination.Set(NULL, GoalLocation);

				UE_VLOG(this, LogPathFollowing, Log, TEXT("Moving directly to move goal rather than following last path segment"));
				UE_VLOG_LOCATION(this, LogPathFollowing, VeryVerbose, GoalLocation, 30, FColor::Green, TEXT("Last-segment-to-actor"));
				UE_VLOG_SEGMENT(this, LogPathFollowing, VeryVerbose, CurrentLocation, GoalLocation, FColor::Green, TEXT_EMPTY);
			}

			UpdateMoveFocus();

#if !UE_BUILD_SHIPPING
			DEBUG_bMovingDirectlyToGoal = true;
#endif // !UE_BUILD_SHIPPING
		}
		// check if current move segment is finished
		else if (HasReachedCurrentTarget(CurrentLocation))
		{
			OnSegmentFinished();
			SetNextMoveSegment();
		}
	}

	if (bCanUpdateState 
		&& Status == EPathFollowingStatus::Moving
		// still the same move request
		&& MoveRequestId == GetCurrentRequestId())
	{
		// check waypoint switch condition in meta paths
		if (MetaNavPath)
		{
			MetaNavPath->ConditionalMoveToNextSection(CurrentLocation, EMetaPathUpdateReason::MoveTick);
		}

		// gather location samples to detect if moving agent is blocked
		const bool bHasNewSample = UpdateBlockDetection();
		if (bHasNewSample && IsBlocked())
		{
			if (Path->GetPathPoints().IsValidIndex(MoveSegmentEndIndex) && Path->GetPathPoints().IsValidIndex(MoveSegmentStartIndex))
			{
				LogBlockHelper(GetOwner(), NavMovementInterface.Get(), MinAgentRadiusPct, MinAgentHalfHeightPct,
					*Path->GetPathPointLocation(MoveSegmentStartIndex),
					*Path->GetPathPointLocation(MoveSegmentEndIndex));
			}
			else
			{
				if ((GetOwner() != nullptr) && (NavMovementInterface != nullptr))
				{
					UE_VLOG(GetOwner(), LogPathFollowing, Verbose, TEXT("Path blocked, but move segment indices are not valid: start %d, end %d of %d"), MoveSegmentStartIndex, MoveSegmentEndIndex, Path->GetPathPoints().Num());
				}
			}
			OnPathFinished(EPathFollowingResult::Blocked, FPathFollowingResultFlags::None);
		}
	}
}

void UPathFollowingComponent::FollowPathSegment(float DeltaTime)
{
	if (!Path.IsValid() || NavMovementInterface == nullptr)
	{
		return;
	}

	const FVector CurrentLocation = NavMovementInterface->GetFeetLocation();
	const FVector CurrentTarget = GetCurrentTargetLocation();
	
	// set to false by default, we will set set this back to true if appropriate
	bIsDecelerating = false;

	const bool bAccelerationBased = NavMovementInterface->UseAccelerationForPathFollowing();
	if (bAccelerationBased)
	{
		CurrentMoveInput = (CurrentTarget - CurrentLocation).GetSafeNormal();

		if (bStopMovementOnFinish && (MoveSegmentStartIndex >= DecelerationSegmentIndex))
		{
			const FVector PathEnd = Path->GetEndLocation();
			const FVector::FReal DistToEndSq = FVector::DistSquared(CurrentLocation, PathEnd);
			const bool bShouldDecelerate = DistToEndSq < FMath::Square(CachedBrakingDistance);
			if (bShouldDecelerate)
			{
				bIsDecelerating = true;

				const FVector::FReal  SpeedPct = FMath::Clamp(FMath::Sqrt(DistToEndSq) / CachedBrakingDistance, 0., 1.);
				CurrentMoveInput *= SpeedPct;
			}
		}

		PostProcessMove.ExecuteIfBound(this, CurrentMoveInput);
		NavMovementInterface->RequestPathMove(CurrentMoveInput);
	}
	else if (DeltaTime != 0.f)
	{
		FVector MoveVelocity = (CurrentTarget - CurrentLocation) / DeltaTime;

		const int32 LastSegmentStartIndex = Path->GetPathPoints().Num() - 2;
		const bool bNotFollowingLastSegment = (MoveSegmentStartIndex < LastSegmentStartIndex);

		PostProcessMove.ExecuteIfBound(this, MoveVelocity);
		NavMovementInterface->RequestDirectMove(MoveVelocity, bNotFollowingLastSegment);
	}
}

bool UPathFollowingComponent::HasReached(const FVector& TestPoint, EPathFollowingReachMode ReachMode, float InAcceptanceRadius) const
{
	// simple test for stationary agent, used as early finish condition
	const FVector CurrentLocation = NavMovementInterface.IsValid() ? NavMovementInterface->GetFeetLocation() : FVector::ZeroVector;
	const float GoalRadius = 0.0f;
	const float GoalHalfHeight = 0.0f;
	if (InAcceptanceRadius == UPathFollowingComponent::DefaultAcceptanceRadius)
	{
		InAcceptanceRadius = MyDefaultAcceptanceRadius;
	}

	const float AgentRadiusMod = (ReachMode == EPathFollowingReachMode::ExactLocation) || (ReachMode == EPathFollowingReachMode::OverlapGoal) ? 0.0f : MinAgentRadiusPct;
	return HasReachedInternal(TestPoint, GoalRadius, GoalHalfHeight, CurrentLocation, InAcceptanceRadius, AgentRadiusMod);
}

bool UPathFollowingComponent::HasReached(const AActor& TestGoal, EPathFollowingReachMode ReachMode, float InAcceptanceRadius, bool bUseNavAgentGoalLocation) const
{
	// simple test for stationary agent, used as early finish condition
	float GoalRadius = 0.0f;
	float GoalHalfHeight = 0.0f;
	FVector GoalOffset = FVector::ZeroVector;
	FVector TestPoint = TestGoal.GetActorLocation();
	if (InAcceptanceRadius == UPathFollowingComponent::DefaultAcceptanceRadius)
	{
		InAcceptanceRadius = MyDefaultAcceptanceRadius;
	}

	const INavAgentInterface* NavAgent = Cast<const INavAgentInterface>(&TestGoal);
	if (NavAgent)
	{
		const AActor* OwnerActor = GetOwner();
		const FVector GoalMoveOffset = NavAgent->GetMoveGoalOffset(OwnerActor);
		NavAgent->GetMoveGoalReachTest(OwnerActor, GoalMoveOffset, GoalOffset, GoalRadius, GoalHalfHeight);

		if (bUseNavAgentGoalLocation)
		{
			TestPoint = FQuatRotationTranslationMatrix(TestGoal.GetActorQuat(), NavAgent->GetNavAgentLocation()).TransformPosition(GoalOffset);

			if ((ReachMode == EPathFollowingReachMode::ExactLocation) || (ReachMode == EPathFollowingReachMode::OverlapAgent))
			{
				GoalRadius = 0.0f;
			}
		}

		if ((ReachMode == EPathFollowingReachMode::ExactLocation) || (ReachMode == EPathFollowingReachMode::OverlapAgent))
		{
			GoalRadius = 0.0f;
		}
	}

	const FVector CurrentLocation = NavMovementInterface.IsValid() ? NavMovementInterface->GetFeetLocation() : FVector::ZeroVector;
	const float AgentRadiusMod = (ReachMode == EPathFollowingReachMode::ExactLocation) || (ReachMode == EPathFollowingReachMode::OverlapGoal) ? 0.0f : MinAgentRadiusPct;
	return HasReachedInternal(TestPoint, GoalRadius, GoalHalfHeight, CurrentLocation, InAcceptanceRadius, AgentRadiusMod);
}

bool UPathFollowingComponent::HasReached(const FAIMoveRequest& MoveRequest) const
{
	const EPathFollowingReachMode ReachMode = MoveRequest.IsReachTestIncludingAgentRadius() ?
		(MoveRequest.IsReachTestIncludingGoalRadius() ? EPathFollowingReachMode::OverlapAgentAndGoal : EPathFollowingReachMode::OverlapAgent) :
		(MoveRequest.IsReachTestIncludingGoalRadius() ? EPathFollowingReachMode::OverlapGoal : EPathFollowingReachMode::ExactLocation);

	return MoveRequest.IsMoveToActorRequest() ?
		(MoveRequest.GetGoalActor() ? HasReached(*MoveRequest.GetGoalActor(), ReachMode, MoveRequest.GetAcceptanceRadius()) : false) :
		HasReached(MoveRequest.GetGoalLocation(), ReachMode, MoveRequest.GetAcceptanceRadius());
}

bool UPathFollowingComponent::HasReachedDestination(const FVector& CurrentLocation) const
{
	// get cylinder at goal location
	FVector GoalLocation = *Path->GetPathPointLocation(Path->GetPathPoints().Num() - 1);
	float GoalRadius = 0.0f;
	float GoalHalfHeight = 0.0f;
	
	// take goal's current location, unless path is partial or last segment doesn't reach goal actor (used by tethered AI)
	if (!Path->IsPartial() && bMoveToGoalOnLastSegment)
	{
		GoalLocation = *CurrentDestination;

		// Testing IsNearlyZero is not optimal as (0,0,0) could be a valid world position. Tracking the initialization state inside FBasedPosition would be better.
		if (GoalLocation.IsNearlyZero() && DestinationActor.IsValid() && !bMoveToGoalClampedToNavigation)
		{
			// In case CurrentDestination is not set because we didn't update MoveSegment yet, let's use the goal Actor's location directly
			if (DestinationAgent)
			{
				const AActor* OwnerActor = GetOwner();
				FVector GoalOffset;
				DestinationAgent->GetMoveGoalReachTest(OwnerActor, MoveOffset, GoalOffset, GoalRadius, GoalHalfHeight);

				GoalLocation = FQuatRotationTranslationMatrix(DestinationActor->GetActorQuat(), DestinationAgent->GetNavAgentLocation()).TransformPosition(GoalOffset);
			}
			else
			{
				GoalLocation = DestinationActor->GetActorLocation();
			}
		}
	}

	const float AcceptanceRangeToUse = GetFinalAcceptanceRadius(*Path, OriginalMoveRequestGoalLocation);
	return HasReachedInternal(GoalLocation, bReachTestIncludesGoalRadius ? GoalRadius : 0.0f, GoalHalfHeight, CurrentLocation
		, AcceptanceRangeToUse, bReachTestIncludesAgentRadius ? MinAgentRadiusPct : 0.0f);
}

bool UPathFollowingComponent::HasReachedCurrentTarget(const FVector& CurrentLocation) const
{
	if (NavMovementInterface == NULL)
	{
		return false;
	}

	// If the next segment is a link with a custom reach condition, we need to call the HasReachedLinkStart on the link interface.
	if (bMoveSegmentIsUsingCustomLinkReachCondition)
	{
		if (const INavLinkCustomInterface* MoveSegmentCustomLink = Cast<const INavLinkCustomInterface>(MoveSegmentCustomLinkOb.Get()))
		{
			if (ensureMsgf(Path.IsValid(), TEXT("%hs: Path should be valid when we get here. Owner [%s]."), __FUNCTION__, *GetNameSafe(GetOwner())))
			{
				const FNavPathPoint& LinkStart = Path->GetPathPoints()[MoveSegmentEndIndex];
				if (Path->GetPathPoints().IsValidIndex(MoveSegmentEndIndex + 1))
				{
					const FNavPathPoint& LinkEnd = Path->GetPathPoints()[MoveSegmentEndIndex + 1];
					return MoveSegmentCustomLink->HasReachedLinkStart(this, CurrentLocation, LinkStart, LinkEnd);
				}
				else
				{
					UE_LOGF(LogPathFollowing, Error, "%s: NavLink has a start, but no end. Custom reach condition won't be called. NavLinkID [%llu] - LinkStartPos [%ls] - Owner [%ls]", __FUNCTION__, LinkStart.CustomNavLinkId.GetId(), *LinkStart.Location.ToString(), *GetNameSafe(GetOwner()));
				}
			}
		}
	}

	const FVector CurrentTarget = GetCurrentTargetLocation();
	const FVector CurrentDirection = GetCurrentDirection();

	// check if moved too far
	const FVector ToTarget = (CurrentTarget - NavMovementInterface->GetFeetLocation());
	const FVector::FReal SegmentDot = FVector::DotProduct(ToTarget, CurrentDirection);
	if (SegmentDot < 0.)
	{
		return true;
	}

	// or standing at target position
	// don't use acceptance radius here, it has to be exact for moving near corners (2D test < 5% of agent radius)
	const float GoalRadius = 0.0f;
	const float GoalHalfHeight = 0.0f;

	return HasReachedInternal(CurrentTarget, GoalRadius, GoalHalfHeight, CurrentLocation, CurrentAcceptanceRadius, 0.05f);
}

bool UPathFollowingComponent::HasReachedInternal(const FVector& GoalLocation, float GoalRadius, float GoalHalfHeight, const FVector& AgentLocation, float RadiusThreshold, float AgentRadiusMultiplier) const
{
	if (NavMovementInterface == NULL)
	{
		return false;
	}

	// get cylinder of moving agent
	float AgentRadius = 0.0f;
	float AgentHalfHeight = 0.0f;
	NavMovementInterface->GetSimpleCollisionCylinder(AgentRadius, AgentHalfHeight);

	// check if they overlap (with added AcceptanceRadius)
	const FVector ToGoal = GoalLocation - AgentLocation;

	const FVector::FReal Dist2DSq = ToGoal.SizeSquared2D();
	const FVector::FReal UseRadius = RadiusThreshold + GoalRadius + (AgentRadius * AgentRadiusMultiplier);
	if (Dist2DSq > FMath::Square(UseRadius))
	{
		return false;
	}

	const FVector::FReal ZDiff = FMath::Abs(ToGoal.Z);
	const FVector::FReal UseHeight = GoalHalfHeight + (AgentHalfHeight * MinAgentHalfHeightPct);
	if (ZDiff > UseHeight)
	{
		return false;
	}

	return true;
}

int32 UPathFollowingComponent::FindPreciseAcceptanceRadiusTestsStartNodeIndex(const FNavigationPath& PathInstance, const FVector& GoalLocation) const
{
	const float DistanceFromEndOfPath = GetFinalAcceptanceRadius(PathInstance, GoalLocation);
	// @todo support based paths
	
	int32 TestStartIndex = MAX_int32;
	const TArray<FNavPathPoint>& PathPoints = PathInstance.GetPathPoints();

	if (PathPoints.Num() > 1)
	{
		FVector::FReal DistanceSum = 0.f;
		int32 NodeIndex = PathPoints.Num() - 1;
		FVector PrevLocation = PathPoints[NodeIndex].Location;
		--NodeIndex;
		for (; NodeIndex >= 0; --NodeIndex)
		{
			const FVector CurrentLocation = PathPoints[NodeIndex].Location;
			DistanceSum += FVector::Dist(CurrentLocation, PrevLocation);
			if (DistanceSum > DistanceFromEndOfPath)
			{
				TestStartIndex = NodeIndex;
				break;
			}
			
			PrevLocation = CurrentLocation;
			TestStartIndex = NodeIndex;
		}
	}

	return TestStartIndex;
}

float UPathFollowingComponent::GetFinalAcceptanceRadius(const FNavigationPath& PathInstance, const FVector OriginalGoalLocation, const FVector* PathEndOverride) const
{
	if (PathInstance.IsPartial())
	{
		ensure(FAISystem::IsValidLocation(OriginalGoalLocation));

		const FVector::FReal PathEndToGoalDistance = FVector::Dist(PathEndOverride ? *PathEndOverride : PathInstance.GetEndLocation(), OriginalGoalLocation);
		const FVector::FReal Remaining = AcceptanceRadius - PathEndToGoalDistance;
		
		// if goal is more than AcceptanceRadius from the end of the path
		// we just need to do regular path following, with default radius value
		return FloatCastChecked<float>(FMath::Max(Remaining, (FVector::FReal)MyDefaultAcceptanceRadius), UE::LWC::DefaultFloatPrecision);
	}

	return AcceptanceRadius;
}

void UPathFollowingComponent::DebugReachTest(float& CurrentDot, float& CurrentDistance, float& CurrentHeight, uint8& bDotFailed, uint8& bDistanceFailed, uint8& bHeightFailed) const
{
	if (!Path.IsValid() || NavMovementInterface == nullptr || NavMovementInterface->GetUpdatedObject() == nullptr)
	{
		return;
	}

	const int32 LastSegmentEndIndex = Path->GetPathPoints().Num() - 1;
	const bool bFollowingLastSegment = (MoveSegmentEndIndex >= LastSegmentEndIndex);

	float GoalRadius = 0.0f;
	float GoalHalfHeight = 0.0f;
	float RadiusThreshold = 0.0f;
	float AgentRadiusPct = 0.05f;

	FVector AgentLocation = NavMovementInterface->GetFeetLocation();
	FVector GoalLocation = GetCurrentTargetLocation();
	RadiusThreshold = CurrentAcceptanceRadius;

	if (bFollowingLastSegment)
	{
		GoalLocation = *Path->GetPathPointLocation(Path->GetPathPoints().Num() - 1);
		AgentRadiusPct = MinAgentRadiusPct;

		// take goal's current location, unless path is partial
		if (DestinationActor.IsValid() && !Path->IsPartial())
		{
			if (DestinationAgent)
			{
				const AActor* OwnerActor = GetOwner();
				FVector GoalOffset;
				DestinationAgent->GetMoveGoalReachTest(OwnerActor, MoveOffset, GoalOffset, GoalRadius, GoalHalfHeight);

				GoalLocation = FQuatRotationTranslationMatrix(DestinationActor->GetActorQuat(), DestinationAgent->GetNavAgentLocation()).TransformPosition(GoalOffset);
			}
			else
			{
				GoalLocation = DestinationActor->GetActorLocation();
			}
		}
	}

	const FVector ToGoal = (GoalLocation - AgentLocation);
	const FVector CurrentDirection = GetCurrentDirection();
	CurrentDot = FloatCastChecked<float>(FVector::DotProduct(ToGoal.GetSafeNormal(), CurrentDirection), /* Precision */ 1./128.);
	bDotFailed = (CurrentDot < 0.0f) ? 1 : 0;

	// get cylinder of moving agent
	float AgentRadius = 0.0f;
	float AgentHalfHeight = 0.0f;
	NavMovementInterface->GetSimpleCollisionCylinder(AgentRadius, AgentHalfHeight);

	CurrentDistance = FloatCastChecked<float>(ToGoal.Size2D(), UE::LWC::DefaultFloatPrecision);
	const float UseRadius = FMath::Max(RadiusThreshold, GoalRadius + (AgentRadius * AgentRadiusPct));
	bDistanceFailed = (CurrentDistance > UseRadius) ? 1 : 0;

	CurrentHeight = FloatCastChecked<float>(FMath::Abs(ToGoal.Z), UE::LWC::DefaultFloatPrecision);
	const float UseHeight = GoalHalfHeight + (AgentHalfHeight * MinAgentHalfHeightPct);
	bHeightFailed = (CurrentHeight > UseHeight) ? 1 : 0;
}

void UPathFollowingComponent::StartUsingCustomLink(INavLinkCustomInterface* CustomNavLink, const FVector& DestPoint)
{
	INavLinkCustomInterface* PrevNavLink = Cast<INavLinkCustomInterface>(CurrentCustomLinkOb.Get());
	if (PrevNavLink)
	{
		UE_VLOG(GetOwner(), LogPathFollowing, Log, TEXT("Force finish custom move using navlink: %s"), *GetPathNameSafe(CurrentCustomLinkOb.Get()));

		PrevNavLink->OnLinkMoveFinished(this);
		CurrentCustomLinkOb.Reset();
	}

	UObject* NewNavLinkOb = Cast<UObject>(CustomNavLink);
	if (NewNavLinkOb)
	{
		CurrentCustomLinkOb = NewNavLinkOb;

		const bool bCustomMove = CustomNavLink->OnLinkMoveStarted(this, DestPoint);
		UE_VLOG(GetOwner(), LogPathFollowing, Log, TEXT("%s navlink: %s"),
			bCustomMove ? TEXT("Custom move using") : TEXT("Notify"), *GetPathNameSafe(NewNavLinkOb));

		if (!bCustomMove)
		{
			CurrentCustomLinkOb = NULL;
		}
	}
}

void UPathFollowingComponent::FinishUsingCustomLink(INavLinkCustomInterface* CustomNavLink)
{
	if (CustomNavLink)
	{
		UObject* NavLinkOb = Cast<UObject>(CustomNavLink);
		if (CurrentCustomLinkOb == NavLinkOb)
		{
			UE_VLOG(GetOwner(), LogPathFollowing, Log, TEXT("Finish custom move using navlink: %s"), *GetPathNameSafe(NavLinkOb));

			CustomNavLink->OnLinkMoveFinished(this);
			CurrentCustomLinkOb.Reset();
		}
	}
}

bool UPathFollowingComponent::UpdateMovementComponent(bool bForce)
{
	if (NavMovementInterface == NULL || bForce == true)
	{
		APawn* MyPawn = Cast<APawn>(GetOwner());
		if (MyPawn == NULL)
		{
			AController* MyController = Cast<AController>(GetOwner());
			if (MyController)
			{
				MyPawn = MyController->GetPawn();
			}
		}

		if (MyPawn)
		{
			MyPawn->OnActorHit.AddUniqueDynamic(this, &UPathFollowingComponent::OnActorBump);

			SetNavMovementInterface(MyPawn->FindComponentByInterface<INavMovementInterface>());
		}
	}

	return (NavMovementInterface != nullptr);
}

void UPathFollowingComponent::OnActorBump(AActor* SelfActor, AActor* OtherActor, FVector NormalImpulse, const FHitResult& Hit)
{
	if (Path.IsValid() && DestinationActor.IsValid() && DestinationActor.Get() == OtherActor)
	{
		UE_VLOG(GetOwner(), LogPathFollowing, Verbose, TEXT("Collided with goal actor"));
		bCollidedWithGoal = true;
	}
}

void UPathFollowingComponent::ResetMoveSegmentCustomLinkCache()
{
	MoveSegmentCustomLinkOb.Reset();
	bMoveSegmentIsUsingCustomLinkReachCondition = false;
}

bool UPathFollowingComponent::IsOnPath() const
{
	bool bOnPath = false;
	if (Path.IsValid() && Path->IsValid() && Path->GetNavigationDataUsed() != NULL)
	{
		const bool bHasNavigationCorridor = (Path->CastPath<FNavMeshPath>() != NULL);
		if (bHasNavigationCorridor)
		{
			FNavLocation NavLoc = GetCurrentNavLocation();
			bOnPath = Path->ContainsNode(NavLoc.NodeRef);
		}
		else
		{
			bOnPath = true;
		}
	}

	return bOnPath;
}

bool UPathFollowingComponent::IsBlocked() const
{
	bool bBlocked = false;

	if (LocationSamples.Num() == BlockDetectionSampleCount && BlockDetectionSampleCount > 0)
	{
		const FVector::FReal BlockDetectionDistanceSq = FMath::Square(BlockDetectionDistance);

		FVector Center = FVector::ZeroVector;
		for (int32 SampleIndex = 0; SampleIndex < LocationSamples.Num(); SampleIndex++)
		{
			Center += *LocationSamples[SampleIndex];
		}

		Center /= LocationSamples.Num();
		bBlocked = true;

		for (int32 SampleIndex = 0; SampleIndex < LocationSamples.Num(); SampleIndex++)
		{
			const FVector::FReal TestDistanceSq = FVector::DistSquared(*LocationSamples[SampleIndex], Center);
			if (TestDistanceSq > BlockDetectionDistanceSq)
			{
				bBlocked = false;
				break;
			}
		}
	}

	return bBlocked;
}

bool UPathFollowingComponent::UpdateBlockDetection()
{
	const double GameTime = GetWorld()->GetTimeSeconds();
	if (bUseBlockDetection &&
		NavMovementInterface.IsValid() && 
		GameTime > (LastSampleTime + BlockDetectionInterval) &&
		BlockDetectionSampleCount > 0)
	{
		LastSampleTime = GameTime;

		if (LocationSamples.Num() == NextSampleIdx)
		{
			LocationSamples.AddZeroed(1);
		}

		LocationSamples[NextSampleIdx] = NavMovementInterface->GetFeetLocationBased();
		NextSampleIdx = (NextSampleIdx + 1) % BlockDetectionSampleCount;
		return true;
	}

	return false;
}

void UPathFollowingComponent::SetBlockDetection(float DistanceThreshold, float Interval, int32 NumSamples)
{
	BlockDetectionDistance = DistanceThreshold;
	BlockDetectionInterval = Interval;
	BlockDetectionSampleCount = NumSamples;
	ResetBlockDetectionData();
}

void UPathFollowingComponent::SetBlockDetectionState(bool bEnable)
{
	bUseBlockDetection = bEnable;
	ResetBlockDetectionData();
}

void UPathFollowingComponent::ResetBlockDetectionData()
{
	LastSampleTime = 0.0f;
	NextSampleIdx = 0;
	LocationSamples.Reset();
}

void UPathFollowingComponent::ForceBlockDetectionUpdate()
{
	LastSampleTime = 0.0f;
}

void UPathFollowingComponent::UpdateDecelerationData()
{
	// no point in updating if we don't have a MovementComp
	if (NavMovementInterface == nullptr)
	{
		return;
	}

	if (!bStopMovementOnFinish)
	{
		return;
	}

	const float CurrentMaxSpeed = NavMovementInterface->GetMaxSpeedForNavMovement();
	bool bUpdatePathSegment = (DecelerationSegmentIndex == INDEX_NONE);
	if (CurrentMaxSpeed != CachedBrakingMaxSpeed)
	{
		const float PrevBrakingDistance = CachedBrakingDistance;
		CachedBrakingDistance = NavMovementInterface->GetPathFollowingBrakingDistance(CurrentMaxSpeed);
		CachedBrakingMaxSpeed = CurrentMaxSpeed;
		
		bUpdatePathSegment = bUpdatePathSegment || (PrevBrakingDistance != CachedBrakingDistance);
	}

	if (bUpdatePathSegment && Path.IsValid())
	{
		DecelerationSegmentIndex = 0;

		const TArray<FNavPathPoint>& PathPoints = Path->GetPathPoints();
		FVector::FReal PathLengthFromEnd = 0.;

		for (int32 Idx = PathPoints.Num() - 1; Idx > 0; Idx--)
		{
			const FVector::FReal PathSegmentLength = FVector::Dist(PathPoints[Idx].Location, PathPoints[Idx - 1].Location);
			PathLengthFromEnd += PathSegmentLength;

			if (PathLengthFromEnd > CachedBrakingDistance)
			{
				DecelerationSegmentIndex = Idx - 1;
				break;
			}
		}

		UE_VLOG(GetOwner(), LogPathFollowing, Log, TEXT("Updated deceleration segment: %d (MaxSpeed:%.2f, BrakingDistance:%.2f"), DecelerationSegmentIndex, CachedBrakingMaxSpeed, CachedBrakingDistance);
	}
}

void UPathFollowingComponent::UpdateMoveFocus()
{
	AAIController* AIOwner = Cast<AAIController>(GetOwner());
	if (AIOwner != NULL)
	{
		if (Status == EPathFollowingStatus::Moving)
		{
			const FVector MoveFocus = GetMoveFocus(AIOwner->bAllowStrafe);
			AIOwner->SetFocalPoint(MoveFocus, EAIFocusPriority::Move);
		}
		else if (Status == EPathFollowingStatus::Idle)
		{
			AIOwner->ClearFocus(EAIFocusPriority::Move);
		}
	}
}

void UPathFollowingComponent::SetPreciseReachThreshold(float AgentRadiusMultiplier, float AgentHalfHeightMultiplier)
{
	MinAgentRadiusPct = AgentRadiusMultiplier;
	MinAgentHalfHeightPct = AgentHalfHeightMultiplier;
}

FVector::FReal UPathFollowingComponent::GetRemainingPathCost() const
{
	FVector::FReal Cost = 0.;

	if (Path.IsValid() && Path->IsValid() && Status == EPathFollowingStatus::Moving)
	{
		const FNavLocation& NavLocation = GetCurrentNavLocation();
		Cost = Path->GetCostFromNode(NavLocation.NodeRef);
	}

	return Cost;
}	

FNavLocation UPathFollowingComponent::GetCurrentNavLocation() const
{
	// get navigation location of moved actor
	if (NavMovementInterface == nullptr)
	{
		return FNavLocation();
	}

	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
	if (NavSys == nullptr)
	{
		return FNavLocation();
	}

	const FVector OwnerLoc = NavMovementInterface->GetNavLocation();
	const float Tolerance = 1.0f;
	
	// Using !Equals rather than != because exact equivalence isn't required.  (Equals allows a small tolerance.)
	if (!OwnerLoc.Equals(CurrentNavLocation.Location, Tolerance))
	{
		const FVector OwnerExtent = NavMovementInterface.IsValid() ? NavMovementInterface->GetSimpleCollisionCylinderExtent() : FVector::ZeroVector;

		NavSys->ProjectPointToNavigation(OwnerLoc, CurrentNavLocation, OwnerExtent, MyNavData);
	}

	return CurrentNavLocation;
}

bool UPathFollowingComponent::IsCurrentSegmentNavigationLink() const
{
	return Path.IsValid() && Path->GetPathPoints().IsValidIndex(MoveSegmentStartIndex) && FNavMeshNodeFlags(Path->GetPathPoints()[MoveSegmentStartIndex].Flags).IsNavLink();
}

FVector UPathFollowingComponent::GetCurrentDirection() const
{
	if (CurrentDestination.Base)
	{
		// calculate direction to based destination
		const FVector SegmentStartLocation = *Path->GetPathPointLocation(MoveSegmentStartIndex);
		const FVector SegmentEndLocation = *CurrentDestination;

		return (SegmentEndLocation - SegmentStartLocation).GetSafeNormal();
	}

	// use cached direction of current path segment
	return MoveSegmentDirection;
}

bool UPathFollowingComponent::HasDirectPath() const
{
	return Path.IsValid() ? (Path->CastPath<FNavMeshPath>() == NULL) : false;
}

FString UPathFollowingComponent::GetStatusDesc() const
{
	const static UEnum* StatusEnum = StaticEnum<EPathFollowingStatus::Type>();
	if (StatusEnum)
	{
		return StatusEnum->GetNameStringByValue(Status);
	}

	return TEXT("Unknown");
}

FString UPathFollowingComponent::GetResultDesc(EPathFollowingResult::Type Result) const
{
	const static UEnum* ResultEnum = StaticEnum<EPathFollowingResult::Type>();
	if (ResultEnum)
	{
		return ResultEnum->GetNameStringByValue(Result);
	}

	return TEXT("Unknown");
}

void UPathFollowingComponent::DisplayDebug(UCanvas* Canvas, const FDebugDisplayInfo& DebugDisplay, float& YL, float& YPos) const
{
	FDisplayDebugManager& DisplayDebugManager = Canvas->DisplayDebugManager;
	DisplayDebugManager.SetDrawColor(FColor::Blue);
	DisplayDebugManager.DrawString(FString::Printf(TEXT("  Move status: %s"), *GetStatusDesc()));

	if (Status == EPathFollowingStatus::Moving)
	{
		const int32 NumMoveSegments = (Path.IsValid() && Path->IsValid()) ? Path->GetPathPoints().Num() : -1;
		FString TargetDesc = FString::Printf(TEXT("  Move target [%d/%d]: %s (%s)"),
			MoveSegmentEndIndex, NumMoveSegments, *GetCurrentTargetLocation().ToString(), *GetNameSafe(DestinationActor.Get()));
		
		DisplayDebugManager.DrawString(FString::Printf(TEXT("  Move status: %s"), *GetStatusDesc()));
	}
}

#if ENABLE_VISUAL_LOG
void UPathFollowingComponent::DescribeSelfToVisLog(FVisualLogEntry* Snapshot) const
{
	FVisualLogStatusCategory Category;
	Category.Category = TEXT("Path following");

	if (DestinationActor.IsValid())
	{
		Category.Add(TEXT("Goal"), GetNameSafe(DestinationActor.Get()));
	}
	
	FString StatusDesc = GetStatusDesc();
	if (Status == EPathFollowingStatus::Moving && Path.IsValid())
	{
		StatusDesc += FString::Printf(TEXT(" [%d..%d/%d]%s%s"), MoveSegmentStartIndex + 1, MoveSegmentEndIndex + 1, Path->GetPathPoints().Num()
			, bWalkingNavLinkStart ? TEXT("navlink") : TEXT(""), Path->IsValid() ? TEXT("") : TEXT(" Invalidated"));
	}

	Category.Add(TEXT("Status"), StatusDesc);
	Category.Add(TEXT("MoveID"), GetCurrentRequestId().ToString());
	Category.Add(TEXT("Path"), !Path.IsValid() ? TEXT("none") :
		(Path->CastPath<FNavMeshPath>() != NULL) ? TEXT("navmesh") :
		(Path->CastPath<FAbstractNavigationPath>() != NULL) ? TEXT("direct") :
		TEXT("unknown"));
	Category.Add(TEXT("Block detection"), bUseBlockDetection ? FString::Printf(TEXT("last sample time %.2f"), LastSampleTime) : TEXT("Disabled"));
	
	UObject* CustomNavLinkOb = CurrentCustomLinkOb.Get();
	if (CustomNavLinkOb)
	{
		Category.Add(TEXT("Custom NavLink"), CustomNavLinkOb->GetName());
	}

	Snapshot->Status.Add(Category);
}
#endif // ENABLE_VISUAL_LOG

void UPathFollowingComponent::GetDebugStringTokens(TArray<FString>& Tokens, TArray<EPathFollowingDebugTokens::Type>& Flags) const
{
	Tokens.Add(GetStatusDesc());
	Flags.Add(EPathFollowingDebugTokens::Description);

	if (Status != EPathFollowingStatus::Moving)
	{
		return;
	}

	FString& StatusDesc = Tokens[0];
	if (Path.IsValid())
	{
		const int32 NumMoveSegments = (Path.IsValid() && Path->IsValid()) ? Path->GetPathPoints().Num() : -1;
		const bool bIsDirect = (Path->CastPath<FAbstractNavigationPath>() != NULL);
		const bool bIsCustomLink = CurrentCustomLinkOb.IsValid();

		if (!bIsDirect)
		{
			StatusDesc += FString::Printf(TEXT(" (%d..%d/%d)%s"), MoveSegmentStartIndex + 1, MoveSegmentEndIndex + 1, NumMoveSegments,
				bIsCustomLink ? TEXT(" (custom NavLink)") : TEXT(""));
		}
		else
		{
			StatusDesc += TEXT(" (direct)");
		}
	}
	else
	{
		StatusDesc += TEXT(" (invalid path)");
	}

	// add debug params
	float CurrentDot = 0.0f, CurrentDistance = 0.0f, CurrentHeight = 0.0f;
	uint8 bFailedDot = 0, bFailedDistance = 0, bFailedHeight = 0;
	DebugReachTest(CurrentDot, CurrentDistance, CurrentHeight, bFailedHeight, bFailedDistance, bFailedHeight);

	Tokens.Add(TEXT("dot"));
	Flags.Add(EPathFollowingDebugTokens::ParamName);
	Tokens.Add(FString::Printf(TEXT("%.2f"), CurrentDot));
	Flags.Add(bFailedDot ? EPathFollowingDebugTokens::FailedValue : EPathFollowingDebugTokens::PassedValue);

	Tokens.Add(TEXT("dist2D"));
	Flags.Add(EPathFollowingDebugTokens::ParamName);
	Tokens.Add(FString::Printf(TEXT("%.0f"), CurrentDistance));
	Flags.Add(bFailedDistance ? EPathFollowingDebugTokens::FailedValue : EPathFollowingDebugTokens::PassedValue);

	Tokens.Add(TEXT("distZ"));
	Flags.Add(EPathFollowingDebugTokens::ParamName);
	Tokens.Add(FString::Printf(TEXT("%.0f"), CurrentHeight));
	Flags.Add(bFailedHeight ? EPathFollowingDebugTokens::FailedValue : EPathFollowingDebugTokens::PassedValue);
}

FString UPathFollowingComponent::GetDebugString() const
{
	TArray<FString> Tokens;
	TArray<EPathFollowingDebugTokens::Type> Flags;

	GetDebugStringTokens(Tokens, Flags);

	FString Desc;
	for (int32 TokenIndex = 0; TokenIndex < Tokens.Num(); TokenIndex++)
	{
		Desc += Tokens[TokenIndex];
		Desc += (Flags.IsValidIndex(TokenIndex) && Flags[TokenIndex] == EPathFollowingDebugTokens::ParamName) ? TEXT(": ") : TEXT(", ");
	}

	return Desc;
}

bool UPathFollowingComponent::IsFollowingNavLink() const
{
	return IsCurrentSegmentNavigationLink();
}

bool UPathFollowingComponent::IsPathFollowingAllowed() const
{
	return NavMovementInterface.IsValid() && NavMovementInterface->CanStartPathFollowing();
}

void UPathFollowingComponent::OnUnableToMove(const UObject& Instigator)
{
	AbortMove(Instigator, FPathFollowingResultFlags::MovementStop);
}

void UPathFollowingComponent::OnStartedFalling()
{
	bWalkingNavLinkStart = false;
}

void UPathFollowingComponent::LockResource(EAIRequestPriority::Type LockSource)
{
	const static UEnum* SourceEnum = StaticEnum<EAILockSource::Type>();
	const bool bWasLocked = ResourceLock.IsLocked();

	ResourceLock.SetLock(LockSource);
	if (bWasLocked == false)
	{
		UE_VLOG(GetOwner(), LogPathFollowing, Log, TEXT("Locking Move by source %s"), SourceEnum ? *SourceEnum->GetNameStringByValue(Status) : TEXT("invalid"));
		PauseMove(CurrentRequestId, EPathFollowingVelocityMode::Reset);
	}
}

void UPathFollowingComponent::ClearResourceLock(EAIRequestPriority::Type LockSource)
{
	const bool bWasLocked = ResourceLock.IsLocked();
	ResourceLock.ClearLock(LockSource);

	if (bWasLocked && !ResourceLock.IsLocked())
	{
		UE_VLOG(GetOwner(), LogPathFollowing, Log, TEXT("Unlocking Move"));
		ResumeMove();
	}
}

void UPathFollowingComponent::ForceUnlockResource()
{
	const bool bWasLocked = ResourceLock.IsLocked();
	ResourceLock.ForceClearAllLocks();

	if (bWasLocked)
	{
		UE_VLOG(GetOwner(), LogPathFollowing, Log, TEXT("Unlocking Move - forced"));
		ResumeMove();
	}
}

bool UPathFollowingComponent::IsResourceLocked() const
{
	return ResourceLock.IsLocked();
}

void UPathFollowingComponent::SetLastMoveAtGoal(bool bFinishedAtGoal)
{
	if (Status == EPathFollowingStatus::Idle)
	{
		bLastMoveReachedGoal = bFinishedAtGoal;
	}
}

void UPathFollowingComponent::OnWaitingPathTimeout()
{
	if (Status == EPathFollowingStatus::Waiting)
	{
		UE_VLOG(GetOwner(), LogPathFollowing, Warning, TEXT("Waiting for path timeout! Aborting current move"));
		OnPathFinished(EPathFollowingResult::Invalid, FPathFollowingResultFlags::None);
	}
}

// deprecated functions
EPathFollowingAction::Type UPathFollowingComponent::GetPathActionType() const
{
	switch (Status)
	{
	case EPathFollowingStatus::Idle:
		return EPathFollowingAction::NoMove;

	case EPathFollowingStatus::Waiting:
	case EPathFollowingStatus::Paused:
	case EPathFollowingStatus::Moving:
		return Path.IsValid() == false ? EPathFollowingAction::Error :
			Path->CastPath<FAbstractNavigationPath>() ? EPathFollowingAction::DirectMove :
			Path->IsPartial() ? EPathFollowingAction::PartialPath : EPathFollowingAction::PathToGoal;

	default:
		break;
	}

	return EPathFollowingAction::Error;
}

FVector UPathFollowingComponent::GetPathDestination() const
{
	return Path.IsValid() ? Path->GetDestinationLocation() : FVector::ZeroVector;
}

#undef SHIPPING_STATIC

