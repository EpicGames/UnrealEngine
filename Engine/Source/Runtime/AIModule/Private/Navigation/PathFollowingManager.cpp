// Copyright Epic Games, Inc. All Rights Reserved.

#include "Navigation/PathFollowingManager.h"
#include "GameFramework/Pawn.h"
#include "Navigation/PathFollowingComponent.h"
#include "VisualLogger/VisualLogger.h"
#include "GameFramework/Controller.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PathFollowingManager)


UPathFollowingManager::UPathFollowingManager(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	struct FDelegatesInitializer
	{
		FDelegatesInitializer()
		{
			IPathFollowingManagerInterface::StopMovementDelegate().BindStatic(&UPathFollowingManager::StopMovement);
			IPathFollowingManagerInterface::IsFollowingAPathDelegate().BindStatic(&UPathFollowingManager::IsFollowingAPath);
		}
	};
	static FDelegatesInitializer DelegatesInitializer;
}

void UPathFollowingManager::StopMovement(const AController& Controller)
{
	UE_VLOG(&Controller, LogNavigation, Log, TEXT("AController::StopMovement: %s STOP MOVEMENT"), *GetNameSafe(Controller.GetPawn()));

	UPathFollowingComponent* PathFollowingComp = Controller.FindComponentByClass<UPathFollowingComponent>();
	if (PathFollowingComp != nullptr)
	{
		PathFollowingComp->AbortMove(Controller, FPathFollowingResultFlags::MovementStop);
	}
}

bool UPathFollowingManager::IsFollowingAPath(const AController& Controller)
{
	UPathFollowingComponent* PathFollowingComp = Controller.FindComponentByClass<UPathFollowingComponent>();
	return (PathFollowingComp != nullptr) && (PathFollowingComp->GetStatus() != EPathFollowingStatus::Idle);
}

