// Copyright Epic Games, Inc. All Rights Reserved.

#include "Navigation/RecastGraphAStar.h"
#include "AI/NavigationSystemBase.h"
#include "NavigationSystem.h"
#include "NavMesh/PImplRecastNavMesh.h"
#include "NavMesh/RecastNavMesh.h"
#include <type_traits>

#include UE_INLINE_GENERATED_CPP_BY_NAME(RecastGraphAStar)

#if WITH_RECAST

void FRecastGraphWrapper::Initialize(const ARecastNavMesh* InRecastNavMeshActor)
{
	RecastNavMeshActor = InRecastNavMeshActor;
	check(RecastNavMeshActor);
	check(RecastNavMeshActor->GetRecastNavMeshImpl());
	DetourNavMesh = RecastNavMeshActor->GetRecastNavMeshImpl()->GetRecastMesh();
	check(DetourNavMesh);
}

void FRecastGraphWrapper::BindFilter(FRecastGraphAStarFilter& AStarFilter)
{
	CachedNextLink = DT_NULL_LINK;
	RecastQuery.init(GetDetourNavMesh(), 0, &AStarFilter.GetLinkFilter());
}

FRecastNeighbour FRecastGraphWrapper::GetNeighbour(const FRecastAStarSearchNode& Node, const int32 NeighbourIndex) const
{
	if (NeighbourIndex == 0)
	{
		checkSlow(Node.HasValidCacheInfo())
		CachedNextLink = Node.Poly->firstLink;
	}

	// Are we done with the neighbours?
	if (CachedNextLink == DT_NULL_LINK)
	{
		return FRecastNeighbour(INVALID_NAVNODEREF);
	}

	const dtLink& Link = GetDetourNavMesh()->getLink(Node.Tile, CachedNextLink);
	CachedNextLink = Link.next;

	return FRecastNeighbour(Link.ref, Link.side);
}

dtStatus FRecastGraphWrapper::ConvertToRecastStatus(const FRecastAStar& Algo, const FRecastGraphAStarFilter& Filter, const EGraphAStarResult AStarResult) const
{
	dtStatus FindPathStatus = DT_SUCCESS;
	switch (AStarResult)
	{
	case SearchSuccess:
		FindPathStatus = DT_SUCCESS;
		break;
	case SearchFail:
		FindPathStatus = DT_FAILURE;
		break;
	case GoalUnreachable:
		FindPathStatus = DT_SUCCESS | DT_PARTIAL_RESULT;
		break;
	case InfiniteLoop:
		FindPathStatus = DT_FAILURE | DT_INVALID_CYCLE_PATH;
	}

	if (Algo.HasReachMaxSearchNodes(Filter))
	{
		FindPathStatus |= DT_OUT_OF_NODES;
	}

	return FindPathStatus;
}

static_assert(std::is_trivially_destructible_v<FRecastAStarSearchNode>, "FRecastAStarSearchNode must be trivially destructible");

dtPolyRef FRecastAStarResult::SetPathInfo(const int32 Index, const FRecastAStarSearchNode& SearchNode)
{
	data[Index].ref = SearchNode.NodeRef;
	data[Index].cost = SearchNode.TotalCost;
	data[Index].pos[0] = SearchNode.Position[0];
	data[Index].pos[1] = SearchNode.Position[1];
	data[Index].pos[2] = SearchNode.Position[2];
	return SearchNode.NodeRef;
}

FRecastGraphAStarFilter::FRecastGraphAStarFilter(FRecastGraphWrapper& InRecastGraphWrapper, const FRecastQueryFilter& InFilter, const uint32 InMaxSearchNodes, const FVector::FReal InCostLimit, const UObject* Owner)
	: Filter{ InFilter }
	, LinkFilter{ FNavigationSystem::GetCurrent<UNavigationSystemV1>(InRecastGraphWrapper.GetRecastNavMeshActor()->GetWorld()), Owner }
	, RecastGraphWrapper{ InRecastGraphWrapper }
	, MaxSearchNodes{ InMaxSearchNodes }
	, CostLimit{ InCostLimit }
{
	InRecastGraphWrapper.BindFilter(*this);
}

#endif // WITH_RECAST
