// Copyright Epic Games, Inc. All Rights Reserved.

#include "Perception/AIPerceptionComponent.h"
#include "Async/TaskGraphInterfaces.h"
#include "GameFramework/Controller.h"
#include "AIController.h"
#include "Perception/AISenseConfig.h"
#include "Stats/Stats.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AIPerceptionComponent)

#if WITH_GAMEPLAY_DEBUGGER_MENU
#include "GameplayDebuggerTypes.h"
#include "GameplayDebuggerCategory.h"
#endif // WITH_GAMEPLAY_DEBUGGER_MENU



DECLARE_CYCLE_STAT(TEXT("Perception Component ProcessStimuli"), STAT_AI_PercepComp_ProcessStimuli, STATGROUP_AI);

DECLARE_CYCLE_STAT(TEXT("Requesting UAIPerceptionComponent::RemoveDeadData call from within a const function"),
	STAT_FSimpleDelegateGraphTask_RequestingRemovalOfDeadPerceptionData,
	STATGROUP_TaskGraphTasks);

//----------------------------------------------------------------------//
// FActorPerceptionInfo
//----------------------------------------------------------------------//
void FActorPerceptionInfo::Merge(const FActorPerceptionInfo& Other)
{
	for (uint32 Index = 0; Index < FAISenseID::GetSize(); ++Index)
	{
		if (LastSensedStimuli[Index].GetAge() > Other.LastSensedStimuli[Index].GetAge())
		{
			LastSensedStimuli[Index] = Other.LastSensedStimuli[Index];
		}
	}
}

//----------------------------------------------------------------------//
// 
//----------------------------------------------------------------------//
FActorPerceptionBlueprintInfo::FActorPerceptionBlueprintInfo(const FActorPerceptionInfo& Info)
{
	Target = Info.Target.Get();
	LastSensedStimuli = Info.LastSensedStimuli;
	bIsHostile = Info.bIsHostile;
	bIsFriendly = Info.bIsFriendly;
}

//----------------------------------------------------------------------//
// 
//----------------------------------------------------------------------//
FActorPerceptionUpdateInfo::FActorPerceptionUpdateInfo(const int32 InTargetId, const TWeakObjectPtr<AActor>& InTarget, const FAIStimulus& InStimulus)
	: TargetId(InTargetId)
	, Target(InTarget)
	, Stimulus(InStimulus)
{
}

//----------------------------------------------------------------------//
// 
//----------------------------------------------------------------------//
const int32 UAIPerceptionComponent::InitialStimuliToProcessArraySize = 10;

UAIPerceptionComponent::UAIPerceptionComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, PerceptionListenerId(FPerceptionListenerID::InvalidID())
	, bCleanedUp(false)
{
	bForgetStaleActors = GET_AI_CONFIG_VAR(bForgetStaleActors);
}

void UAIPerceptionComponent::RequestStimuliListenerUpdate()
{
	UAIPerceptionSystem* AIPerceptionSys = UAIPerceptionSystem::GetCurrent(GetWorld());
	if (AIPerceptionSys != nullptr)
	{
		AIPerceptionSys->UpdateListener(*this);
	}
}

namespace
{
	struct FConfigOfSenseID
	{
		FConfigOfSenseID(const FAISenseID& InSenseID)
			: SenseID(InSenseID)
		{}

		bool operator()(const UAISenseConfig* SenseConfig) const
		{
			return SenseConfig && SenseConfig->GetSenseID() == SenseID;
		}

		const FAISenseID SenseID;
	};
}

const UAISenseConfig* UAIPerceptionComponent::GetSenseConfig(const FAISenseID& SenseID) const
{
	int32 ConfigIndex = SensesConfig.IndexOfByPredicate(FConfigOfSenseID(SenseID));
	return ConfigIndex != INDEX_NONE ? SensesConfig[ConfigIndex] : nullptr;
}

UAISenseConfig* UAIPerceptionComponent::GetSenseConfig(const FAISenseID& SenseID)
{
	int32 ConfigIndex = SensesConfig.IndexOfByPredicate(FConfigOfSenseID(SenseID));
	return ConfigIndex != INDEX_NONE ? SensesConfig[ConfigIndex] : nullptr;
}

void UAIPerceptionComponent::PostInitProperties() 
{
	Super::PostInitProperties();

	if (DominantSense)
	{
		DominantSenseID = UAISense::GetSenseID(DominantSense);
	}
}

void UAIPerceptionComponent::ConfigureSense(UAISenseConfig& Config)
{
	// first check if we're reconfiguring a sense
	bool bIsNewConfig = true;
	for (TObjectPtr<UAISenseConfig>& SenseConfig : SensesConfig)
	{
		if (SenseConfig != nullptr && SenseConfig->GetClass() == Config.GetClass())
		{
			SenseConfig = &Config;
			bIsNewConfig = false;
			break;
		}
	}

	if (bIsNewConfig)
	{
		SensesConfig.Add(&Config);
	}

	if (IsRegistered())
	{
	    UAIPerceptionSystem* AIPerceptionSys = UAIPerceptionSystem::GetCurrent(GetWorld());
	    if (AIPerceptionSys != nullptr)
	    {
			if (bIsNewConfig)
			{
				RegisterSenseConfig(Config, *AIPerceptionSys);
			}
			else
			{
				SetMaxStimulusAge(Config.GetSenseID(), Config.GetMaxAge());
			}
		    AIPerceptionSys->OnListenerConfigUpdated(Config.GetSenseID(), *this);
	    }
	}
	// else the sense will be auto-configured during OnRegister
}

UAIPerceptionComponent::TAISenseConfigConstIterator UAIPerceptionComponent::GetSensesConfigIterator() const
{
	return ToRawPtrTArrayUnsafe(SensesConfig).CreateConstIterator();
}

void UAIPerceptionComponent::SetMaxStimulusAge(const FAISenseID SenseID, float MaxAge)
{
	if (!ensureMsgf(SenseID.IsValid(), TEXT("Sense must exist to update max age")))
	{
		return;
	}

	if (MaxActiveAge.IsValidIndex(SenseID) == false)
	{
		MaxActiveAge.AddUninitialized(SenseID - MaxActiveAge.Num() + 1);
	}
	MaxActiveAge[SenseID] = MaxAge;

	// @todo process all data already gathered and see if any _still_active_ stimuli
	// got it's expiration prolonged, with SetExpirationAge
}

void UAIPerceptionComponent::OnRegister()
{
	Super::OnRegister();

	bCleanedUp = false;

	AActor* Owner = GetOwner();
	if (Owner)
	{
		Owner->OnEndPlay.AddUniqueDynamic(this, &UAIPerceptionComponent::OnOwnerEndPlay);
	}

	UAIPerceptionSystem* AIPerceptionSys = UAIPerceptionSystem::GetCurrent(GetWorld());
	if (AIPerceptionSys != nullptr)
	{
		PerceptionFilter.Clear();

		if (SensesConfig.Num() > 0)
		{
			// set up perception listener based on SensesConfig
			for (auto SenseConfig : SensesConfig)
			{
				if (SenseConfig)
				{
					RegisterSenseConfig(*SenseConfig, *AIPerceptionSys);
				}
			}

			AIPerceptionSys->UpdateListener(*this);
		}
	}
	
	AIOwner = Cast<AAIController>(Owner);
	if (AIOwner)
	{
		if (AIOwner->GetAIPerceptionComponent() == nullptr)
		{
			// this should not be needed but apparently AAIController::PostRegisterAllComponents
			// gets called component's OnRegister
			AIOwner->SetPerceptionComponent(*this);
		}
		else
		{
			ensure(AIOwner->GetAIPerceptionComponent() == this || (AIOwner->GetWorld() && AIOwner->GetWorld()->WorldType != EWorldType::Editor));
		}
	}
	else if (Owner)
	{
		UE_CVLOG_UELOG(Owner->GetWorld() && (Owner->GetWorld()->WorldType != EWorldType::Editor), Owner, LogAIPerception, Log, TEXT("%s: Owner %s is not an AIController so make sure you bind to OnPerceptionUpdated, otherwise you won't get notified about updated actor perception events"), ANSI_TO_TCHAR(__FUNCTION__), *Owner->GetName());
	}
}

void UAIPerceptionComponent::RegisterSenseConfig(const UAISenseConfig& SenseConfig, UAIPerceptionSystem& AIPerceptionSys)
{
	const TSubclassOf<UAISense> SenseImplementation = SenseConfig.GetSenseImplementation();
	if (SenseImplementation)
	{
		// make sure it's registered with perception system
		const FAISenseID SenseID = AIPerceptionSys.RegisterSenseClass(SenseImplementation);
		check(SenseID.IsValid());

		if (SenseConfig.GetStartsEnabled())
		{
			PerceptionFilter.AcceptChannel(SenseID);
		}

		SetMaxStimulusAge(SenseID, SenseConfig.GetMaxAge());
	}
}

void UAIPerceptionComponent::OnUnregister()
{
	CleanUp();
	Super::OnUnregister();
}

void UAIPerceptionComponent::OnOwnerEndPlay(AActor* Actor, EEndPlayReason::Type EndPlayReason)
{
	if (EndPlayReason != EEndPlayReason::EndPlayInEditor && EndPlayReason != EEndPlayReason::Quit)
	{
		CleanUp();
	}
}

void UAIPerceptionComponent::CleanUp()
{
	if (bCleanedUp == false)
	{
		ForgetAll();

		UAIPerceptionSystem* AIPerceptionSys = UAIPerceptionSystem::GetCurrent(GetWorld());
		if (AIPerceptionSys != nullptr)
		{
			AIPerceptionSys->UnregisterListener(*this);
			AActor* MutableBodyActor = GetMutableBodyActor();
			if (MutableBodyActor)
			{
				AIPerceptionSys->UnregisterSource(*MutableBodyActor);
			}
		}

		if (HasAnyFlags(RF_BeginDestroyed) == false)
		{
			AActor* Owner = GetOwner();
			if (Owner != nullptr)
			{
				Owner->OnEndPlay.RemoveDynamic(this, &UAIPerceptionComponent::OnOwnerEndPlay);
			}
		}

		bCleanedUp = true;
	}
}

void UAIPerceptionComponent::BeginDestroy()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		CleanUp();
	}
	Super::BeginDestroy();
}

void UAIPerceptionComponent::UpdatePerceptionAllowList(const FAISenseID Channel, const bool bNewValue)
{
	// Return if we don't have a Sense Config as it doesn't make sense to update the perception allow list.
	// Also modifying this often requires the Sense Config further along the call stack.
	if (GetSenseConfig(Channel) == nullptr)
	{
		UE_VLOG_UELOG(GetOwner(), LogAIPerception, Warning, TEXT("%s: %s: Channel has no Sense Config. Bailing out!!"), ANSI_TO_TCHAR(__FUNCTION__), *Channel.Name.ToString());
		return;
	}

	const bool bCurrentValue = PerceptionFilter.ShouldRespondToChannel(Channel);
	if (bNewValue != bCurrentValue)
	{
		bNewValue ? PerceptionFilter.AcceptChannel(Channel) : PerceptionFilter.FilterOutChannel(Channel);
		RequestStimuliListenerUpdate();
	}
}

bool UAIPerceptionComponent::GetFilteredActors(const TFunctionRef<bool(const FActorPerceptionInfo&)>& Predicate, TArray<AActor*>& OutActors) const
{
	bool bDeadDataFound = false;

	OutActors.Reserve(PerceptualData.Num());
	for (FActorPerceptionContainer::TConstIterator DataIt = GetPerceptualDataConstIterator(); DataIt; ++DataIt)
	{
		const FActorPerceptionInfo& ActorPerceptionInfo = DataIt->Value;
		if (Predicate(ActorPerceptionInfo))
		{
			if (AActor* Actor = ActorPerceptionInfo.Target.Get())
			{
				OutActors.Add(Actor);
			}
			else
			{
				bDeadDataFound = true;
			}
		}
	}
	return bDeadDataFound;
}

void UAIPerceptionComponent::GetHostileActors(TArray<AActor*>& OutActors) const
{
	const bool bDeadDataFound = GetFilteredActors([](const FActorPerceptionInfo& ActorPerceptionInfo) {
			return (ActorPerceptionInfo.bIsHostile && ActorPerceptionInfo.HasAnyKnownStimulus());
		}, OutActors);

	if (bDeadDataFound)
	{
		FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
			FSimpleDelegateGraphTask::FDelegate::CreateUObject(const_cast<UAIPerceptionComponent*>(this), &UAIPerceptionComponent::RemoveDeadData),
			GET_STATID(STAT_FSimpleDelegateGraphTask_RequestingRemovalOfDeadPerceptionData), nullptr, ENamedThreads::GameThread);
	}
}

void UAIPerceptionComponent::GetHostileActorsBySense(TSubclassOf<UAISense> SenseToFilterBy, TArray<AActor*>& OutActors) const
{
	const FAISenseID SenseIdFilter = UAISense::GetSenseID(SenseToFilterBy);

	if (SenseIdFilter == FAISenseID::InvalidID())
	{
		UE_VLOG(GetOwner(), LogAIPerception, Warning, TEXT("UAIPerceptionComponent::GetHostileActorsBySense called with an invalid or yet unregistered sense. Bailing out."));
		return;
	}

	const bool bDeadDataFound = GetFilteredActors([SenseIdFilter](const FActorPerceptionInfo& ActorPerceptionInfo) {
		return (ActorPerceptionInfo.bIsHostile && ActorPerceptionInfo.HasKnownStimulusOfSense(SenseIdFilter));
		}, OutActors);

	if (bDeadDataFound)
	{
		FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
			FSimpleDelegateGraphTask::FDelegate::CreateUObject(const_cast<UAIPerceptionComponent*>(this), &UAIPerceptionComponent::RemoveDeadData),
			GET_STATID(STAT_FSimpleDelegateGraphTask_RequestingRemovalOfDeadPerceptionData), nullptr, ENamedThreads::GameThread);
	}
}

const FActorPerceptionInfo* UAIPerceptionComponent::GetFreshestTrace(const FAISenseID Sense) const
{
	// @note will stop on first age 0 stimulus
	float BestAge = FAIStimulus::NeverHappenedAge;
	const FActorPerceptionInfo* Result = nullptr;

	bool bDeadDataFound = false;
	
	for (FActorPerceptionContainer::TConstIterator DataIt = GetPerceptualDataConstIterator(); DataIt; ++DataIt)
	{
		const FActorPerceptionInfo* Info = &DataIt->Value;
		const float Age = Info->LastSensedStimuli[Sense].GetAge();
		if (Age < BestAge)
		{
			if (Info->Target.IsValid())
			{
				BestAge = Age;
				Result = Info;
				if (BestAge == 0.f)
				{
					// won't find any younger then this
					break;
				}
			}
			else
			{
				bDeadDataFound = true;
			}
		}
	}

	if (bDeadDataFound)
	{
		FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
			FSimpleDelegateGraphTask::FDelegate::CreateUObject(const_cast<UAIPerceptionComponent*>(this), &UAIPerceptionComponent::RemoveDeadData),
			GET_STATID(STAT_FSimpleDelegateGraphTask_RequestingRemovalOfDeadPerceptionData), nullptr, ENamedThreads::GameThread);
	}

	return Result;
}

void UAIPerceptionComponent::SetDominantSense(TSubclassOf<UAISense> InDominantSense)
{
	if (DominantSense != InDominantSense)
	{
		DominantSense = InDominantSense;
		DominantSenseID = UAISense::GetSenseID(InDominantSense);
		// update all perceptual info with this info
		for (FActorPerceptionContainer::TIterator DataIt = GetPerceptualDataIterator(); DataIt; ++DataIt)
		{
			DataIt->Value.DominantSense = DominantSenseID;
		}
	}
}

FGenericTeamId UAIPerceptionComponent::GetTeamIdentifier() const
{
	return FGenericTeamId::GetTeamIdentifier(GetOwner());
}

FVector UAIPerceptionComponent::GetActorLocation(const AActor& Actor) const 
{ 
	// not that Actor == NULL is valid
	const FActorPerceptionInfo* ActorInfo = GetActorInfo(Actor);
	return ActorInfo ? ActorInfo->GetLastStimulusLocation() : FAISystem::InvalidLocation;
}

void UAIPerceptionComponent::GetLocationAndDirection(FVector& Location, FVector& Direction) const
{
	const AActor* OwnerActor = Cast<AActor>(GetOuter());
	if (OwnerActor != nullptr)
	{
		FRotator ViewRotation(ForceInitToZero);
		OwnerActor->GetActorEyesViewPoint(Location, ViewRotation);
		Direction = ViewRotation.Vector();
	}
}

const AActor* UAIPerceptionComponent::GetBodyActor() const
{
	const AController* OwnerController = Cast<AController>(GetOuter());
	if (OwnerController != nullptr)
	{
		return OwnerController->GetPawn();
	}

	return Cast<AActor>(GetOuter());
}

AActor* UAIPerceptionComponent::GetMutableBodyActor()
{
	return const_cast<AActor*>(GetBodyActor());
}

void UAIPerceptionComponent::RegisterStimulus(AActor* Source, const FAIStimulus& Stimulus)
{
	FStimulusToProcess& StimulusToProcess = StimuliToProcess.Add_GetRef(FStimulusToProcess(Source, Stimulus));
	StimulusToProcess.Stimulus.SetExpirationAge(MaxActiveAge[int32(Stimulus.Type)]);
}

void UAIPerceptionComponent::ProcessStimuli()
{
	SCOPE_CYCLE_COUNTER(STAT_AI_PercepComp_ProcessStimuli);
	
	if(StimuliToProcess.Num() == 0)
	{
		UE_VLOG(GetOwner(), LogAIPerception, Warning, TEXT("UAIPerceptionComponent::ProcessStimuli called without any Stimuli to process"));
		return;
	}

	const bool bBroadcastEveryTargetUpdate = OnTargetPerceptionUpdated.IsBound();
	const bool bBroadcastEveryTargetInfoUpdate = OnTargetPerceptionInfoUpdated.IsBound();
	
	TArray<FStimulusToProcess> ProcessingStimuli = MoveTemp(StimuliToProcess);
	TArray<AActor*> UpdatedActors;
	UpdatedActors.Reserve(ProcessingStimuli.Num());
	TArray<AActor*> ActorsToForget;
	ActorsToForget.Reserve(ProcessingStimuli.Num());
	TArray<TObjectKey<AActor>, TInlineAllocator<8>> DataToRemove;

	for (FStimulusToProcess& SourcedStimulus : ProcessingStimuli)
	{
		const TObjectKey<AActor>& SourceKey = SourcedStimulus.Source;

		FActorPerceptionInfo* PerceptualInfo = PerceptualData.Find(SourceKey);
		AActor* SourceActor = nullptr;

		if (PerceptualInfo == nullptr)
		{
			if (SourcedStimulus.Stimulus.WasSuccessfullySensed() == false)
			{
				// this means it's a failed perception of an actor our owner is not aware of
				// at all so there's no point in creating perceptual data for a failed stimulus
				continue;
			}
			else
			{
				SourceActor = CastChecked<AActor>(SourceKey.ResolveObjectPtr(), ECastCheckedType::NullAllowed);

				// no existing perceptual data and source no longer valid: nothing to do with this stimulus
				if (SourceActor == nullptr)
				{
					continue;
				}
				
				// create an entry
				PerceptualInfo = &PerceptualData.Add(SourceKey, FActorPerceptionInfo(SourceActor));
				// tell it what's our dominant sense
				PerceptualInfo->DominantSense = DominantSenseID;

				PerceptualInfo->bIsHostile = (FGenericTeamId::GetAttitude(GetOwner(), SourceActor) == ETeamAttitude::Hostile);
				PerceptualInfo->bIsFriendly = PerceptualInfo->bIsHostile ? false : (FGenericTeamId::GetAttitude(GetOwner(), SourceActor) == ETeamAttitude::Friendly);
			}
		}

		if (PerceptualInfo->LastSensedStimuli.Num() <= SourcedStimulus.Stimulus.Type)
		{
			const int32 NumberToAdd = SourcedStimulus.Stimulus.Type - PerceptualInfo->LastSensedStimuli.Num() + 1;
			PerceptualInfo->LastSensedStimuli.AddDefaulted(NumberToAdd);
		}

		check(SourcedStimulus.Stimulus.Type.IsValid());

		FAIStimulus& StimulusStore = PerceptualInfo->LastSensedStimuli[SourcedStimulus.Stimulus.Type];
		const bool bActorInfoUpdated = SourcedStimulus.Stimulus.WantsToNotifyOnlyOnPerceptionChange() == false 
			|| SourcedStimulus.Stimulus.WasSuccessfullySensed() != StimulusStore.WasSuccessfullySensed();

		bool bRequiresUpdate = bActorInfoUpdated;

		if (SourcedStimulus.Stimulus.WasSuccessfullySensed())
		{
			bRequiresUpdate |= ConditionallyStoreSuccessfulStimulus(StimulusStore, SourcedStimulus.Stimulus);
			
		}
		else if (StimulusStore.IsExpired() == false)
		{	
			if (bActorInfoUpdated)
			{
				// @note there some more valid info in SourcedStimulus->Stimulus regarding test that failed
				// may be useful in future
				StimulusStore.MarkNoLongerSensed();
				StimulusStore.SetStimulusAge(0);
			}
		}
		else
		{
			HandleExpiredStimulus(StimulusStore);

			if (bForgetStaleActors && !PerceptualInfo->HasAnyCurrentStimulus())
			{
				if (AActor* ActorToForget = PerceptualInfo->Target.Get())
				{
					ActorsToForget.Add(ActorToForget);
				}
			}
		}

		// if the new stimulus is "valid" or it's info that "no longer sensed" and it used to be sensed successfully
		if (bRequiresUpdate)
		{
			// Source Actor is only resolved from SourceKey when required but might already have been resolved for new entry
			SourceActor = (SourceActor == nullptr) ? CastChecked<AActor>(SourceKey.ResolveObjectPtr(), ECastCheckedType::NullAllowed) : SourceActor;
			if (SourceActor == nullptr)
			{
				DataToRemove.Add(SourceKey);
			}
			else
			{
				UpdatedActors.AddUnique(SourceActor);
				if (bBroadcastEveryTargetUpdate)
				{
					OnTargetPerceptionUpdated.Broadcast(SourceActor, StimulusStore);
				}
			}

			if (bBroadcastEveryTargetInfoUpdate)
			{
				OnTargetPerceptionInfoUpdated.Broadcast(FActorPerceptionUpdateInfo(GetTypeHash(SourceKey), PerceptualInfo->Target, StimulusStore));
			}
		}
	}

	if (UpdatedActors.Num() > 0)
	{
		if (AIOwner != nullptr)
		{
			AIOwner->ActorsPerceptionUpdated(UpdatedActors);
		}

		OnPerceptionUpdated.Broadcast(UpdatedActors);
	}

	// forget actors that are no longer perceived
	for (AActor* ActorToForget : ActorsToForget)
	{
		ForgetActor(ActorToForget);
	}

	// remove perceptual info related to stale actors
	for (const TObjectKey<AActor>& SourceKey : DataToRemove)
	{
		PerceptualData.Remove(SourceKey);
	}
}

void UAIPerceptionComponent::RefreshStimulus(FAIStimulus& StimulusStore, const FAIStimulus& NewStimulus)
{
	ConditionallyStoreSuccessfulStimulus(StimulusStore, NewStimulus);
}

bool UAIPerceptionComponent::ConditionallyStoreSuccessfulStimulus(FAIStimulus& StimulusStore, const FAIStimulus& NewStimulus)
{
	// if new stimulus is younger or stronger
	// note that stimulus Age depends on PerceptionSystem::PerceptionAgingRate. It's possible that 
	// both already stored and the new stimulus have Age of 0, but stored stimulus' actual age is in [0, PerceptionSystem::PerceptionAgingRate)
	if (NewStimulus.GetAge() <= StimulusStore.GetAge() || StimulusStore.Strength < NewStimulus.Strength)
	{
		StimulusStore = NewStimulus;
	}

	return false;
}

void UAIPerceptionComponent::HandleExpiredStimulus(FAIStimulus& StimulusStore)
{
	ensure(StimulusStore.IsExpired() == true);
}

bool UAIPerceptionComponent::AgeStimuli(const float ConstPerceptionAgingRate)
{
	bool bExpiredStimuli = false;

	for (FActorPerceptionContainer::TIterator It(PerceptualData); It; ++It)
	{
		FActorPerceptionInfo& ActorPerceptionInfo = It->Value;

		for (FAIStimulus& Stimulus : ActorPerceptionInfo.LastSensedStimuli)
		{
			// Age the stimulus. If it is active but has just expired, mark it as such
			if (Stimulus.AgeStimulus(ConstPerceptionAgingRate) == false
				&& (Stimulus.IsActive() || Stimulus.WantsToNotifyOnlyOnPerceptionChange())
				&& Stimulus.IsExpired() == false)
			{
				AActor* TargetActor = ActorPerceptionInfo.Target.Get();
				if (TargetActor)
				{
					Stimulus.MarkExpired();
					RegisterStimulus(TargetActor, Stimulus);
					bExpiredStimuli = true;
				}
			}
		}
	}

	return bExpiredStimuli;
}

void UAIPerceptionComponent::ForgetActor(AActor* ActorToForget)
{
	if (PerceptualData.Num() > 0)
	{
		UAIPerceptionSystem* AIPerceptionSys = UAIPerceptionSystem::GetCurrent(GetWorld());
		if (AIPerceptionSys != nullptr && ActorToForget != nullptr)
		{
			AIPerceptionSys->OnListenerForgetsActor(*this, *ActorToForget);
		}

		const int32 NumRemoved = PerceptualData.Remove(ActorToForget);

		// notify anyone interested
		if (NumRemoved > 0 && OnTargetPerceptionForgotten.IsBound())
		{
			OnTargetPerceptionForgotten.Broadcast(ActorToForget);
		}
	}
}

void UAIPerceptionComponent::ForgetAll()
{
	if (PerceptualData.Num() > 0)
	{
		UAIPerceptionSystem* AIPerceptionSys = UAIPerceptionSystem::GetCurrent(GetWorld());
		if (AIPerceptionSys != nullptr)
		{
			AIPerceptionSys->OnListenerForgetsAll(*this);
		}

		// notify anyone interested
		if (OnTargetPerceptionForgotten.IsBound())
		{
			for (FActorPerceptionContainer::TConstIterator DataIt = GetPerceptualDataConstIterator(); DataIt; ++DataIt)
			{
				if (AActor* ActorToForget = DataIt.Value().Target.Get())
				{
					OnTargetPerceptionForgotten.Broadcast(ActorToForget);
				}
			}
		}

		PerceptualData.Reset();
	}
}

float UAIPerceptionComponent::GetYoungestStimulusAge(const AActor& Source) const
{
	const FActorPerceptionInfo* Info = GetActorInfo(Source);
	if (Info == nullptr)
	{
		return FAIStimulus::NeverHappenedAge;
	}

	float SmallestAge = FAIStimulus::NeverHappenedAge;
	for (int32 SenseID = 0; SenseID < Info->LastSensedStimuli.Num(); ++SenseID)
	{
		if (Info->LastSensedStimuli[SenseID].WasSuccessfullySensed())
		{
			float SenseAge = Info->LastSensedStimuli[SenseID].GetAge();
			if (SenseAge < SmallestAge)
			{
				SmallestAge = SenseAge;
			}
		}
	}

	return SmallestAge;
}

bool UAIPerceptionComponent::HasAnyActiveStimulus(const AActor& Source) const
{
	const FActorPerceptionInfo* Info = GetActorInfo(Source);
	if (Info == nullptr)
	{
		return false;
	}

	return Info->HasAnyKnownStimulus();
}

bool UAIPerceptionComponent::HasAnyCurrentStimulus(const AActor& Source) const
{
	const FActorPerceptionInfo* Info = GetActorInfo(Source);
	if (Info == nullptr)
	{
		return false;
	}

	return Info->HasAnyCurrentStimulus();
}

bool UAIPerceptionComponent::HasActiveStimulus(const AActor& Source, const FAISenseID Sense) const
{
	const FActorPerceptionInfo* Info = GetActorInfo(Source);
	return (Info 
		&& Info->LastSensedStimuli.IsValidIndex(Sense) 
		&& Info->LastSensedStimuli[Sense].WasSuccessfullySensed()
		&& Info->LastSensedStimuli[Sense].GetAge() < FAIStimulus::NeverHappenedAge
		&& (Info->LastSensedStimuli[Sense].GetAge() <= MaxActiveAge[Sense] || MaxActiveAge[Sense] == 0.f));
}

void UAIPerceptionComponent::RemoveDeadData()
{
	for (FActorPerceptionContainer::TIterator It(PerceptualData); It; ++It)
	{
		if (It->Value.Target.IsValid() == false)
		{
			It.RemoveCurrent();
		}
	}
}

//----------------------------------------------------------------------//
// blueprint interface
//----------------------------------------------------------------------//
void UAIPerceptionComponent::GetPerceivedHostileActors(TArray<AActor*>& OutActors) const
{
	GetHostileActors(OutActors);
}

void UAIPerceptionComponent::GetPerceivedHostileActorsBySense(const TSubclassOf<UAISense> SenseToUse, TArray<AActor*>& OutActors) const
{
	GetHostileActorsBySense(SenseToUse, OutActors);
}

void UAIPerceptionComponent::GetCurrentlyPerceivedActors(TSubclassOf<UAISense> SenseToUse, TArray<AActor*>& OutActors) const
{
	const FAISenseID SenseID = UAISense::GetSenseID(SenseToUse);

	OutActors.Reserve(PerceptualData.Num());
	for (FActorPerceptionContainer::TConstIterator DataIt = GetPerceptualDataConstIterator(); DataIt; ++DataIt)
	{
		const bool bCurrentlyPerceived = (SenseToUse == nullptr) ? DataIt->Value.HasAnyCurrentStimulus() : DataIt->Value.IsSenseActive(SenseID);
		if (bCurrentlyPerceived)
		{
			if (AActor* Actor = DataIt->Value.Target.Get())
			{
				OutActors.Add(Actor);
			}
		}
	}
}

void UAIPerceptionComponent::GetKnownPerceivedActors(TSubclassOf<UAISense> SenseToUse, TArray<AActor*>& OutActors) const
{
	const FAISenseID SenseID = UAISense::GetSenseID(SenseToUse);

	OutActors.Reserve(PerceptualData.Num());
	for (FActorPerceptionContainer::TConstIterator DataIt = GetPerceptualDataConstIterator(); DataIt; ++DataIt)
	{
		const bool bWasEverPerceived = (SenseToUse == nullptr) ? DataIt->Value.HasAnyKnownStimulus() : DataIt->Value.HasKnownStimulusOfSense(SenseID);
		if (bWasEverPerceived)
		{
			if (DataIt->Value.Target.IsValid())
			{
				OutActors.Add(DataIt->Value.Target.Get());
			}
		}
	}
}

bool UAIPerceptionComponent::GetActorsPerception(AActor* Actor, FActorPerceptionBlueprintInfo& Info)
{
	bool bInfoFound = false;
	if (Actor != nullptr && Actor->IsPendingKillPending() == false)
	{
		const FActorPerceptionInfo* PerceivedInfo = GetActorInfo(*Actor);
		if (PerceivedInfo)
		{
			Info = FActorPerceptionBlueprintInfo(*PerceivedInfo);
			bInfoFound = true;
		}
	}

	return bInfoFound;
}

void UAIPerceptionComponent::SetSenseEnabled(TSubclassOf<UAISense> SenseClass, const bool bEnable)
{
	const FAISenseID SenseID = UAISense::GetSenseID(SenseClass);
	if (SenseID.IsValid())
	{
		UpdatePerceptionAllowList(SenseID, bEnable);
	}
}

bool UAIPerceptionComponent::IsSenseEnabled(TSubclassOf<UAISense> SenseClass) const
{
	const FAISenseID SenseID = UAISense::GetSenseID(SenseClass);
	if (!SenseID.IsValid() || GetSenseConfig(SenseID) == nullptr)
	{
		return false;
	}

	return PerceptionFilter.ShouldRespondToChannel(SenseID);
}

//----------------------------------------------------------------------//
// debug
//----------------------------------------------------------------------//
#if WITH_GAMEPLAY_DEBUGGER_MENU
void UAIPerceptionComponent::DescribeSelfToGameplayDebugger(FGameplayDebuggerCategory* DebuggerCategory) const
{
	if (DebuggerCategory == nullptr)
	{
		return;
	}

	for (UAIPerceptionComponent::FActorPerceptionContainer::TConstIterator It(GetPerceptualDataConstIterator()); It; ++It)
	{
		const FActorPerceptionInfo& ActorPerceptionInfo = It->Value;
		const AActor* Target = ActorPerceptionInfo.Target.Get();
		if (Target != nullptr)
		{
			const FVector TargetLocation = Target->GetActorLocation();
			for (const FAIStimulus& Stimulus : ActorPerceptionInfo.LastSensedStimuli)
			{
				const UAISenseConfig* SenseConfig = GetSenseConfig(Stimulus.Type);
				if (Stimulus.IsValid() && (Stimulus.IsExpired() == false) && SenseConfig)
				{
					const FString Description = FString::Printf(TEXT("%s: %.2f age:%.2f"), *SenseConfig->GetSenseName(), Stimulus.Strength, Stimulus.GetAge());
					const FColor DebugColor = SenseConfig->GetDebugColor();

					DebuggerCategory->AddShape(FGameplayDebuggerShape::MakePoint(Stimulus.StimulusLocation + FVector(0, 0, 30), 30.0f, DebugColor, Description));
					DebuggerCategory->AddShape(FGameplayDebuggerShape::MakeSegment(Stimulus.ReceiverLocation, Stimulus.StimulusLocation, DebugColor));
					DebuggerCategory->AddShape(FGameplayDebuggerShape::MakeSegment(TargetLocation, Stimulus.StimulusLocation, FColor::Black));
				}
			}
		}
	}

	for (const UAISenseConfig* SenseConfig : SensesConfig)
	{
		if (SenseConfig)
		{
			SenseConfig->DescribeSelfToGameplayDebugger(this, DebuggerCategory);
		}
	}
}
#endif // WITH_GAMEPLAY_DEBUGGER_MENU

#if ENABLE_VISUAL_LOG
void UAIPerceptionComponent::DescribeSelfToVisLog(FVisualLogEntry* Snapshot) const
{

}
#endif // ENABLE_VISUAL_LOG
