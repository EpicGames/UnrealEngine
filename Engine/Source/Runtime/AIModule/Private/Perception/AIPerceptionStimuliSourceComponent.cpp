// Copyright Epic Games, Inc. All Rights Reserved.

#include "Perception/AIPerceptionStimuliSourceComponent.h"
#include "Perception/AIPerceptionSystem.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AIPerceptionStimuliSourceComponent)


UAIPerceptionStimuliSourceComponent::UAIPerceptionStimuliSourceComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bAutoRegisterAsSource(false)
{
	bSuccessfullyRegistered = false;
}

void UAIPerceptionStimuliSourceComponent::OnRegister()
{
	Super::OnRegister();

#if WITH_EDITOR
	// when in the editor world we don't remove the null entries
	// since those can get changed to something else by the user
	if (!GIsEditor || GIsPlayInEditorWorld)
#endif // WITH_EDITOR
	{
		RegisterAsSourceForSenses.RemoveAllSwap([](const TSubclassOf<UAISense>& SenseClass) {
			return SenseClass == nullptr;
		});
	}

	if (bAutoRegisterAsSource)
	{
		RegisterWithPerceptionSystem();
	}
}

void UAIPerceptionStimuliSourceComponent::RegisterWithPerceptionSystem()
{
	if (bSuccessfullyRegistered)
	{
		return;
	}
	if (RegisterAsSourceForSenses.Num() == 0)
	{
		bSuccessfullyRegistered = true;
		return;
	}

	AActor* OwnerActor = GetOwner();
	if (OwnerActor == nullptr)
	{
		return;
	}

	UWorld* World = OwnerActor->GetWorld();
	if (World)
	{
		UAIPerceptionSystem* PerceptionSystem = UAIPerceptionSystem::GetCurrent(World);
		if (PerceptionSystem)
		{
			for (auto& SenseClass : RegisterAsSourceForSenses)
			{
				if(SenseClass)
				{
					PerceptionSystem->RegisterSourceForSenseClass(SenseClass, *OwnerActor);
					bSuccessfullyRegistered = true;
				}
				// we just ignore the empty entries
			}
		}
	}
}

void UAIPerceptionStimuliSourceComponent::RegisterForSense(TSubclassOf<UAISense> SenseClass)
{
	if (!SenseClass)
	{
		return;
	}

	AActor* OwnerActor = GetOwner();
	if (OwnerActor == nullptr)
	{
		return;
	}

	UWorld* World = OwnerActor->GetWorld();
	if (World)
	{
		UAIPerceptionSystem* PerceptionSystem = UAIPerceptionSystem::GetCurrent(World);
		if (PerceptionSystem)
		{
			UE_CVLOG(bSuccessfullyRegistered == false && RegisterAsSourceForSenses.Num(), OwnerActor, LogAIPerception, Warning
				, TEXT("Registering as stimuli source for sense %s while the UAIPerceptionStimuliSourceComponent has not registered with the AIPerceptionSystem just yet. This will result in posing as a source for only this one sense.")
				, *SenseClass->GetName());

			PerceptionSystem->RegisterSourceForSenseClass(SenseClass, *OwnerActor);
			RegisterAsSourceForSenses.AddUnique(SenseClass);
			bSuccessfullyRegistered = true;
		}
	}
}

void UAIPerceptionStimuliSourceComponent::UnregisterFromPerceptionSystem()
{
	if (bSuccessfullyRegistered == false)
	{
		return;
	}

	AActor* OwnerActor = GetOwner();
	if (OwnerActor == nullptr)
	{
		return;
	}

	UWorld* World = OwnerActor->GetWorld();
	if (World)
	{
		UAIPerceptionSystem* PerceptionSystem = UAIPerceptionSystem::GetCurrent(World);
		if (PerceptionSystem)
		{
			for (auto& SenseClass : RegisterAsSourceForSenses)
			{
				PerceptionSystem->UnregisterSource(*OwnerActor, SenseClass);
			}
		}
	}

	bSuccessfullyRegistered = false;
}

void UAIPerceptionStimuliSourceComponent::UnregisterFromSense(TSubclassOf<UAISense> SenseClass)
{
	if (!SenseClass)
	{
		return;
	}

	AActor* OwnerActor = GetOwner();
	if (OwnerActor == nullptr)
	{
		return;
	}

	UWorld* World = OwnerActor->GetWorld();
	if (World)
	{
		UAIPerceptionSystem* PerceptionSystem = UAIPerceptionSystem::GetCurrent(World);
		if (PerceptionSystem)
		{
			PerceptionSystem->UnregisterSource(*OwnerActor, SenseClass);
			RegisterAsSourceForSenses.RemoveSingleSwap(SenseClass, EAllowShrinking::No);
			bSuccessfullyRegistered = RegisterAsSourceForSenses.Num() > 0;
		}
	}
}

#if WITH_EDITOR
void UAIPerceptionStimuliSourceComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName NAME_RegisterAsSourceForSenses = GET_MEMBER_NAME_CHECKED(UAIPerceptionStimuliSourceComponent, RegisterAsSourceForSenses);

	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property != nullptr)
	{
		if (PropertyChangedEvent.Property->GetFName() == NAME_RegisterAsSourceForSenses)
		{
			if (PropertyChangedEvent.ChangeType == EPropertyChangeType::Duplicate)
			{
				const int32 ChangeAtIndex = PropertyChangedEvent.GetArrayIndex(NAME_RegisterAsSourceForSenses.ToString());
				if (ensure(ChangeAtIndex != INDEX_NONE))
				{
					// clear duplicate
					RegisterAsSourceForSenses[ChangeAtIndex] = nullptr;
				}
			}
			else if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet)
			{
				TArray<TSubclassOf<UAISense>> TmpCopy = RegisterAsSourceForSenses;
				RegisterAsSourceForSenses.Empty(RegisterAsSourceForSenses.Num());
				for (TSubclassOf<UAISense> Sense : TmpCopy)
				{
					if (Sense)
					{
						RegisterAsSourceForSenses.AddUnique(Sense);
					}
				}
			}
		}
	}
}
#endif // WITH_EDITOR

