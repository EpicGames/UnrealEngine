// Copyright Epic Games, Inc. All Rights Reserved.

#include "Perception/AIPerceptionSystem.h"

#include "AISystem.h"
#include "Engine/Engine.h"
#include "EngineGlobals.h"
#include "EngineUtils.h"
#include "GameFramework/Pawn.h"
#include "Perception/AIPerceptionComponent.h"
#include "Perception/AISenseConfig.h"
#include "Perception/AISenseEvent.h"
#include "Perception/AISense_Hearing.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "TimerManager.h"
#include "VisualLogger/VisualLogger.h"

#if WITH_GAMEPLAY_DEBUGGER_MENU
#include "GameplayDebuggerTypes.h"
#include "GameplayDebuggerCategory.h"
#endif // WITH_GAMEPLAY_DEBUGGER_MENU

#include UE_INLINE_GENERATED_CPP_BY_NAME(AIPerceptionSystem)

DECLARE_CYCLE_STAT(TEXT("Perception System"),STAT_AI_PerceptionSys,STATGROUP_AI);
DECLARE_CYCLE_STAT(TEXT("Perception System - Process Stim"),STAT_AI_Perception_ProcessStim,STATGROUP_AI);

DEFINE_LOG_CATEGORY(LogAIPerception);

//----------------------------------------------------------------------//
// UAISenseConfig
//----------------------------------------------------------------------//
FAISenseID UAISenseConfig::GetSenseID() const
{
	TSubclassOf<UAISense> SenseClass = GetSenseImplementation();
	return UAISense::GetSenseID(SenseClass);
}

//----------------------------------------------------------------------//
// UAIPerceptionSystem
//----------------------------------------------------------------------//
UAIPerceptionSystem::UAIPerceptionSystem(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, PerceptionAgingRate(0.3f)
	, bHandlePawnNotification(false)
	, NextStimuliAgingTick(0.)
	, CurrentTime(0.)
{
	StimuliSourceEndPlayDelegate.BindDynamic(this, &UAIPerceptionSystem::OnPerceptionStimuliSourceEndPlay);
}

FAISenseID UAIPerceptionSystem::RegisterSenseClass(TSubclassOf<UAISense> SenseClass)
{
	check(SenseClass);
	FAISenseID SenseID = UAISense::GetSenseID(SenseClass);
	if (SenseID.IsValid() == false)
	{
		UAISense* SenseCDO = GetMutableDefault<UAISense>(SenseClass);
		SenseID = SenseCDO->UpdateSenseID();

		if (SenseID.IsValid() == false)
		{
			// @todo log a message here
			return FAISenseID::InvalidID();
		}
	}

	if (SenseID.Index >= Senses.Num())
	{
		const int32 ItemsToAdd = SenseID.Index - Senses.Num() + 1;
		Senses.AddZeroed(ItemsToAdd);
	}

	if (Senses[SenseID] == nullptr)
	{
		Senses[SenseID] = NewObject<UAISense>(this, SenseClass);
		check(Senses[SenseID]);
		bHandlePawnNotification |= Senses[SenseID]->ShouldAutoRegisterAllPawnsAsSources() || Senses[SenseID]->WantsNewPawnNotification();

		if (Senses[SenseID]->ShouldAutoRegisterAllPawnsAsSources())
		{
			UWorld* World = GetWorld();
			if (World->HasBegunPlay())
			{
				// this @hack is required due to UAIPerceptionSystem::RegisterSenseClass
				// being potentially called from UAIPerceptionComponent::OnRegister
				// and at that point UWorld might not have registered 
				// the pawn related to given UAIPerceptionComponent.
				World->GetTimerManager().SetTimerForNextTick(FTimerDelegate::CreateUObject(this, &UAIPerceptionSystem::RegisterAllPawnsAsSourcesForSense, SenseID));
			}
			// otherwise it will get called in StartPlay()
		}

		// make senses v-log to perception system's log
		REDIRECT_OBJECT_TO_VLOG(Senses[SenseID], this);
		UE_VLOG(this, LogAIPerception, Log, TEXT("Registering sense %s"), *Senses[SenseID]->GetName());
	}

	return SenseID;
}

TStatId UAIPerceptionSystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UAIPerceptionSystem, STATGROUP_Tickables);
}

void UAIPerceptionSystem::RegisterSource(FAISenseID SenseID, AActor& SourceActor)
{
	ensure(IsSenseInstantiated(SenseID));
	SourcesToRegister.AddUnique(FPerceptionSourceRegistration(SenseID, &SourceActor));
}

void UAIPerceptionSystem::PerformSourceRegistration()
{
	SCOPE_CYCLE_COUNTER(STAT_AI_PerceptionSys);

	for (const FPerceptionSourceRegistration& PercSource : SourcesToRegister)
	{
		AActor* SourceActor = PercSource.Source.Get();
		if (SourceActor != nullptr && SourceActor->IsPendingKillPending() == false && Senses[PercSource.SenseID] != nullptr)
		{
			Senses[PercSource.SenseID]->RegisterSource(*SourceActor);

			// hook into notification about actor's EndPlay to remove it as a source 
			SourceActor->OnEndPlay.AddUnique(StimuliSourceEndPlayDelegate);

			// store information we have this actor as given sense's source
			FPerceptionStimuliSource& StimuliSource = RegisteredStimuliSources.FindOrAdd(SourceActor);
			StimuliSource.SourceActor = SourceActor;
			StimuliSource.RelevantSenses.AcceptChannel(PercSource.SenseID);
		}
	}

	SourcesToRegister.Reset();
}

void UAIPerceptionSystem::OnNewListener(const FPerceptionListener& NewListener)
{
	for (UAISense* const SenseInstance : Senses)
	{
		// @todo filter out the ones that do not declare using this sense
		if (SenseInstance != nullptr && NewListener.HasSense(SenseInstance->GetSenseID()))
		{
			SenseInstance->OnNewListener(NewListener);
		}
	}
}

void UAIPerceptionSystem::OnListenerUpdate(const FPerceptionListener& UpdatedListener)
{
	for (UAISense* const SenseInstance : Senses)
	{
		if (SenseInstance != nullptr)
		{
			SenseInstance->OnListenerUpdate(UpdatedListener);
		}
	}
}

void UAIPerceptionSystem::Tick(float DeltaSeconds)
{
	SCOPE_CYCLE_COUNTER(STAT_AI_PerceptionSys);
	SCOPE_CYCLE_COUNTER(STAT_AI_Overall);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(AIPerception);


	// if no new stimuli
	// and it's not time to remove stimuli from "know events"

	UWorld* World = GEngine->GetWorldFromContextObjectChecked(GetOuter());
	check(World);

	if (World->bPlayersOnly == false)
	{
		// cache it
		CurrentTime = World->GetTimeSeconds();
		
		if (SourcesToRegister.Num() > 0)
		{
			PerformSourceRegistration();
		}

		bool bSomeListenersNeedUpdateDueToStimuliAging = false;
		if (NextStimuliAgingTick <= CurrentTime)
		{
			constexpr double Precision = 1./64.;
			const float AgingDt = FloatCastChecked<float>(CurrentTime - NextStimuliAgingTick, Precision);
			bSomeListenersNeedUpdateDueToStimuliAging = AgeStimuli(PerceptionAgingRate + AgingDt);
			NextStimuliAgingTick = CurrentTime + PerceptionAgingRate;
		}

		bool bNeedsUpdate = false;
		for (UAISense* const SenseInstance : Senses)
		{
			bNeedsUpdate |= SenseInstance != nullptr && SenseInstance->ProgressTime(DeltaSeconds);
		}

		if (bNeedsUpdate)
		{
			// first update cached location of all listener, and remove invalid listeners
			for (AIPerception::FListenerMap::TIterator ListenerIt(ListenerContainer); ListenerIt; ++ListenerIt)
			{
				if (ListenerIt->Value.Listener.IsValid())
				{
					ListenerIt->Value.CacheLocation();
				}
				else
				{
					OnListenerRemoved(ListenerIt->Value);
					ListenerIt.RemoveCurrent();
				}
			}

			for (UAISense* const SenseInstance : Senses)
			{
				if (SenseInstance != nullptr)
				{
					SenseInstance->Tick();
				}
			}
		}
		{
			SCOPE_CYCLE_COUNTER(STAT_AI_Perception_ProcessStim);
			/** no point in sorting if no new stimuli was processed */
			const bool bStimuliDelivered = DeliverDelayedStimuli(bNeedsUpdate ? RequiresSorting : NoNeedToSort);

			if (bNeedsUpdate || bStimuliDelivered || bSomeListenersNeedUpdateDueToStimuliAging)
			{
				for (AIPerception::FListenerMap::TIterator ListenerIt(ListenerContainer); ListenerIt; ++ListenerIt)
				{
					check(ListenerIt->Value.Listener.IsValid());

					if (ListenerIt->Value.HasAnyNewStimuli())
					{
						ListenerIt->Value.ProcessStimuli();
					}
				}
			}
		}
	}
}

bool UAIPerceptionSystem::AgeStimuli(const float Amount)
{
	ensure(Amount >= 0.f);
	bool bTagged = false;

	for (AIPerception::FListenerMap::TIterator ListenerIt(ListenerContainer); ListenerIt; ++ListenerIt)
	{
		FPerceptionListener& Listener = ListenerIt->Value;
		if (Listener.Listener.IsValid())
		{
			// AgeStimuli will return true if this listener requires an update after stimuli aging
			if (Listener.Listener->AgeStimuli(Amount))
			{
				Listener.MarkForStimulusProcessing();
				bTagged = true;
			}
		}
	}
	return bTagged;
}

UAIPerceptionSystem* UAIPerceptionSystem::GetCurrent(UObject* WorldContextObject)
{
	UWorld* World = Cast<UWorld>(WorldContextObject);
	if (World == nullptr && WorldContextObject != nullptr)
	{
		World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	}

	if (World && World->GetAISystem())
	{
		UAISystem* AISys = CastChecked<UAISystem>(World->GetAISystem());

		return AISys->GetPerceptionSystem();
	}

	return nullptr;
}

UAIPerceptionSystem* UAIPerceptionSystem::GetCurrent(UWorld& World)
{
	if (World.GetAISystem())
	{
		check(Cast<UAISystem>(World.GetAISystem()));
		UAISystem* AISys = (UAISystem*)(World.GetAISystem());

		return AISys->GetPerceptionSystem();
	}

	return nullptr;
}

void UAIPerceptionSystem::UpdateListener(UAIPerceptionComponent& Listener)
{
	SCOPE_CYCLE_COUNTER(STAT_AI_PerceptionSys);

	if (!IsValid(&Listener))
	{
		UnregisterListener(Listener);
		return;
	}

	const FPerceptionListenerID ListenerId = Listener.GetListenerId();

	if (ListenerId != FPerceptionListenerID::InvalidID())
	{
		FPerceptionListener& ListenerEntry = ListenerContainer[ListenerId];
		ListenerEntry.UpdateListenerProperties(Listener);
		OnListenerUpdate(ListenerEntry);
	}
	else
	{			
		const FPerceptionListenerID NewListenerId = FPerceptionListenerID::GetNextID();
		Listener.StoreListenerId(NewListenerId);
		FPerceptionListener& ListenerEntry = ListenerContainer.Add(NewListenerId, FPerceptionListener(Listener));
		ListenerEntry.CacheLocation();
				
		OnNewListener(ListenerContainer[NewListenerId]);
	}
}

void UAIPerceptionSystem::OnListenerConfigUpdated(FAISenseID SenseID, const UAIPerceptionComponent& Listener)
{
	SCOPE_CYCLE_COUNTER(STAT_AI_PerceptionSys);

	if (!IsSenseInstantiated(SenseID))
	{
		UE_LOGF(LogAIPerception, Warning, "Sense must exist to update its sense config");
		return;
	}

	const FPerceptionListenerID ListenerId = Listener.GetListenerId();
	if (ListenerId == FPerceptionListenerID::InvalidID() || !ListenerContainer.Contains(ListenerId))
	{
		UE_LOGF(LogAIPerception, Warning, "Listener must have a valid id to update its sense config");
		return;
	}

	FPerceptionListener& ListenerEntry = ListenerContainer[ListenerId];
	check(ListenerEntry.Listener.IsValid() && ListenerEntry.Listener.Get() == &Listener);

	Senses[SenseID]->OnListenerConfigUpdated(ListenerEntry);
}

void UAIPerceptionSystem::UnregisterListener(UAIPerceptionComponent& Listener)
{
	SCOPE_CYCLE_COUNTER(STAT_AI_PerceptionSys);

	const FPerceptionListenerID ListenerId = Listener.GetListenerId();

	// can already be removed from ListenerContainer as part of cleaning up 
	// listeners with invalid WeakObjectPtr to UAIPerceptionComponent
	if (ListenerId != FPerceptionListenerID::InvalidID() && ListenerContainer.Contains(ListenerId))
	{
		check(ListenerContainer[ListenerId].Listener.IsValid() == false
			|| ListenerContainer[ListenerId].Listener.Get() == &Listener);
		OnListenerRemoved(ListenerContainer[ListenerId]);
		ListenerContainer.Remove(ListenerId);

		// mark it as unregistered
		Listener.StoreListenerId(FPerceptionListenerID::InvalidID());
	}
}

void UAIPerceptionSystem::UnregisterSource(AActor& SourceActor, const TSubclassOf<UAISense> Sense)
{
	// Log a message if it turns out the source actor was not registered nor pending registration
	bool bSourceWasKnown = false;
	
	FPerceptionStimuliSource* StimuliSource = RegisteredStimuliSources.Find(&SourceActor);
	if (StimuliSource)
	{
		// Source actor was registered
		bSourceWasKnown = true;

		// A single sense can be targeted, or Sense == null for all senses
		if (Sense)
		{
			// Unregister the source actor from a single sense
			const FAISenseID SenseID = UAISense::GetSenseID(Sense);
			if (IsSenseInstantiated(SenseID) && StimuliSource->RelevantSenses.ShouldRespondToChannel(Senses[SenseID]->GetSenseID()))
			{
				Senses[SenseID]->UnregisterSource(SourceActor);
				StimuliSource->RelevantSenses.FilterOutChannel(SenseID);
			}
		}
		else
		{
			// Unregister the source actor from all senses
			for (UAISense* const SenseInstance : Senses)
			{
				if (SenseInstance != nullptr && StimuliSource->RelevantSenses.ShouldRespondToChannel(SenseInstance->GetSenseID()))
				{
					SenseInstance->UnregisterSource(SourceActor);
				}
			}
			StimuliSource->RelevantSenses.Clear();
		}

		// If the source actor is no longer relevant for any senses, we can remove its stimuli source entry
		if (StimuliSource->RelevantSenses.IsEmpty())
		{
			SourceActor.OnEndPlay.Remove(StimuliSourceEndPlayDelegate);
			RegisteredStimuliSources.Remove(&SourceActor);
		}
	}
	
	// Remove this from any pending adds (add/remove same frame)
	for (int32 RemoveIndex = SourcesToRegister.Num() - 1; RemoveIndex >= 0; RemoveIndex--)
	{
		if (SourcesToRegister[RemoveIndex].Source == &SourceActor)
		{
			// Source actor was pending registration
			bSourceWasKnown = true;

			// A single sense can be targeted, or Sense == null for all senses
			if (!Sense || SourcesToRegister[RemoveIndex].SenseID == UAISense::GetSenseID(Sense))
			{
				SourcesToRegister.RemoveAt(RemoveIndex, EAllowShrinking::No);
			}
		}
	}

	// Log if SourceActor was not registered or pending registration for any sense
	if (!bSourceWasKnown)
	{
		UE_VLOG(this, LogAIPerception, Log, TEXT("UnregisterSource called for %s but it doesn't seem to be registered as a source"), *SourceActor.GetName());
	}
}

void UAIPerceptionSystem::OnListenerRemoved(const FPerceptionListener& NewListener)
{
	for (UAISense* const SenseInstance : Senses)
	{
		if (SenseInstance != nullptr && NewListener.HasSense(SenseInstance->GetSenseID()))
		{
			SenseInstance->OnListenerRemoved(NewListener);
		}
	}
}

void UAIPerceptionSystem::OnListenerForgetsActor(const UAIPerceptionComponent& Listener, AActor& ActorToForget)
{
	const FPerceptionListenerID ListenerId = Listener.GetListenerId();

	if (ListenerId != FPerceptionListenerID::InvalidID())
	{
		FPerceptionListener& ListenerEntry = ListenerContainer[ListenerId];
		
		for (UAISense* Sense : Senses)
		{
			if (Sense != nullptr && Sense->NeedsNotificationOnForgetting() && ListenerEntry.HasSense(Sense->GetSenseID()))
			{
				Sense->OnListenerForgetsActor(ListenerEntry, ActorToForget);
			}
		}
	}
}

void UAIPerceptionSystem::OnListenerForgetsAll(const UAIPerceptionComponent& Listener)
{
	const FPerceptionListenerID ListenerId = Listener.GetListenerId();

	if (ListenerId != FPerceptionListenerID::InvalidID())
	{
		FPerceptionListener& ListenerEntry = ListenerContainer[ListenerId];

		for (UAISense* Sense : Senses)
		{
			if (Sense != nullptr && Sense->NeedsNotificationOnForgetting() && ListenerEntry.HasSense(Sense->GetSenseID()))
			{
				Sense->OnListenerForgetsAll(ListenerEntry);
			}
		}
	}
}

void UAIPerceptionSystem::RegisterDelayedStimulus(FPerceptionListenerID ListenerId, float Delay, AActor* Instigator, const FAIStimulus& Stimulus)
{
	FDelayedStimulus DelayedStimulus;
	DelayedStimulus.DeliveryTimestamp = CurrentTime + Delay;
	DelayedStimulus.ListenerId = ListenerId;
	DelayedStimulus.Instigator = Instigator;
	DelayedStimulus.Stimulus = Stimulus;
	DelayedStimuli.Add(DelayedStimulus);
}

bool UAIPerceptionSystem::DeliverDelayedStimuli(UAIPerceptionSystem::EDelayedStimulusSorting Sorting)
{
	struct FTimestampSort
	{ 
		bool operator()(const FDelayedStimulus& A, const FDelayedStimulus& B) const
		{
			return A.DeliveryTimestamp < B.DeliveryTimestamp;
		}
	};

	if (DelayedStimuli.Num() <= 0)
	{
		return false;
	}

	if (Sorting == RequiresSorting)
	{
		DelayedStimuli.Sort(FTimestampSort());
	}

	int Index = 0;
	while (Index < DelayedStimuli.Num() && DelayedStimuli[Index].DeliveryTimestamp < CurrentTime)
	{
		FDelayedStimulus& DelayedStimulus = DelayedStimuli[Index];
		
		if (DelayedStimulus.ListenerId != FPerceptionListenerID::InvalidID() && ListenerContainer.Contains(DelayedStimulus.ListenerId))
		{
			FPerceptionListener& ListenerEntry = ListenerContainer[DelayedStimulus.ListenerId];
			// this has been already checked during tick, so if it's no longer the case then it's a bug
			check(ListenerEntry.Listener.IsValid());

			// deliver
			ListenerEntry.RegisterStimulus(DelayedStimulus.Instigator.Get(), DelayedStimulus.Stimulus);
		}

		++Index;
	}

	DelayedStimuli.RemoveAt(0, Index, EAllowShrinking::No);

	return Index > 0;
}

void UAIPerceptionSystem::MakeNoiseImpl(AActor* NoiseMaker, float Loudness, APawn* NoiseInstigator, const FVector& NoiseLocation, float MaxRange, FName Tag)
{
	UE_CLOGF(NoiseMaker == nullptr && NoiseInstigator == nullptr, LogAIPerception, Warning
		, "UAIPerceptionSystem::MakeNoiseImpl called with both NoiseMaker and NoiseInstigator being null. Unable to resolve UWorld context!");
	
	UWorld* World = NoiseMaker ? NoiseMaker->GetWorld() : (NoiseInstigator ? NoiseInstigator->GetWorld() : nullptr);

	if (World)
	{
		UAIPerceptionSystem::OnEvent(World, FAINoiseEvent(NoiseInstigator ? NoiseInstigator : NoiseMaker
			, NoiseLocation
			, Loudness
			, MaxRange
			, Tag));
	}
}

void UAIPerceptionSystem::OnNewPawn(APawn& Pawn)
{
	if (bHandlePawnNotification == false)
	{
		return;
	}

	for (UAISense* Sense : Senses)
	{
		if (Sense == nullptr)
		{
			continue;
		}

		if (Sense->WantsNewPawnNotification())
		{
			Sense->OnNewPawn(Pawn);
		}

		if (Sense->ShouldAutoRegisterAllPawnsAsSources())
		{
			FAISenseID SenseID = Sense->GetSenseID();
			check(IsSenseInstantiated(SenseID));
			RegisterSource(SenseID, Pawn);
		}
	}
}

void UAIPerceptionSystem::RegisterSource(AActor& SourceActor)
{
	for (UAISense* Sense : Senses)
	{
		if (Sense == nullptr)
		{
			continue;
		}

		const FAISenseID SenseID = Sense->GetSenseID();
		if (IsSenseInstantiated(SenseID))
		{
			RegisterSource(SenseID, SourceActor);
		}
	}
}

void UAIPerceptionSystem::StartPlay()
{
	for (UAISense* Sense : Senses)
	{
		if (Sense != nullptr && Sense->ShouldAutoRegisterAllPawnsAsSources())
		{
			FAISenseID SenseID = Sense->GetSenseID();
			RegisterAllPawnsAsSourcesForSense(SenseID);
		}
	}

	UWorld* World = GetWorld();
	NextStimuliAgingTick = World ? World->GetTimeSeconds() : 0.;
}

void UAIPerceptionSystem::RegisterAllPawnsAsSourcesForSense(FAISenseID SenseID)
{
	UWorld* World = GetWorld();
	for (TActorIterator<APawn> PawnIt(World); PawnIt; ++PawnIt)
	{
		RegisterSource(SenseID, **PawnIt);
	}
}

bool UAIPerceptionSystem::RegisterPerceptionStimuliSource(UObject* WorldContextObject, TSubclassOf<UAISense> Sense, AActor* Target)
{
	bool bResult = false;
	if (Sense && Target)
	{
		UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
		if (World && World->GetAISystem())
		{
			UAISystem* AISys = Cast<UAISystem>(World->GetAISystem());
			if (AISys != nullptr && AISys->GetPerceptionSystem() != nullptr)
			{
				// just a cache
				UAIPerceptionSystem* PerceptionSys = AISys->GetPerceptionSystem();
				
				PerceptionSys->RegisterSourceForSenseClass(Sense, *Target);

				bResult = true;
			}
		}
	}

	return bResult;
}

void UAIPerceptionSystem::RegisterSourceForSenseClass(TSubclassOf<UAISense> Sense, AActor& Target)	
{
	FAISenseID SenseID = UAISense::GetSenseID(Sense);
	if (IsSenseInstantiated(SenseID) == false)
	{
		SenseID = RegisterSenseClass(Sense);
	}

	RegisterSource(SenseID, Target);
}

void UAIPerceptionSystem::OnPerceptionStimuliSourceEndPlay(AActor* Actor, EEndPlayReason::Type EndPlayReason)
{
	UnregisterSource(*Actor);
}

TSubclassOf<UAISense> UAIPerceptionSystem::GetSenseClassForStimulus(UObject* WorldContextObject, const FAIStimulus& Stimulus)
{
	TSubclassOf<UAISense> Result = nullptr;
	UAIPerceptionSystem* PercSys = GetCurrent(WorldContextObject);
	if (PercSys && PercSys->Senses.IsValidIndex(Stimulus.Type) && PercSys->Senses[Stimulus.Type] != nullptr)
	{
		Result = PercSys->Senses[Stimulus.Type]->GetClass();
	}

	return Result;
}

//----------------------------------------------------------------------//
// Blueprint API
//----------------------------------------------------------------------//
void UAIPerceptionSystem::ReportEvent(UAISenseEvent* PerceptionEvent)
{
	if (PerceptionEvent)
	{
		const FAISenseID SenseID = PerceptionEvent->GetSenseID();
		if (SenseID.IsValid() && Senses.IsValidIndex(SenseID) && Senses[SenseID] != nullptr)
		{
			Senses[SenseID]->RegisterWrappedEvent(*PerceptionEvent);
		}
		else
		{
			UE_VLOG(this, LogAIPerception, Log, TEXT("Skipping perception event %s since related sense class has not been registered (no listeners)")
				, *PerceptionEvent->GetName());
			PerceptionEvent->DrawToVLog(*this);
		}
	}
}

void UAIPerceptionSystem::ReportPerceptionEvent(UObject* WorldContextObject, UAISenseEvent* PerceptionEvent)
{
	UAIPerceptionSystem* PerceptionSys = GetCurrent(WorldContextObject);
	if (PerceptionSys != nullptr)
	{
		PerceptionSys->ReportEvent(PerceptionEvent);
	}
}

//----------------------------------------------------------------------//
// Debug
//----------------------------------------------------------------------//
#if WITH_GAMEPLAY_DEBUGGER_MENU
void UAIPerceptionSystem::DescribeSelfToGameplayDebugger(FGameplayDebuggerCategory& DebuggerCategory) const
{
	DebuggerCategory.AddTextLine(FString::Printf(TEXT("%d Listeners"), ListenerContainer.Num()));

	for (const UAISense* Sense : Senses)
	{
		if (Sense)
		{
			Sense->DescribeSelfToGameplayDebugger(*this, DebuggerCategory);
		}
	}
}
#endif // WITH_GAMEPLAY_DEBUGGER_MENU