// Copyright Epic Games, Inc. All Rights Reserved.

#include "Perception/AIPerceptionTypes.h"
#include "Perception/AIPerceptionComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AIPerceptionTypes)

template<>
FAISenseCounter FAINamedID<FAISenseCounter>::Counter = FAISenseCounter();

template<>
FPerceptionListenerCounter FAIGenericID<FPerceptionListenerCounter>::Counter = FPerceptionListenerCounter();

//----------------------------------------------------------------------//
// FAIStimulus
//----------------------------------------------------------------------//

// mind that this needs to be > 0 since checks are done if Age < FAIStimulus::NeverHappenedAge
// @todo maybe should be a function (IsValidAge)
const float FAIStimulus::NeverHappenedAge = FLT_MAX;

FAIStimulus::FAIStimulus(const UAISense& Sense, float StimulusStrength, const FVector& InStimulusLocation, const FVector& InReceiverLocation, FResult Result, FName InStimulusTag)
	: Age(0.f)
	, ExpirationAge(FAIStimulus::NeverHappenedAge)
	, Strength(Result == SensingSucceeded ? StimulusStrength : -1.f)
	, StimulusLocation(InStimulusLocation)
	, ReceiverLocation(InReceiverLocation)
	, Tag(InStimulusTag)
	, bWantsToNotifyOnlyOnValueChange(Sense.WantsUpdateOnlyOnPerceptionValueChange())
	, bSuccessfullySensed(Result == SensingSucceeded)
	, bExpired(false)
{
	Type = Sense.GetSenseID();
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
FString FAIStimulus::GetDebugDescription() const
{
	if (IsValid() == false)
	{
		return TEXT("Uninitialized");
	}
	
	if (bExpired)
	{
		return FString::Printf(TEXT("%s Expired"), *Type.Name.ToString());
	}

	return FString::Printf(TEXT("%s %s %.1fs ago at %s")
		, *Type.Name.ToString()
		, bSuccessfullySensed ? TEXT("+") : TEXT("-")
		, Age
		, *StimulusLocation.ToString()
	);
}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

//----------------------------------------------------------------------//
// FPerceptionListener
//----------------------------------------------------------------------//
const FPerceptionListener FPerceptionListener::NullListener;

FPerceptionListener::FPerceptionListener()
	: CachedLocation(FVector::ZeroVector)
	, CachedDirection(FVector::UpVector)
	, bHasStimulusToProcess(false)
	, ListenerID(FPerceptionListenerID::InvalidID())
{

}

FPerceptionListener::FPerceptionListener(UAIPerceptionComponent& InListener) 
	: Listener(&InListener)
	, CachedLocation(FVector::ZeroVector)
	, CachedDirection(FVector::UpVector)
	, bHasStimulusToProcess(false)
	, ListenerID(FPerceptionListenerID::InvalidID())
{
	UpdateListenerProperties(InListener);
	ListenerID = InListener.GetListenerId();
}

void FPerceptionListener::CacheLocation()
{
	if (Listener.IsValid())
	{
		Listener->GetLocationAndDirection(CachedLocation, CachedDirection);
	}
}

void FPerceptionListener::UpdateListenerProperties(UAIPerceptionComponent& InListener)
{
	verify(&InListener == Listener.Get());

	// using InListener rather then Listener to avoid slight overhead of TWeakObjectPtr
	TeamIdentifier = InListener.GetTeamIdentifier();
	Filter = InListener.GetPerceptionFilter();
}

void FPerceptionListener::RegisterStimulus(AActor* Source, const FAIStimulus& Stimulus)
{
	bHasStimulusToProcess = true;
	Listener->RegisterStimulus(Source, Stimulus);
}

void FPerceptionListener::ProcessStimuli()
{
	ensure(bHasStimulusToProcess);
	Listener->ProcessStimuli();
	bHasStimulusToProcess = false;
}

FName FPerceptionListener::GetBodyActorName() const 
{
	const AActor* OwnerActor = Listener.IsValid() ? Listener->GetBodyActor() : NULL;
	return OwnerActor ? OwnerActor->GetFName() : NAME_None;
}

uint32 FPerceptionListener::GetBodyActorUniqueID() const
{
	const AActor* OwnerActor = Listener.IsValid() ? Listener->GetBodyActor() : nullptr;
	return OwnerActor ? OwnerActor->GetUniqueID() : FAISystem::InvalidUnsignedID;
}

const AActor* FPerceptionListener::GetBodyActor() const 
{ 
	return Listener.IsValid() ? Listener->GetBodyActor() : NULL; 
}

const IGenericTeamAgentInterface* FPerceptionListener::GetTeamAgent() const
{
	const UAIPerceptionComponent* PercComponent = Listener.Get();
	if (PercComponent == NULL)
	{	// This could be NULL if the Listener is pending kill; in order to get the pointer ANYWAY, we'd need to use
		// Listener.Get(true) instead.  This issue was hit when using KillPawns cheat at the same moment that pawns
		// were spawning into the world.
		return NULL;
	}

	const AActor* OwnerActor = PercComponent->GetOwner();
	const IGenericTeamAgentInterface* OwnerTeamAgent = Cast<const IGenericTeamAgentInterface>(OwnerActor);
	return OwnerTeamAgent != NULL ? OwnerTeamAgent : Cast<const IGenericTeamAgentInterface>(PercComponent->GetBodyActor());
}


