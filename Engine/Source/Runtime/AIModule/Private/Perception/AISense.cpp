// Copyright Epic Games, Inc. All Rights Reserved.

#include "Perception/AISense.h"
#include "Perception/AIPerceptionSystem.h"
#include "Perception/AISenseConfig.h"
#include "VisualLogger/VisualLogger.h"
#include "Perception/AISenseConfig_Blueprint.h"
#include "Perception/AISenseConfig_Hearing.h"
#include "Perception/AISenseConfig_Prediction.h"
#include "Perception/AISenseConfig_Sight.h"
#include "Perception/AISenseConfig_Team.h"
#include "Perception/AISenseConfig_Touch.h"
#include "Perception/AIPerceptionComponent.h"
#include "Perception/AISense_Blueprint.h"
#include "Perception/AISense_Prediction.h"
#include "Perception/AISense_Touch.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AISense)

#if WITH_GAMEPLAY_DEBUGGER_MENU
#include "GameplayDebuggerTypes.h"
#include "GameplayDebuggerCategory.h"
#endif // WITH_GAMEPLAY_DEBUGGER_MENU

const float UAISense::SuspendNextUpdate = FLT_MAX;

UAISense::UAISense(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, TimeUntilNextUpdate(SuspendNextUpdate)
	, SenseID(FAISenseID::InvalidID())
{
	bNeedsForgettingNotification = false;

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		SenseID = ((const UAISense*)GetClass()->GetDefaultObject())->GetSenseID();
	}
}

UWorld* UAISense::GetWorld() const
{
	return PerceptionSystemInstance ? PerceptionSystemInstance->GetWorld() : nullptr;
}

void UAISense::HardcodeSenseID(TSubclassOf<UAISense> SenseClass, FAISenseID HardcodedID)
{
	UAISense* MutableCDO = GetMutableDefault<UAISense>(SenseClass);
	check(MutableCDO);
	MutableCDO->SenseID = HardcodedID;
}

void UAISense::PostInitProperties() 
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject) == false) 
	{
		PerceptionSystemInstance = Cast<UAIPerceptionSystem>(GetOuter());
	}
}

AIPerception::FListenerMap* UAISense::GetListeners() 
{
	check(PerceptionSystemInstance);
	return &(PerceptionSystemInstance->GetListenersMap());
}

void UAISense::OnNewPawn(APawn& NewPawn)
{
	if (WantsNewPawnNotification())
	{
		UE_VLOG(GetPerceptionSystem(), LogAIPerception, Warning
			, TEXT("%s declars it needs New Pawn notification but does not override OnNewPawn"), *GetName());		
	}		
}

void UAISense::SetSenseID(FAISenseID Index)
{
	check(Index != FAISenseID::InvalidID());
	SenseID = Index;
}

void UAISense::ForceSenseID(FAISenseID InSenseID)
{
	check(GetClass()->HasAnyClassFlags(CLASS_CompiledFromBlueprint) == true);
	ensure(GetClass()->HasAnyClassFlags(CLASS_Abstract) == false);
	
	SenseID = InSenseID;
}

FAISenseID UAISense::UpdateSenseID()
{
	check(HasAnyFlags(RF_ClassDefaultObject) == true && GetClass()->HasAnyClassFlags(CLASS_Abstract | CLASS_CompiledFromBlueprint) == false);

	if (SenseID.IsValid() == false)
	{
		SenseID = FAISenseID(GetFName());
	}

	return SenseID;
}

void UAISense::RegisterWrappedEvent(UAISenseEvent& PerceptionEvent)
{
	UE_VLOG(GetPerceptionSystem(), LogAIPerception, Error, TEXT("%s did not override UAISense::RegisterWrappedEvent!"), *GetName());
}

#if WITH_GAMEPLAY_DEBUGGER_MENU
void UAISense::DescribeSelfToGameplayDebugger(const UAIPerceptionSystem& PerceptionSystem, FGameplayDebuggerCategory& DebuggerCategory) const
{
	DebuggerCategory.AddTextLine(FString::Printf(TEXT("%s"), *SenseID.Name.ToString()));
}
#endif // WITH_GAMEPLAY_DEBUGGER_MENU

//----------------------------------------------------------------------//
// 
//----------------------------------------------------------------------//
UAISenseConfig::UAISenseConfig(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer), DebugColor(FColor::White), bStartsEnabled(true)
{
}

TSubclassOf<UAISense> UAISenseConfig::GetSenseImplementation() const 
{ 
	return UAISense::StaticClass(); 
}

FString UAISenseConfig::GetSenseName() const
{
	if (CachedSenseName.Len() == 0)
	{
		CachedSenseName = GetSenseImplementation()->GetName();
		CachedSenseName.RemoveFromEnd(TEXT("_C"));

		int32 SeparatorIdx = INDEX_NONE;
		const bool bHasSeparator = CachedSenseName.FindLastChar(TEXT('_'), SeparatorIdx);
		if (bHasSeparator)
		{
			CachedSenseName.MidInline(SeparatorIdx + 1, MAX_int32, EAllowShrinking::No);
		}
	}

	return CachedSenseName;
}

#if WITH_GAMEPLAY_DEBUGGER_MENU
static FString DescribeColorHelper(const FColor& Color)
{
	const int32 MaxColors = GColorList.GetColorsNum();
	for (int32 Idx = 0; Idx < MaxColors; Idx++)
	{
		if (Color == GColorList.GetFColorByIndex(Idx))
		{
			return GColorList.GetColorNameByIndex(Idx);
		}
	}

	return FString("color");
}

void UAISenseConfig::DescribeSelfToGameplayDebugger(const UAIPerceptionComponent* PerceptionComponent, FGameplayDebuggerCategory* DebuggerCategory) const
{
	if (DebuggerCategory)
	{
		DebuggerCategory->AddTextLine(
			FString::Printf(TEXT("%s: {%s}%s"), *GetSenseName(), *GetDebugColor().ToString(), *DescribeColorHelper(GetDebugColor()))
			);
	}
}
#endif // WITH_GAMEPLAY_DEBUGGER_MENU

//----------------------------------------------------------------------//
// 
//----------------------------------------------------------------------//
UAISenseConfig_Sight::UAISenseConfig_Sight(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	DebugColor = FColor::Green;
}

TSubclassOf<UAISense> UAISenseConfig_Sight::GetSenseImplementation() const 
{ 
	return *Implementation; 
}	

#if WITH_EDITOR
void UAISenseConfig_Sight::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	static const FName NAME_AutoSuccessRangeFromLastSeenLocation = GET_MEMBER_NAME_CHECKED(UAISenseConfig_Sight, AutoSuccessRangeFromLastSeenLocation);

	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		const FName PropName = PropertyChangedEvent.Property->GetFName();
		if (PropName == NAME_AutoSuccessRangeFromLastSeenLocation)
		{
			if (AutoSuccessRangeFromLastSeenLocation < 0)
			{
				AutoSuccessRangeFromLastSeenLocation = FAISystem::InvalidRange;
			}
		}
	}
}
#endif // WITH_EDITOR

#if WITH_GAMEPLAY_DEBUGGER_MENU
void UAISenseConfig_Sight::DescribeSelfToGameplayDebugger(const UAIPerceptionComponent* PerceptionComponent, FGameplayDebuggerCategory* DebuggerCategory) const
{
	if (PerceptionComponent == nullptr || DebuggerCategory == nullptr)
	{
		return;
	}

	FColor SightRangeColor = FColor::Green;
	FColor LoseSightRangeColor = FColorList::NeonPink;

	// don't call Super implementation on purpose, replace color description line
	DebuggerCategory->AddTextLine(
		FString::Printf(TEXT("%s: {%s}%s {white}rangeIN:{%s} %.2f (%s) {white} rangeOUT:{%s} %.2f (%s)"), *GetSenseName(),
			*GetDebugColor().ToString(), *DescribeColorHelper(GetDebugColor()),
			*SightRangeColor.ToString(), SightRadius, *DescribeColorHelper(SightRangeColor),
			*LoseSightRangeColor.ToString(), LoseSightRadius, *DescribeColorHelper(LoseSightRangeColor))
		);

	const AActor* BodyActor = PerceptionComponent->GetBodyActor();
	if (BodyActor != nullptr)
	{
		FVector BodyLocation, BodyFacing;
		PerceptionComponent->GetLocationAndDirection(BodyLocation, BodyFacing);

		DebuggerCategory->AddShape(FGameplayDebuggerShape::MakeCylinder(BodyLocation, LoseSightRadius, 25.0f, LoseSightRangeColor));
		DebuggerCategory->AddShape(FGameplayDebuggerShape::MakeCylinder(BodyLocation, SightRadius, 25.0f, SightRangeColor));

		const float SightPieLength = FMath::Max(LoseSightRadius, SightRadius) + PointOfViewBackwardOffset;
		const FVector RootLocation = BodyLocation - (BodyFacing * PointOfViewBackwardOffset);
		const FVector LeftDirection = BodyFacing.RotateAngleAxis(PeripheralVisionAngleDegrees, FVector::UpVector);
		const FVector RightDirection = BodyFacing.RotateAngleAxis(-PeripheralVisionAngleDegrees, FVector::UpVector);
		DebuggerCategory->AddShape(FGameplayDebuggerShape::MakeSegment(RootLocation + (BodyFacing * NearClippingRadius), RootLocation + (BodyFacing * SightPieLength), SightRangeColor));
		DebuggerCategory->AddShape(FGameplayDebuggerShape::MakeSegment(RootLocation + (LeftDirection * NearClippingRadius), RootLocation + (LeftDirection * SightPieLength), SightRangeColor));
		DebuggerCategory->AddShape(FGameplayDebuggerShape::MakeSegment(RootLocation + (RightDirection * NearClippingRadius), RootLocation + (RightDirection * SightPieLength), SightRangeColor));
		DebuggerCategory->AddShape(FGameplayDebuggerShape::MakeSegment(RootLocation + (LeftDirection * NearClippingRadius), RootLocation + (BodyFacing * NearClippingRadius), SightRangeColor));
		DebuggerCategory->AddShape(FGameplayDebuggerShape::MakeSegment(RootLocation + (BodyFacing * NearClippingRadius), RootLocation + (RightDirection * NearClippingRadius), SightRangeColor));
	}
}
#endif // WITH_GAMEPLAY_DEBUGGER_MENU

//----------------------------------------------------------------------//
// UAISenseConfig_Hearing
//----------------------------------------------------------------------//

UAISenseConfig_Hearing::UAISenseConfig_Hearing(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer), HearingRange(3000.f)
{
	DebugColor = FColor::Yellow;
}

TSubclassOf<UAISense> UAISenseConfig_Hearing::GetSenseImplementation() const 
{ 
	return *Implementation; 
}

#if WITH_GAMEPLAY_DEBUGGER_MENU
void UAISenseConfig_Hearing::DescribeSelfToGameplayDebugger(const UAIPerceptionComponent* PerceptionComponent, FGameplayDebuggerCategory* DebuggerCategory) const
{
	if (PerceptionComponent == nullptr || DebuggerCategory == nullptr)
	{
		return;
	}

	FColor HearingRangeColor = FColor::Yellow;
	FColor LoSHearingRangeColor = FColorList::Cyan;

	// don't call Super implementation on purpose, replace color description line
	DebuggerCategory->AddTextLine(
		FString::Printf(TEXT("%s: {%s}%s {white}range:{%s}%s {white} rangeLoS:{%s}%s"), *GetSenseName(),
			*GetDebugColor().ToString(), *DescribeColorHelper(GetDebugColor()),
			*HearingRangeColor.ToString(), *DescribeColorHelper(HearingRangeColor),
			*LoSHearingRangeColor.ToString(), *DescribeColorHelper(LoSHearingRangeColor))
		);

	const AActor* BodyActor = PerceptionComponent->GetBodyActor();
	if (BodyActor != nullptr)
	{
		FVector OwnerLocation = BodyActor->GetActorLocation();
		
		DebuggerCategory->AddShape(FGameplayDebuggerShape::MakeCylinder(OwnerLocation, HearingRange, 25.0f, HearingRangeColor));
	}
}
#endif // WITH_GAMEPLAY_DEBUGGER_MENU

//----------------------------------------------------------------------//
// UAISenseConfig_Prediction
//----------------------------------------------------------------------//
UAISenseConfig_Prediction::UAISenseConfig_Prediction(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	DebugColor = FColorList::Grey;
}

TSubclassOf<UAISense> UAISenseConfig_Prediction::GetSenseImplementation() const 
{ 
	return UAISense_Prediction::StaticClass(); 
}


UAISenseConfig_Blueprint::UAISenseConfig_Blueprint(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	DebugColor = FColorList::Grey;
}

TSubclassOf<UAISense> UAISenseConfig_Blueprint::GetSenseImplementation() const
{
	return *Implementation;
}

UAISenseConfig_Team::UAISenseConfig_Team(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	DebugColor = FColor::Blue;
}

UAISenseConfig_Touch::UAISenseConfig_Touch(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	DebugColor = FColor::Cyan;
}

TSubclassOf<UAISense> UAISenseConfig_Touch::GetSenseImplementation() const
{
	return UAISense_Touch::StaticClass();
}

