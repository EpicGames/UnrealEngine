// Copyright Epic Games, Inc. All Rights Reserved.

#include "Perception/AISenseConfig_Damage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AISenseConfig_Damage)

UAISenseConfig_Damage::UAISenseConfig_Damage(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer) 
{
	DebugColor = FColor::Red;
}

TSubclassOf<UAISense> UAISenseConfig_Damage::GetSenseImplementation() const
{
	return Implementation;
}

