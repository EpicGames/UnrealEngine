// Copyright Epic Games, Inc. All Rights Reserved.

#include "Perception/AISenseEvent.h"
#include "Perception/AISense.h"
#include "Perception/AISense_Hearing.h"
#include "Perception/AISenseEvent_Hearing.h"
#include "Perception/AISense_Damage.h"
#include "Perception/AISenseEvent_Damage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AISenseEvent)

//----------------------------------------------------------------------//
// UAISenseEvent_Hearing
//----------------------------------------------------------------------//
UAISenseEvent_Hearing::UAISenseEvent_Hearing(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

FAISenseID UAISenseEvent_Hearing::GetSenseID() const
{
	return UAISense::GetSenseID<UAISense_Hearing>();
}

//----------------------------------------------------------------------//
// UAISenseEvent_Damage
//----------------------------------------------------------------------//
FAISenseID UAISenseEvent_Damage::GetSenseID() const
{
	return UAISense::GetSenseID<UAISense_Damage>();
}

