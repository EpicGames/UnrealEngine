// Copyright Epic Games, Inc. All Rights Reserved.

#include "Perception/AISense_Blueprint.h"
#include "BlueprintNodeHelpers.h"
#include "Perception/AIPerceptionComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AISense_Blueprint)

#if WITH_EDITOR
#include "Kismet2/KismetEditorUtilities.h"
#endif // WITH_EDITOR

//----------------------------------------------------------------------//
// 
//----------------------------------------------------------------------//
TMap<FNameEntryId, FAISenseID> UAISense_Blueprint::BPSenseToSenseID;

UAISense_Blueprint::UAISense_Blueprint(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		UpdateSenseID();

		UClass* StopAtClass = UAISense_Blueprint::StaticClass();
		OnNewListenerDelegate.BindUObject(this, &UAISense_Blueprint::OnNewListenerImpl);
		OnListenerRemovedDelegate.BindUObject(this, &UAISense_Blueprint::OnListenerRemovedImpl);
		// update is optional
		if (BlueprintNodeHelpers::HasBlueprintFunction(TEXT("OnListenerUpdated"), *this, *StopAtClass))
		{
			OnListenerUpdateDelegate.BindUObject(this, &UAISense_Blueprint::OnListenerUpdateImpl);
		}
	}
}

void UAISense_Blueprint::OnNewListenerImpl(const FPerceptionListener& NewListener)
{
	UAIPerceptionComponent* PercComp = NewListener.Listener.Get();
	if (PercComp)
	{
		ListenerContainer.AddUnique(PercComp);
		OnListenerRegistered(PercComp->GetOwner(), PercComp);

		RequestImmediateUpdate();
	}
}

void UAISense_Blueprint::OnListenerUpdateImpl(const FPerceptionListener& UpdatedListener)
{
	UAIPerceptionComponent* PercComp = UpdatedListener.Listener.Get();
	if (PercComp)
	{
		OnListenerUpdated(PercComp->GetOwner(), PercComp);
	}
}

void UAISense_Blueprint::OnListenerRemovedImpl(const FPerceptionListener& UpdatedListener)
{
	UAIPerceptionComponent* PercComp = UpdatedListener.Listener.Get();
	if (PercComp)
	{
		ListenerContainer.RemoveSingleSwap(PercComp);
		OnListenerUnregistered(PercComp->GetOwner(), PercComp);
	}
}

void UAISense_Blueprint::RegisterWrappedEvent(UAISenseEvent& PerceptionEvent)
{
	UnprocessedEvents.Add(&PerceptionEvent);

	RequestImmediateUpdate();
}

void UAISense_Blueprint::OnNewPawn(APawn& NewPawn)
{
	K2_OnNewPawn(&NewPawn);
}

float UAISense_Blueprint::Update()
{
	const float TimeUntilUpdate = OnUpdate(UnprocessedEvents);
	UnprocessedEvents.Reset();
	return TimeUntilUpdate;
}

FAISenseID UAISense_Blueprint::UpdateSenseID()
{
#if WITH_EDITOR
	// ignore skeleton and "old version"-classes
	if (FKismetEditorUtilities::IsClassABlueprintSkeleton(GetClass()) || GetClass()->HasAnyClassFlags(CLASS_NewerVersionExists)
		|| (GetOutermost() == GetTransientPackage()))
	{
		return FAISenseID::InvalidID();
	}
#endif
	if (GetClass()->HasAnyClassFlags(CLASS_Abstract) == false)
	{
		const FNameEntryId NameIndex = GetClass()->GetFName().GetDisplayIndex();
		const FAISenseID* StoredID = BPSenseToSenseID.Find(NameIndex);
		if (StoredID != nullptr)
		{
			ForceSenseID(*StoredID);
		}
		else
		{
			const FAISenseID NewSenseID = FAISenseID(GetFName());
			ForceSenseID(NewSenseID);
			BPSenseToSenseID.Add(NameIndex, GetSenseID());
		}
	}

	return GetSenseID();
}

void UAISense_Blueprint::GetAllListenerActors(TArray<AActor*>& OutListenerActors) const
{
	OutListenerActors.Reserve(OutListenerActors.Num() + ListenerContainer.Num());
	for (auto Listener : ListenerContainer)
	{
		AActor* ActorOwner = Listener->GetOwner();
		OutListenerActors.Add(ActorOwner);
	}
}

void UAISense_Blueprint::GetAllListenerComponents(TArray<UAIPerceptionComponent*>& OutListenerComponents) const
{
	OutListenerComponents = ListenerContainer;
}

