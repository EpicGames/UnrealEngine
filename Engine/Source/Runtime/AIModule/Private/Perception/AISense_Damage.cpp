// Copyright Epic Games, Inc. All Rights Reserved.

#include "Perception/AISense_Damage.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/Controller.h"
#include "Perception/AIPerceptionListenerInterface.h"
#include "Perception/AIPerceptionSystem.h"
#include "Perception/AIPerceptionComponent.h"
#include "Perception/AISenseEvent_Damage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AISense_Damage)

//----------------------------------------------------------------------//
// 
//----------------------------------------------------------------------//
FAIDamageEvent::FAIDamageEvent()
	: Amount(1.f), Location(FAISystem::InvalidLocation), HitLocation(FAISystem::InvalidLocation)
	, DamagedActor(nullptr), Instigator(nullptr), Tag(NAME_None)
{}

FAIDamageEvent::FAIDamageEvent(AActor* InDamagedActor, AActor* InInstigator, float DamageAmount, const FVector& EventLocation, const FVector& InHitLocation/* = FAISystem::InvalidLocation*/, FName InTag/* = NAME_None*/)
	: Amount(DamageAmount), Location(EventLocation), HitLocation(InHitLocation), DamagedActor(InDamagedActor), Instigator(InInstigator), Tag(InTag)
{
	Compile();
}

void FAIDamageEvent::Compile()
{
	if (DamagedActor == nullptr)
	{
		// nothing to do here, this event is invalid
		return;
	}

	const bool bHitLocationValid = FAISystem::IsValidLocation(HitLocation);
	const bool bEventLocationValid = FAISystem::IsValidLocation(Location);

	if (bHitLocationValid != bEventLocationValid)
	{
		if (bHitLocationValid)
		{
			HitLocation = Location;
		}
		else
		{
			Location = HitLocation;
		}
	}
	// both invalid
	else if ((bHitLocationValid || bEventLocationValid) == false)
	{
		HitLocation = Location = DamagedActor->GetActorLocation();
	}

}

IAIPerceptionListenerInterface* FAIDamageEvent::GetDamagedActorAsPerceptionListener() const
{
	IAIPerceptionListenerInterface* Listener = nullptr;
	if (DamagedActor)
	{
		Listener = Cast<IAIPerceptionListenerInterface>(DamagedActor);
		if (Listener == nullptr)
		{
			APawn* ListenerAsPawn = Cast<APawn>(DamagedActor);
			if (ListenerAsPawn)
			{
				Listener = Cast<IAIPerceptionListenerInterface>(ListenerAsPawn->GetController());
			}
		}
	}
	return Listener;
}
//----------------------------------------------------------------------//
// 
//----------------------------------------------------------------------//
UAISense_Damage::UAISense_Damage(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

float UAISense_Damage::Update()
{
	AIPerception::FListenerMap& ListenersMap = *GetListeners();

	for (const FAIDamageEvent& Event : RegisteredEvents)
	{
		IAIPerceptionListenerInterface* PerceptionListener = Event.GetDamagedActorAsPerceptionListener();
		if (PerceptionListener != nullptr)
		{
			UAIPerceptionComponent* PerceptionComponent = PerceptionListener->GetPerceptionComponent();
			if (PerceptionComponent != nullptr && PerceptionComponent->GetListenerId().IsValid())
			{
				// this has to succeed, will assert a failure
				FPerceptionListener& Listener = ListenersMap[PerceptionComponent->GetListenerId()];

				if (Listener.HasSense(GetSenseID()))
				{
					Listener.RegisterStimulus(Event.Instigator, FAIStimulus(*this, Event.Amount, Event.Location, Event.HitLocation, FAIStimulus::SensingSucceeded, Event.Tag));
				}
			}
		}
	}

	RegisteredEvents.Reset();

	// return decides when next tick is going to happen
	return SuspendNextUpdate;
}

void UAISense_Damage::RegisterEvent(const FAIDamageEvent& Event)
{
	if (Event.IsValid())
	{
		RegisteredEvents.Add(Event);

		RequestImmediateUpdate();
	}
}

void UAISense_Damage::RegisterWrappedEvent(UAISenseEvent& PerceptionEvent)
{
	UAISenseEvent_Damage* DamageEvent = Cast<UAISenseEvent_Damage>(&PerceptionEvent);
	ensure(DamageEvent);
	if (DamageEvent)
	{
		RegisterEvent(DamageEvent->GetDamageEvent());
	}
}

void UAISense_Damage::ReportDamageEvent(UObject* WorldContextObject, AActor* DamagedActor, AActor* Instigator, float DamageAmount, FVector EventLocation, FVector HitLocation, FName Tag/* = NAME_None*/)
{
	UAIPerceptionSystem* PerceptionSystem = UAIPerceptionSystem::GetCurrent(WorldContextObject);
	if (PerceptionSystem)
	{
		FAIDamageEvent Event(DamagedActor, Instigator, DamageAmount, EventLocation, HitLocation, Tag);
		PerceptionSystem->OnEvent(Event);
	}
}

