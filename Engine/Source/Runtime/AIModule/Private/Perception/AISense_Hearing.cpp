// Copyright Epic Games, Inc. All Rights Reserved.

#include "Perception/AISense_Hearing.h"
#include "Perception/AIPerceptionSystem.h"
#include "Perception/AIPerceptionComponent.h"
#include "Perception/AISenseConfig_Hearing.h"
#include "Perception/AISenseEvent_Hearing.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AISense_Hearing)

//----------------------------------------------------------------------//
// FAINoiseEvent
//----------------------------------------------------------------------//
FAINoiseEvent::FAINoiseEvent()
	: Age(0.f), NoiseLocation(FAISystem::InvalidLocation), Loudness(1.f), MaxRange(0.f)
	, Instigator(nullptr), Tag(NAME_None), TeamIdentifier(FGenericTeamId::NoTeam)
{
}

FAINoiseEvent::FAINoiseEvent(AActor* InInstigator, const FVector& InNoiseLocation, float InLoudness, float InMaxRange, FName InTag)
	: Age(0.f), NoiseLocation(InNoiseLocation), Loudness(InLoudness), MaxRange(InMaxRange)
	, Instigator(InInstigator), Tag(InTag), TeamIdentifier(FGenericTeamId::NoTeam)
{
	Compile();
}

void FAINoiseEvent::Compile()
{
	TeamIdentifier = FGenericTeamId::GetTeamIdentifier(Instigator);
	if (FAISystem::IsValidLocation(NoiseLocation) == false && Instigator != nullptr)
	{
		NoiseLocation = Instigator->GetActorLocation();
	}
}

//----------------------------------------------------------------------//
// FDigestedHearingProperties
//----------------------------------------------------------------------//
UAISense_Hearing::FDigestedHearingProperties::FDigestedHearingProperties(const UAISenseConfig_Hearing& SenseConfig)
{
	HearingRangeSq = FMath::Square(SenseConfig.HearingRange);
	AffiliationFlags = SenseConfig.DetectionByAffiliation.GetAsFlags();
}

UAISense_Hearing::FDigestedHearingProperties::FDigestedHearingProperties()
	: HearingRangeSq(-1.f)
{
	AffiliationFlags = FAISenseAffiliationFilter::DetectAllFlags();
}

//----------------------------------------------------------------------//
// UAISense_Hearing
//----------------------------------------------------------------------//
UAISense_Hearing::UAISense_Hearing(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer)
{
	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		OnNewListenerDelegate.BindUObject(this, &UAISense_Hearing::OnNewListenerImpl);
		OnListenerUpdateDelegate.BindUObject(this, &UAISense_Hearing::OnListenerUpdateImpl);
		OnListenerRemovedDelegate.BindUObject(this, &UAISense_Hearing::OnListenerRemovedImpl);
	}
}

void UAISense_Hearing::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		RegisterMakeNoiseDelegate();
	}
}

void UAISense_Hearing::RegisterMakeNoiseDelegate()
{
	AActor::SetMakeNoiseDelegate(FMakeNoiseDelegate::CreateStatic(&UAIPerceptionSystem::MakeNoiseImpl));
}

void UAISense_Hearing::ReportNoiseEvent(UObject* WorldContextObject, FVector NoiseLocation, float Loudness, AActor* Instigator, float MaxRange, FName Tag)
{
	UAIPerceptionSystem* PerceptionSystem = UAIPerceptionSystem::GetCurrent(WorldContextObject);
	if (PerceptionSystem)
	{
		FAINoiseEvent Event(Instigator, NoiseLocation, Loudness, MaxRange, Tag);
		PerceptionSystem->OnEvent(Event);
	}
}

void UAISense_Hearing::OnNewListenerImpl(const FPerceptionListener& NewListener)
{
	UAIPerceptionComponent* ListenerPtr = NewListener.Listener.Get();
	check(ListenerPtr);
	const UAISenseConfig_Hearing* SenseConfig = Cast<const UAISenseConfig_Hearing>(ListenerPtr->GetSenseConfig(GetSenseID()));
	check(SenseConfig);
	const FDigestedHearingProperties PropertyDigest(*SenseConfig);
	DigestedProperties.Add(NewListener.GetListenerID(), PropertyDigest);
}

void UAISense_Hearing::OnListenerUpdateImpl(const FPerceptionListener& UpdatedListener)
{
	// @todo add updating code here
	const FPerceptionListenerID ListenerID = UpdatedListener.GetListenerID();
	
	if (UpdatedListener.HasSense(GetSenseID()))
	{
		const UAISenseConfig_Hearing* SenseConfig = Cast<const UAISenseConfig_Hearing>(UpdatedListener.Listener->GetSenseConfig(GetSenseID()));
		check(SenseConfig);
		FDigestedHearingProperties& PropertiesDigest = DigestedProperties.FindOrAdd(ListenerID);
		PropertiesDigest = FDigestedHearingProperties(*SenseConfig);
	}
	else
	{
		DigestedProperties.Remove(ListenerID);
	}
}

void UAISense_Hearing::OnListenerRemovedImpl(const FPerceptionListener& RemovedListener)
{
	DigestedProperties.FindAndRemoveChecked(RemovedListener.GetListenerID());
}

float UAISense_Hearing::Update()
{
	AIPerception::FListenerMap& ListenersMap = *GetListeners();
	UAIPerceptionSystem* PerceptionSys = GetPerceptionSystem();
	const float SpeedOfSoundSqScalar = SpeedOfSoundSq > 0.f ? 1.f / SpeedOfSoundSq : 0.f;

	for (AIPerception::FListenerMap::TIterator ListenerIt(ListenersMap); ListenerIt; ++ListenerIt)
	{
		FPerceptionListener& Listener = ListenerIt->Value;
		
		if (Listener.HasSense(GetSenseID()) == false)
		{
			// skip listeners not interested in this sense
			continue;
		}

		const FDigestedHearingProperties& PropDigest = DigestedProperties[Listener.GetListenerID()];

		for (const FAINoiseEvent& Event : NoiseEvents)
		{
			const float ClampedLoudness = FMath::Max(0.f, Event.Loudness);
			const FVector::FReal DistToSoundSquared = FVector::DistSquared(Event.NoiseLocation, Listener.CachedLocation);
			
			// Limit by loudness modified squared range (this is the old behavior)
			if (DistToSoundSquared > PropDigest.HearingRangeSq * FMath::Square(ClampedLoudness))
			{
				continue;
			}
			// Limit by max range
			else if (Event.MaxRange > 0.f && DistToSoundSquared > FMath::Square(Event.MaxRange * ClampedLoudness))
			{
				continue;
			}

			if (FAISenseAffiliationFilter::ShouldSenseTeam(Listener.TeamIdentifier, Event.TeamIdentifier, PropDigest.AffiliationFlags) == false)
			{
				continue;
			}
			// calculate delay and fake it with Age, Delay should be pretty small so a static_cast is safe enough here.
			const float Delay = FloatCastChecked<float>(FMath::Sqrt(DistToSoundSquared * SpeedOfSoundSqScalar), UE::LWC::DefaultFloatPrecision);
			// pass over to listener to process 			
			PerceptionSys->RegisterDelayedStimulus(Listener.GetListenerID(), Delay, Event.Instigator
				, FAIStimulus(*this, ClampedLoudness, Event.NoiseLocation, Listener.CachedLocation, FAIStimulus::SensingSucceeded, Event.Tag) );
		}
	}

	NoiseEvents.Reset();

	// return decides when next tick is going to happen
	return SuspendNextUpdate;
}

void UAISense_Hearing::RegisterEvent(const FAINoiseEvent& Event)
{
	NoiseEvents.Add(Event);

	RequestImmediateUpdate();
}

void UAISense_Hearing::RegisterEventsBatch(const TArray<FAINoiseEvent>& Events)
{
	NoiseEvents.Append(Events);

	RequestImmediateUpdate();
}

void UAISense_Hearing::RegisterWrappedEvent(UAISenseEvent& PerceptionEvent)
{
	UAISenseEvent_Hearing* HearingEvent = Cast<UAISenseEvent_Hearing>(&PerceptionEvent);
	ensure(HearingEvent);
	if (HearingEvent)
	{
		RegisterEvent(HearingEvent->GetNoiseEvent());
	}
}

