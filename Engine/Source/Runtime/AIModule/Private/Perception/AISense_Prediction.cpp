// Copyright Epic Games, Inc. All Rights Reserved.

#include "Perception/AISense_Prediction.h"
#include "Perception/AIPerceptionListenerInterface.h"
#include "Perception/AIPerceptionSystem.h"
#include "AIController.h"
#include "Perception/AIPerceptionComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AISense_Prediction)

UAISense_Prediction::UAISense_Prediction(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer)
{
}

float UAISense_Prediction::Update()
{
	AIPerception::FListenerMap& ListenersMap = *GetListeners();

	for (const FAIPredictionEvent& Event : RegisteredEvents)
	{
		if (Event.Requestor != NULL && Event.PredictedActor != NULL)
		{
			IAIPerceptionListenerInterface* PerceptionListener = Cast<IAIPerceptionListenerInterface>(Event.Requestor);
			if (PerceptionListener != NULL)
			{
				UAIPerceptionComponent* PerceptionComponent = PerceptionListener->GetPerceptionComponent();
				if (PerceptionComponent != NULL && ListenersMap.Contains(PerceptionComponent->GetListenerId()))
				{
					// this has to succeed, will assert a failure
					FPerceptionListener& Listener = ListenersMap[PerceptionComponent->GetListenerId()];

					if (Listener.HasSense(GetSenseID()))
					{
						// calculate the prediction here:
						const FVector PredictedLocation = Event.PredictedActor->GetActorLocation() + Event.PredictedActor->GetVelocity() * Event.TimeToPredict;

						Listener.RegisterStimulus(Event.PredictedActor, FAIStimulus(*this, 1.f, PredictedLocation, Listener.CachedLocation));
					}
				}
			}
		}
	}

	RegisteredEvents.Reset();

	// return decides when next tick is going to happen
	return SuspendNextUpdate;
}

void UAISense_Prediction::RegisterEvent(const FAIPredictionEvent& Event)
{
	RegisteredEvents.Add(Event);
	RequestImmediateUpdate();
}

void UAISense_Prediction::RequestControllerPredictionEvent(AAIController* Requestor, AActor* PredictedActor, float PredictionTime)
{
	UAIPerceptionSystem* PerceptionSystem = UAIPerceptionSystem::GetCurrent(Requestor);
	if (PerceptionSystem)
	{
		FAIPredictionEvent Event(Requestor, PredictedActor, PredictionTime);
		PerceptionSystem->OnEvent(Event);
	}
}

void UAISense_Prediction::RequestPawnPredictionEvent(APawn* Requestor, AActor* PredictedActor, float PredictionTime)
{
	UAIPerceptionSystem* PerceptionSystem = UAIPerceptionSystem::GetCurrent(Requestor);
	if (PerceptionSystem && Requestor->GetController())
	{
		FAIPredictionEvent Event(Requestor->GetController(), PredictedActor, PredictionTime);
		PerceptionSystem->OnEvent(Event);
	}
}

