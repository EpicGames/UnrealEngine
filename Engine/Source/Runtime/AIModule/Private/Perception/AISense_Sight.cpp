// Copyright Epic Games, Inc. All Rights Reserved.

#include "Perception/AISense_Sight.h"

#include "AIHelpers.h"
#include "AISystem.h"
#include "CollisionQueryParams.h"
#include "Engine/Engine.h"
#include "Engine/HitResult.h"
#include "EngineDefines.h"
#include "EngineGlobals.h"
#include "Perception/AIPerceptionComponent.h"
#include "Perception/AISenseConfig_Sight.h"
#include "Perception/AISightTargetInterface.h"
#include "VisualLogger/VisualLogger.h"

#if WITH_GAMEPLAY_DEBUGGER_MENU
#include "GameplayDebuggerTypes.h"
#include "GameplayDebuggerCategory.h"
#endif // WITH_GAMEPLAY_DEBUGGER_MENU

#include UE_INLINE_GENERATED_CPP_BY_NAME(AISense_Sight)

#define AISENSE_SIGHT_TIMESLICING_DEBUG 0
#define DO_SIGHT_VLOGGING (0 && ENABLE_VISUAL_LOG)

#if DO_SIGHT_VLOGGING
	#define SIGHT_LOG_SEGMENT(LogOwner, SegmentStart, SegmentEnd, Color, Format, ...) UE_VLOG_SEGMENT(LogOwner, LogAIPerception, Verbose, SegmentStart, SegmentEnd, Color, Format, ##__VA_ARGS__)
	#define SIGHT_LOG_LOCATION(LogOwner, Location, Radius, Color, Format, ...) UE_VLOG_LOCATION(LogOwner, LogAIPerception, Verbose, Location, Radius, Color, Format, ##__VA_ARGS__)
#else
	#define SIGHT_LOG_SEGMENT(...)
	#define SIGHT_LOG_LOCATION(...)
#endif // DO_SIGHT_VLOGGING

DECLARE_CYCLE_STAT(TEXT("Perception Sense: Sight"),STAT_AI_Sense_Sight,STATGROUP_AI);
DECLARE_CYCLE_STAT(TEXT("Perception Sense: Sight, Update Sort"),STAT_AI_Sense_Sight_UpdateSort,STATGROUP_AI);
DECLARE_CYCLE_STAT(TEXT("Perception Sense: Sight, Compute visibility"),STAT_AI_Sense_Sight_ComputeVisibility,STATGROUP_AI);
DECLARE_CYCLE_STAT(TEXT("Perception Sense: Sight, Query operations"),STAT_AI_Sense_Sight_QueryOperations,STATGROUP_AI);
DECLARE_CYCLE_STAT(TEXT("Perception Sense: Sight, Listener Update"), STAT_AI_Sense_Sight_ListenerUpdate, STATGROUP_AI);
DECLARE_CYCLE_STAT(TEXT("Perception Sense: Sight, Register Target"), STAT_AI_Sense_Sight_RegisterTarget, STATGROUP_AI);
DECLARE_CYCLE_STAT(TEXT("Perception Sense: Sight, Remove By Listener"), STAT_AI_Sense_Sight_RemoveByListener, STATGROUP_AI);
DECLARE_CYCLE_STAT(TEXT("Perception Sense: Sight, Remove To Target"), STAT_AI_Sense_Sight_RemoveToTarget, STATGROUP_AI);
DECLARE_CYCLE_STAT(TEXT("Perception Sense: Sight, Process pending result"), STAT_AI_Sense_Sight_ProcessPendingQuery, STATGROUP_AI);


constexpr int32 DefaultMaxTracesPerTick = 6;
constexpr int32 DefaultMaxAsyncTracesPerTick = 10;
constexpr int32 DefaultMinQueriesPerTimeSliceCheck = 40;
constexpr float DefaultPendingQueriesBudgetReductionRatio = 0.5f;
constexpr bool bDefaultUseAsynchronousTraceForDefaultSightQueries = false;
constexpr float DefaultStimulusStrength = 1.f;

enum class EForEachResult : uint8
{
	Break,
	Continue,
};

template <typename T, class PREDICATE_CLASS>
EForEachResult ForEach(T& Array, const PREDICATE_CLASS& Predicate)
{
	for (typename T::ElementType& Element : Array)
	{
		if (Predicate(Element) == EForEachResult::Break)
		{
			return EForEachResult::Break;
		}
	}
	return EForEachResult::Continue;
}

enum EReverseForEachResult : uint8
{
	UnTouched,
	Modified,
};

template <typename T, class PREDICATE_CLASS>
EReverseForEachResult ReverseForEach(T& Array, const PREDICATE_CLASS& Predicate)
{
	EReverseForEachResult RetVal = EReverseForEachResult::UnTouched;
	for (int32 Index = Array.Num()-1; Index >= 0; --Index)
	{
		if (Predicate(Array, Index) == EReverseForEachResult::Modified)
		{
			RetVal = EReverseForEachResult::Modified;
		}
	}
	return RetVal;
}

//----------------------------------------------------------------------//
// FAISightTarget
//----------------------------------------------------------------------//
const FAISightTarget::FTargetId FAISightTarget::InvalidTargetId = FAISystem::InvalidUnsignedID;

FAISightTarget::FAISightTarget(AActor* InTarget, FGenericTeamId InTeamId)
	: Target(InTarget), TeamId(InTeamId)
{
	if (InTarget)
	{
		TargetId = InTarget->GetUniqueID();
	}
	else
	{
		TargetId = InvalidTargetId;
	}
}

//----------------------------------------------------------------------//
// FDigestedSightProperties
//----------------------------------------------------------------------//
UAISense_Sight::FDigestedSightProperties::FDigestedSightProperties(const UAISenseConfig_Sight& SenseConfig)
{
	SightRadiusSq = FMath::Square(SenseConfig.SightRadius + SenseConfig.PointOfViewBackwardOffset);
	LoseSightRadiusSq = FMath::Square(SenseConfig.LoseSightRadius + SenseConfig.PointOfViewBackwardOffset);
	PointOfViewBackwardOffset = SenseConfig.PointOfViewBackwardOffset;
	NearClippingRadiusSq = FMath::Square(SenseConfig.NearClippingRadius);
	PeripheralVisionAngleCos = FMath::Cos(FMath::Clamp(FMath::DegreesToRadians(SenseConfig.PeripheralVisionAngleDegrees), 0.f, PI));
	AffiliationFlags = SenseConfig.DetectionByAffiliation.GetAsFlags();
	// keep the special value of FAISystem::InvalidRange (-1.f) if it's set.
	AutoSuccessRangeSqFromLastSeenLocation = (SenseConfig.AutoSuccessRangeFromLastSeenLocation == FAISystem::InvalidRange) ? FAISystem::InvalidRange : FMath::Square(SenseConfig.AutoSuccessRangeFromLastSeenLocation);
}

UAISense_Sight::FDigestedSightProperties::FDigestedSightProperties()
	: PeripheralVisionAngleCos(0.f), SightRadiusSq(-1.f), AutoSuccessRangeSqFromLastSeenLocation(FAISystem::InvalidRange), LoseSightRadiusSq(-1.f), PointOfViewBackwardOffset(0.0f), NearClippingRadiusSq(0.0f)
{
	AffiliationFlags = FAISenseAffiliationFilter::DetectAllFlags();
}

//----------------------------------------------------------------------//
// UAISense_Sight
//----------------------------------------------------------------------//
UAISense_Sight::UAISense_Sight(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, MaxTracesPerTick(DefaultMaxTracesPerTick)
	, MaxAsyncTracesPerTick(DefaultMaxAsyncTracesPerTick)
	, MinQueriesPerTimeSliceCheck(DefaultMinQueriesPerTimeSliceCheck)
	, MaxTimeSlicePerTick(0.005) // 5ms
	, HighImportanceQueryDistanceThreshold(300.f)
	, MaxQueryImportance(60.f)
	, SightLimitQueryImportance(10.f)
	, PendingQueriesBudgetReductionRatio(DefaultPendingQueriesBudgetReductionRatio)
	, bUseAsynchronousTraceForDefaultSightQueries(bDefaultUseAsynchronousTraceForDefaultSightQueries)
{
	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		UAISenseConfig_Sight* SightConfigCDO = GetMutableDefault<UAISenseConfig_Sight>();
		SightConfigCDO->Implementation = UAISense_Sight::StaticClass();

		OnNewListenerDelegate.BindUObject(this, &UAISense_Sight::OnNewListenerImpl);
		OnListenerUpdateDelegate.BindUObject(this, &UAISense_Sight::OnListenerUpdateImpl);
		OnListenerRemovedDelegate.BindUObject(this, &UAISense_Sight::OnListenerRemovedImpl);

		OnPendingCanBeSeenQueryProcessedDelegate.BindUObject(this, &UAISense_Sight::OnPendingCanBeSeenQueryProcessed);
		OnPendingTraceQueryProcessedDelegate.BindUObject(this, &UAISense_Sight::OnPendingTraceQueryProcessed);
	}

	NotifyType = EAISenseNotifyType::OnPerceptionChange;
	
	bAutoRegisterAllPawnsAsSources = true;
	bNeedsForgettingNotification = true;

	DefaultSightCollisionChannel = GET_AI_CONFIG_VAR(DefaultSightCollisionChannel);
}

float UAISense_Sight::CalcQueryImportance(const FPerceptionListener& Listener, const FVector& TargetLocation, const float SightRadiusSq) const
{
	const FVector::FReal DistanceSq = FVector::DistSquared(Listener.CachedLocation, TargetLocation);
	return DistanceSq <= HighImportanceDistanceSquare ? MaxQueryImportance
		: static_cast<float>(FMath::Clamp((SightLimitQueryImportance - MaxQueryImportance) / SightRadiusSq * DistanceSq + MaxQueryImportance, 0.f, MaxQueryImportance));
}

void UAISense_Sight::PostInitProperties()
{
	Super::PostInitProperties();
	HighImportanceDistanceSquare = FMath::Square(HighImportanceQueryDistanceThreshold);
}

bool UAISense_Sight::ShouldAutomaticallySeeTarget(const FDigestedSightProperties& PropDigest, FAISightQuery* SightQuery, FPerceptionListener& Listener, AActor* TargetActor, float& OutStimulusStrength) const
{
	OutStimulusStrength = 1.0f;

	if ((PropDigest.AutoSuccessRangeSqFromLastSeenLocation != FAISystem::InvalidRange) && (SightQuery->LastSeenLocation != FAISystem::InvalidLocation))
	{
		const FVector::FReal DistanceToLastSeenLocationSq = FVector::DistSquared(TargetActor->GetActorLocation(), SightQuery->LastSeenLocation);
		return (DistanceToLastSeenLocationSq <= PropDigest.AutoSuccessRangeSqFromLastSeenLocation);
	}

	return false;
}

namespace UE::AISense_Sight
{
#if AISENSE_SIGHT_TIMESLICING_DEBUG
	struct FTimingSlicingInfo
	{
		FTimingSlicingInfo() 
		{
			Start(); 
		}

		double StartTime = 0.;
		double EndTime = 0.;

		int32 InRangeCount = 0;
		int32 OutOfRangeCount = 0;

		float InRangeAgeSum = 0.f;
		float OutOfRangeAgeSum = 0.f;

		void Start() { StartTime = FPlatformTime::Seconds();}
		void Stop() { EndTime = FPlatformTime::Seconds();}

		void PushQueryInfo(const bool bIsInRange, const float Age)
		{
			if (bIsInRange)
			{
				++InRangeCount;
				InRangeAgeSum += Age;
			}
			else
			{
				++OutOfRangeCount;
				OutOfRangeAgeSum += Age;
			}
		}

		FString ToString() const
		{
			FString Info = FString::Format(TEXT("in {0} seconds"), {EndTime - StartTime});
			if (InRangeCount > 0)
			{
				Info.Append(FString::Format(TEXT("[{0} InRange Age:{1}]"), {InRangeCount, InRangeAgeSum/InRangeCount}));
			}
			if (OutOfRangeCount > 0)
			{
				Info.Append(FString::Format(TEXT("[{0} OutOfRange Age:{1}]"), {OutOfRangeCount, OutOfRangeAgeSum/OutOfRangeCount}));
			}
			return Info;
		}
	};
#endif // AISENSE_SIGHT_TIMESLICING_DEBUG

	bool IsTraceConsideredVisible(const FHitResult* HitResult, const AActor* TargetActor)
	{
		if (HitResult == nullptr)
		{
			return true;
		}

		const AActor* HitResultActor = HitResult->HitObjectHandle.FetchActor();
		return (HitResultActor ? HitResultActor->IsOwnedBy(TargetActor) : false);
	}
}

float UAISense_Sight::Update()
{
	SCOPE_CYCLE_COUNTER(STAT_AI_Sense_Sight);

	UWorld* World = GEngine->GetWorldFromContextObject(GetPerceptionSystem()->GetOuter(), EGetWorldErrorMode::LogAndReturnNull);

	if (World == nullptr)
	{
		return SuspendNextUpdate;
	}

	UE_MT_SCOPED_WRITE_ACCESS(QueriesListAccessDetector);

	// sort Sight Queries
	{
		auto RecalcScore = [](FAISightQuery& SightQuery)->EForEachResult
		{
			SightQuery.RecalcScore();
			return EForEachResult::Continue;
		};

		SCOPE_CYCLE_COUNTER(STAT_AI_Sense_Sight_UpdateSort);
        // Sort out of range queries
    	if (bSightQueriesOutOfRangeDirty)
		{
			ForEach(SightQueriesOutOfRange, RecalcScore);
			SightQueriesOutOfRange.Sort(FAISightQuery::FSortPredicate());
			NextOutOfRangeIndex = 0;
			bSightQueriesOutOfRangeDirty = false;
		}

        // Sort in range queries
		ForEach(SightQueriesInRange, RecalcScore);
		SightQueriesInRange.Sort(FAISightQuery::FSortPredicate());
	}

	int32 TracesCount = 0;
	int32 AsyncTracesCount = FMath::Max(0, static_cast<int32>(PendingQueriesBudgetReductionRatio * SightQueriesPending.Num()));	// pending queries should be requesting async collisions traces at this frame, so we might want to restrain ourself in this update
	int32 NumQueriesProcessed = 0;
	const double TimeSliceEnd = FPlatformTime::Seconds() + MaxTimeSlicePerTick;
	bool bHitTimeSliceLimit = false;
#if AISENSE_SIGHT_TIMESLICING_DEBUG
	UE::AISense_Sight::FTimingSlicingInfo SlicingInfo;
#endif // AISENSE_SIGHT_TIMESLICING_DEBUG
	constexpr int32 InitialInvalidItemsSize = 16;
	enum class EOperationType : uint8
	{
		Remove,
		SwapList,
		MoveToPending
	};
	struct FQueryOperation
	{
		FQueryOperation(bool bInInRange, EOperationType InOpType, int32 InIndex) : bInRange(bInInRange), OpType(InOpType), Index(InIndex) {}
		bool bInRange;
		EOperationType OpType;
		int32 Index;
	};
	TArray<FQueryOperation> QueryOperations;
	TArray<FAISightTarget::FTargetId> InvalidTargets;
	QueryOperations.Reserve(InitialInvalidItemsSize);
	InvalidTargets.Reserve(InitialInvalidItemsSize);

	AIPerception::FListenerMap& ListenersMap = *GetListeners();

	int32 InRangeItr = 0;
	int32 OutOfRangeItr = 0;
	for (int32 QueryIndex = 0; QueryIndex < SightQueriesInRange.Num() + SightQueriesOutOfRange.Num(); ++QueryIndex)
	{
		// Time slice limit check - spread out checks to every N queries so we don't spend more time checking timer than doing work
		NumQueriesProcessed++;
		if ((NumQueriesProcessed % MinQueriesPerTimeSliceCheck) == 0 && FPlatformTime::Seconds() > TimeSliceEnd)
		{
			bHitTimeSliceLimit = true;
		}

		if (bHitTimeSliceLimit || TracesCount >= MaxTracesPerTick || AsyncTracesCount >= MaxAsyncTracesPerTick)
		{
			break;
		}

		// Calculate next in range query
		int32 InRangeIndex = SightQueriesInRange.IsValidIndex(InRangeItr) ? InRangeItr : INDEX_NONE;
		FAISightQuery* InRangeQuery = InRangeIndex != INDEX_NONE ? &SightQueriesInRange[InRangeIndex] : nullptr;

		// Calculate next out of range query
		int32 OutOfRangeIndex = SightQueriesOutOfRange.IsValidIndex(OutOfRangeItr) ? (NextOutOfRangeIndex + OutOfRangeItr) % SightQueriesOutOfRange.Num() : INDEX_NONE;
		FAISightQuery* OutOfRangeQuery = OutOfRangeIndex != INDEX_NONE ? &SightQueriesOutOfRange[OutOfRangeIndex] : nullptr;
		if (OutOfRangeQuery)
		{
			OutOfRangeQuery->RecalcScore();
		}

		// Compare to real find next query
		const bool bIsInRangeQuery = (InRangeQuery && OutOfRangeQuery) ? FAISightQuery::FSortPredicate()(*InRangeQuery,*OutOfRangeQuery) : !OutOfRangeQuery;
		FAISightQuery* SightQuery = bIsInRangeQuery ? InRangeQuery : OutOfRangeQuery;
		ensure(SightQuery);

#if AISENSE_SIGHT_TIMESLICING_DEBUG
		SlicingInfo.PushQueryInfo(bIsInRangeQuery, SightQuery->GetAge());
#endif //AISENSE_SIGHT_TIMESLICING_DEBUG

		bIsInRangeQuery ? ++InRangeItr : ++OutOfRangeItr;

		FPerceptionListener& Listener = ListenersMap[SightQuery->ObserverId];
		FAISightTarget& Target = ObservedTargets[SightQuery->TargetId];

		AActor* TargetActor = Target.Target.Get();
		UAIPerceptionComponent* ListenerPtr = Listener.Listener.Get();
		ensure(ListenerPtr);

		// @todo figure out what should we do if not valid
		if (TargetActor && ListenerPtr)
		{
			const FDigestedSightProperties& PropDigest = DigestedProperties[SightQuery->ObserverId];
			const AActor* ListenerBodyActor = ListenerPtr->GetBodyActor();
			float StimulusStrength = DefaultStimulusStrength;
			FVector SeenLocation(0.f);
			int32 NumberOfLoSChecksPerformed = 0;
			int32 NumberOfAsyncLosCheckRequested = 0;

			const EVisibilityResult VisibilityResult = ComputeVisibility(World, *SightQuery, Listener, ListenerBodyActor, Target, TargetActor, PropDigest, StimulusStrength, SeenLocation, NumberOfLoSChecksPerformed, NumberOfAsyncLosCheckRequested);

			TracesCount += NumberOfLoSChecksPerformed;
			AsyncTracesCount += NumberOfAsyncLosCheckRequested;

			if (VisibilityResult == EVisibilityResult::Pending)
			{
				QueryOperations.Add(FQueryOperation(bIsInRangeQuery, EOperationType::MoveToPending, bIsInRangeQuery ? InRangeIndex : OutOfRangeIndex));
			}
			else
			{
				UE_CLOGF(VisibilityResult != EVisibilityResult::Visible && VisibilityResult != EVisibilityResult::NotVisible, LogAIPerception, Error, "UAISense_Sight::Update received invalid Visibility result [%d] for query between Listener %ls and Target %ls. We'll consider it as NotVisible", int(VisibilityResult), *GetNameSafe(ListenerBodyActor), *GetNameSafe(TargetActor));

				const bool bIsVisible = VisibilityResult == EVisibilityResult::Visible;
				const bool bWasVisible = SightQuery->GetLastResult();
				const FVector TargetLocation = TargetActor->GetActorLocation();
				UpdateQueryVisibilityStatus(*SightQuery, Listener, bIsVisible, SeenLocation, StimulusStrength, *TargetActor, TargetLocation);

				const float SightRadiusSq = bWasVisible ? PropDigest.LoseSightRadiusSq : PropDigest.SightRadiusSq;
				SightQuery->Importance = CalcQueryImportance(Listener, TargetLocation, SightRadiusSq);
				const bool bShouldBeInRange = SightQuery->Importance > 0.0f;
				if (bIsInRangeQuery != bShouldBeInRange)
				{
					QueryOperations.Add(FQueryOperation(bIsInRangeQuery, EOperationType::SwapList, bIsInRangeQuery ? InRangeIndex : OutOfRangeIndex));
				}

				// restart query
				SightQuery->OnProcessed();
			}
		}
		else
		{
			// put this index to "to be removed" array
			QueryOperations.Add( FQueryOperation(bIsInRangeQuery, EOperationType::Remove, bIsInRangeQuery ? InRangeIndex : OutOfRangeIndex) );
			if (TargetActor == nullptr)
			{
				InvalidTargets.AddUnique(SightQuery->TargetId);
			}
		}
	}
	NextOutOfRangeIndex = SightQueriesOutOfRange.Num() > 0 ? (NextOutOfRangeIndex + OutOfRangeItr) % SightQueriesOutOfRange.Num() : 0;

#if AISENSE_SIGHT_TIMESLICING_DEBUG
	SlicingInfo.Stop();
	UE_LOGF(LogAIPerception, VeryVerbose, "UAISense_Sight::Update processed %d sources %ls [time slice limited? %d]", NumQueriesProcessed, *SlicingInfo.ToString(), bHitTimeSliceLimit ? 1 : 0);
#else
	UE_LOGF(LogAIPerception, VeryVerbose, "UAISense_Sight::Update processed %d sources [time slice limited? %d]", NumQueriesProcessed, bHitTimeSliceLimit ? 1 : 0);
#endif // AISENSE_SIGHT_TIMESLICING_DEBUG

	if (QueryOperations.Num() > 0)
	{
		SCOPE_CYCLE_COUNTER(STAT_AI_Sense_Sight_QueryOperations);

		// Sort by InRange and by descending Index 
		QueryOperations.Sort([](const FQueryOperation& LHS, const FQueryOperation& RHS)->bool
		{
			if (LHS.bInRange != RHS.bInRange)
				return LHS.bInRange;
			return LHS.Index > RHS.Index;
		});
        // Do all the removes first and save the out of range swaps because we will insert them at the right location to prevent sorting
		TArray<FAISightQuery> SightQueriesOutOfRangeToInsert;
		for (const FQueryOperation& Operation : QueryOperations)
		{
			switch (Operation.OpType)
			{
			case EOperationType::SwapList:
				{
					if (Operation.bInRange)
					{
						SightQueriesOutOfRangeToInsert.Push(SightQueriesInRange[Operation.Index]);
					}
					else
					{
						SightQueriesInRange.Add(SightQueriesOutOfRange[Operation.Index]);
					}
				}
				break;

			case EOperationType::MoveToPending:
				{
					SightQueriesPending.Add(Operation.bInRange ? SightQueriesInRange[Operation.Index] : SightQueriesOutOfRange[Operation.Index]);
				}
				break;

			case EOperationType::Remove:
				break;

			default:
				check(false);
				break;
			}

			if (Operation.bInRange)
			{
				// In range queries are always sorted at the beginning of the update
				SightQueriesInRange.RemoveAtSwap(Operation.Index, EAllowShrinking::No);
			}
			else
			{
				// Preserve the list ordered
				SightQueriesOutOfRange.RemoveAt(Operation.Index, EAllowShrinking::No);
				if (Operation.Index < NextOutOfRangeIndex)
				{
					NextOutOfRangeIndex--;
				}
			}
		}
        // Reinsert the saved out of range swaps
		if (SightQueriesOutOfRangeToInsert.Num() > 0)
		{
			SightQueriesOutOfRange.Insert(SightQueriesOutOfRangeToInsert.GetData(), SightQueriesOutOfRangeToInsert.Num(), NextOutOfRangeIndex);
			NextOutOfRangeIndex += SightQueriesOutOfRangeToInsert.Num();
		}

		if (InvalidTargets.Num() > 0)
		{
			// this should not be happening since UAIPerceptionSystem::OnPerceptionStimuliSourceEndPlay introduction
			UE_VLOG(GetPerceptionSystem(), LogAIPerception, Error, TEXT("Invalid sight targets found during UAISense_Sight::Update call"));

			for (const auto& TargetId : InvalidTargets)
			{
				// remove affected queries
				RemoveAllQueriesToTarget_Internal(TargetId);
				// remove target itself
				ObservedTargets.Remove(TargetId);
			}

			// remove holes
			ObservedTargets.Compact();
		}
	}

	//return SightQueries.Num() > 0 ? 1.f/6 : FLT_MAX;
	return 0.f;
}

UAISense_Sight::EVisibilityResult UAISense_Sight::ComputeVisibility(UWorld* World, FAISightQuery& SightQuery, FPerceptionListener& Listener, const AActor* ListenerActor, FAISightTarget& Target, AActor* TargetActor, const FDigestedSightProperties& PropDigest, float& OutStimulusStrength, FVector& OutSeenLocation, int32& OutNumberOfLoSChecksPerformed, int32& OutNumberOfAsyncLosCheckRequested) const
{
	SCOPE_CYCLE_COUNTER(STAT_AI_Sense_Sight_ComputeVisibility);

	// @Note that automagical "seeing" does not care about sight range nor vision cone
	if (ShouldAutomaticallySeeTarget(PropDigest, &SightQuery, Listener, TargetActor, OutStimulusStrength))
	{
		OutSeenLocation = FAISystem::InvalidLocation;
		return EVisibilityResult::Visible;
	}

	const FVector TargetLocation = TargetActor->GetActorLocation();
	const float SightRadiusSq = SightQuery.GetLastResult() ? PropDigest.LoseSightRadiusSq : PropDigest.SightRadiusSq;
	if (!FAISystem::CheckIsTargetInSightCone(Listener.CachedLocation, Listener.CachedDirection, PropDigest.PeripheralVisionAngleCos, PropDigest.PointOfViewBackwardOffset, PropDigest.NearClippingRadiusSq, SightRadiusSq, TargetLocation))
	{
		return EVisibilityResult::NotVisible;
	}

	if (IAISightTargetInterface* SightTargetInterface = Target.WeakSightTargetInterface.Get())
	{
		const bool bWasVisible = SightQuery.GetLastResult();
		FCanBeSeenFromContext Context;
		Context.SightQueryID = FAISightQueryID(SightQuery);
		Context.ObserverLocation = Listener.CachedLocation;
		Context.IgnoreActor = ListenerActor;
		Context.bWasVisible = &bWasVisible;

		const EVisibilityResult Result = SightTargetInterface->CanBeSeenFrom(Context, OutSeenLocation, OutNumberOfLoSChecksPerformed, OutNumberOfAsyncLosCheckRequested, OutStimulusStrength, &SightQuery.UserData, &OnPendingCanBeSeenQueryProcessedDelegate);
		if (Result == EVisibilityResult::Pending)
		{
			// we need to clear the trace info value in order to avoid interfering with the engine processed asynchronous queries
			SightQuery.SetTraceInfo(FTraceHandle());
		}
		return Result;
	}
	else
	{
		// we need to do tests ourselves
		const FCollisionQueryParams QueryParams = FCollisionQueryParams(SCENE_QUERY_STAT(AILineOfSight), true, ListenerActor);
		if (bUseAsynchronousTraceForDefaultSightQueries)
		{
			const FTraceHandle TraceHandle = World->AsyncLineTraceByChannel(EAsyncTraceType::Single, Listener.CachedLocation, TargetLocation, DefaultSightCollisionChannel, QueryParams, FCollisionResponseParams::DefaultResponseParam, &OnPendingTraceQueryProcessedDelegate);
			if (!TraceHandle.IsValid())
			{
				return EVisibilityResult::NotVisible;
			}

			++OutNumberOfAsyncLosCheckRequested;

			// store the trace handle information here so that we can identify the associated query when we'll receive the delegate callback
			SightQuery.SetTraceInfo(TraceHandle);
			return EVisibilityResult::Pending;
		}
		else
		{
			FHitResult HitResult;
			const bool bHit = World->LineTraceSingleByChannel(HitResult, Listener.CachedLocation, TargetLocation, DefaultSightCollisionChannel, QueryParams, FCollisionResponseParams::DefaultResponseParam);

			++OutNumberOfLoSChecksPerformed;

			if (UE::AISense_Sight::IsTraceConsideredVisible(bHit ? &HitResult : nullptr, TargetActor))
			{
				OutSeenLocation = TargetLocation;
				return EVisibilityResult::Visible;
			}
			else
			{
				return EVisibilityResult::NotVisible;
			}
		}
	}
}

void UAISense_Sight::UpdateQueryVisibilityStatus(FAISightQuery& SightQuery, FPerceptionListener& Listener, const bool bIsVisible, const FVector& SeenLocation, const float StimulusStrength, AActor* TargetActor, const FVector& TargetLocation) const
{
	if (TargetActor)
	{
		UpdateQueryVisibilityStatus(SightQuery, Listener, bIsVisible, SeenLocation, StimulusStrength, *TargetActor, TargetLocation);
	}
}

void UAISense_Sight::UpdateQueryVisibilityStatus(FAISightQuery& SightQuery, FPerceptionListener& Listener, const bool bIsVisible, const FVector& SeenLocation, const float StimulusStrength, AActor& TargetActor, const FVector& TargetLocation) const
{
	if (bIsVisible)
	{
		const bool bHasValidSeenLocation = SeenLocation != FAISystem::InvalidLocation;
		Listener.RegisterStimulus(&TargetActor, FAIStimulus(*this, StimulusStrength, bHasValidSeenLocation ? SeenLocation : SightQuery.LastSeenLocation, Listener.CachedLocation));
		SightQuery.SetLastResult(true);
		if (bHasValidSeenLocation)
		{
			SightQuery.LastSeenLocation = SeenLocation;
		}
	}
	// communicate failure only if we've seen given actor before
	else if (SightQuery.GetLastResult())
	{
		Listener.RegisterStimulus(&TargetActor, FAIStimulus(*this, 0.f, TargetLocation, Listener.CachedLocation, FAIStimulus::SensingFailed));
		SightQuery.SetLastResult(false);
		SightQuery.LastSeenLocation = FAISystem::InvalidLocation;
	}

	SIGHT_LOG_SEGMENT(Listener.GetBodyActor(), Listener.CachedLocation, TargetLocation, bIsVisible ? FColor::Green : FColor::Red, TEXT("Target: %s"), *TargetActor.GetName());
}

void UAISense_Sight::OnPendingCanBeSeenQueryProcessed(const FAISightQueryID& QueryID, const bool bIsVisible, const float StimulusStrength, const FVector& SeenLocation, const TOptional<int32>& UserData)
{
	SCOPE_CYCLE_COUNTER(STAT_AI_Sense_Sight_ProcessPendingQuery);

	UE_MT_SCOPED_WRITE_ACCESS(QueriesListAccessDetector);

	const int32 QueryIdx = SightQueriesPending.IndexOfByPredicate([&QueryID](const FAISightQuery& Element)
		{
			return Element.ObserverId == QueryID.ObserverId
				&& Element.TargetId == QueryID.TargetId;
		});

	if (QueryIdx == INDEX_NONE)
	{
		// the query is not pending. It must have been removed because the source or the target have been removed
		return;
	}

	OnPendingQueryProcessed(QueryIdx, bIsVisible, StimulusStrength, SeenLocation, UserData);
}

void UAISense_Sight::OnPendingTraceQueryProcessed(const FTraceHandle& TraceHandle, FTraceDatum& TraceDatum)
{
	SCOPE_CYCLE_COUNTER(STAT_AI_Sense_Sight_ProcessPendingQuery);
	UE_MT_SCOPED_WRITE_ACCESS(QueriesListAccessDetector);

	const int32 QueryIdx = SightQueriesPending.IndexOfByPredicate([&TraceHandle](const FAISightQuery& Element)
	{
		return Element.TraceInfo.FrameNumber == TraceHandle._Data.FrameNumber
			&& Element.TraceInfo.Index == TraceHandle._Data.Index;
	});

	if (QueryIdx == INDEX_NONE)
	{
		// the query is not pending. It must have been removed because the source or the target have been removed
		return;
	}

	AActor* TargetActor = nullptr;
	if (const FAISightTarget* Target = ObservedTargets.Find(SightQueriesPending[QueryIdx].TargetId))
	{
		TargetActor = Target->Target.Get();
	}
	const bool bIsVisible = UE::AISense_Sight::IsTraceConsideredVisible(TraceDatum.OutHits.Num() > 0 ? &TraceDatum.OutHits[0] : nullptr, TargetActor);

	OnPendingQueryProcessed(QueryIdx, bIsVisible, DefaultStimulusStrength, TraceDatum.End, NullOpt, TargetActor);
}

void UAISense_Sight::OnPendingQueryProcessed(const int32 SightQueryIndex, const bool bIsVisible, const float StimulusStrength, const FVector& SeenLocation, const TOptional<int32>& UserData, const TOptional<AActor*> InTargetActor)
{
	FAISightQuery SightQuery = SightQueriesPending[SightQueryIndex];
	SightQueriesPending.RemoveAtSwap(SightQueryIndex, EAllowShrinking::No);

	AIPerception::FListenerMap& ListenersMap = *GetListeners();
	FPerceptionListener* Listener = ListenersMap.Find(SightQuery.ObserverId);
	if (Listener == nullptr)
	{
		return;
	}

	AActor* TargetActor = nullptr;
	if (InTargetActor.IsSet())
	{
		TargetActor = InTargetActor.GetValue();
	}
	else
	{
		const FAISightTarget* Target = ObservedTargets.Find(SightQuery.TargetId);
		TargetActor = Target ? Target->Target.Get() : nullptr;
	}

	if (TargetActor == nullptr)
	{
		return;
	}

	const bool bWasVisible = SightQuery.GetLastResult();
	const FVector TargetLocation = TargetActor->GetActorLocation();
	UpdateQueryVisibilityStatus(SightQuery, *Listener, bIsVisible, SeenLocation, StimulusStrength, *TargetActor, TargetLocation);

	if (UserData.IsSet())
	{
		SightQuery.UserData = UserData.GetValue();
	}

	// Call this to be able to have an accurate tick time
	SightQuery.OnProcessed();

	const FDigestedSightProperties& PropDigest = DigestedProperties[SightQuery.ObserverId];
	const float SightRadiusSq = bWasVisible ? PropDigest.LoseSightRadiusSq : PropDigest.SightRadiusSq;
	SightQuery.Importance = CalcQueryImportance(*Listener, TargetLocation, SightRadiusSq);
	const bool bShouldBeInRange = SightQuery.Importance > 0.0f;
	if (bShouldBeInRange)
	{
		SightQueriesInRange.Add(SightQuery);
	}
	else
	{
		if (bSightQueriesOutOfRangeDirty)
		{
			SightQueriesOutOfRange.Add(SightQuery); 
		}
		else
		{
			SightQueriesOutOfRange.Insert(SightQuery, NextOutOfRangeIndex);
			++NextOutOfRangeIndex;
		}
	}
}

void UAISense_Sight::RegisterEvent(const FAISightEvent& Event)
{

}

void UAISense_Sight::RegisterSource(AActor& SourceActor)
{
	RegisterTarget(SourceActor);
}

void UAISense_Sight::UnregisterSource(AActor& SourceActor)
{
	UE_MT_SCOPED_WRITE_ACCESS(QueriesListAccessDetector);

	const FAISightTarget::FTargetId AsTargetId = SourceActor.GetUniqueID();
	FAISightTarget AsTarget;

	if (ObservedTargets.RemoveAndCopyValue(AsTargetId, AsTarget) 
		&& (SightQueriesInRange.Num() + SightQueriesOutOfRange.Num() + SightQueriesPending.Num()) > 0)
	{
		AActor* TargetActor = AsTarget.Target.Get();

		// notify all interested observers that this source is no longer
		// visible		
		AIPerception::FListenerMap& ListenersMap = *GetListeners();
		auto RemoveQuery = [this,&ListenersMap,&AsTargetId,&TargetActor](TArray<FAISightQuery>& SightQueries, const int32 QueryIndex)->EReverseForEachResult
		{
			FAISightQuery* SightQuery = &SightQueries[QueryIndex];
			if (SightQuery->TargetId == AsTargetId)
			{
				if (SightQuery->GetLastResult() && TargetActor)
				{
					FPerceptionListener& Listener = ListenersMap[SightQuery->ObserverId];
					ensure(Listener.Listener.IsValid());

					Listener.RegisterStimulus(TargetActor, FAIStimulus(*this, 0.f, SightQuery->LastSeenLocation, Listener.CachedLocation, FAIStimulus::SensingFailed));
				}

				SightQueries.RemoveAtSwap(QueryIndex, EAllowShrinking::No);
				return EReverseForEachResult::Modified;
			}
			return EReverseForEachResult::UnTouched;
		};
		ReverseForEach(SightQueriesInRange, RemoveQuery);
		if (ReverseForEach(SightQueriesOutOfRange, RemoveQuery) == EReverseForEachResult::Modified)
		{
			bSightQueriesOutOfRangeDirty = true;
		}
		ReverseForEach(SightQueriesPending, RemoveQuery);
	}
}

bool UAISense_Sight::RegisterTarget(AActor& TargetActor, const TFunction<void(FAISightQuery&)>& OnAddedFunc /*= nullptr*/)
{
	SCOPE_CYCLE_COUNTER(STAT_AI_Sense_Sight_RegisterTarget);
	
	FAISightTarget* SightTarget = ObservedTargets.Find(TargetActor.GetUniqueID());
	
	// Check if the target is recycled OR new
	if (SightTarget == nullptr || SightTarget->GetTargetActor() != &TargetActor)
	{
		FAISightTarget NewSightTarget(&TargetActor);

		SightTarget = &(ObservedTargets.Add(NewSightTarget.TargetId, NewSightTarget));

		// we're looking at components first and only if nothing is found we proceed to check 
		// if the TargetActor implements IAISightTargetInterface. The advantage of doing it in 
		// this order is that you can have components override the original Actor's implementation
		if (IAISightTargetInterface* InterfaceComponent = TargetActor.FindComponentByInterface<IAISightTargetInterface>())
		{
			SightTarget->WeakSightTargetInterface = InterfaceComponent;
		}
		else 
		{
			SightTarget->WeakSightTargetInterface = Cast<IAISightTargetInterface>(&TargetActor);
		}
	}

	// set/update data
	SightTarget->TeamId = FGenericTeamId::GetTeamIdentifier(&TargetActor);

	// generate all pairs and add them to current Sight Queries
	bool bNewQueriesAdded = false;
	AIPerception::FListenerMap& ListenersMap = *GetListeners();
	const FVector TargetLocation = TargetActor.GetActorLocation();

	for (AIPerception::FListenerMap::TConstIterator ItListener(ListenersMap); ItListener; ++ItListener)
	{
		const FPerceptionListener& Listener = ItListener->Value;
		if (!Listener.HasSense(GetSenseID()) || Listener.GetBodyActor() == &TargetActor)
		{
			continue;
		}

		const FDigestedSightProperties& PropDigest = DigestedProperties[Listener.GetListenerID()];
		const IGenericTeamAgentInterface* ListenersTeamAgent = Listener.GetTeamAgent();
		if (RegisterNewQuery(Listener, ListenersTeamAgent, TargetActor, SightTarget->TargetId, TargetLocation, PropDigest, OnAddedFunc))
		{
			bNewQueriesAdded = true;
		}
	}

	// sort Sight Queries
	if (bNewQueriesAdded)
	{
		RequestImmediateUpdate();
	}

	return bNewQueriesAdded;
}

void UAISense_Sight::OnNewListenerImpl(const FPerceptionListener& NewListener)
{
	UAIPerceptionComponent* NewListenerPtr = NewListener.Listener.Get();
	check(NewListenerPtr);
	const UAISenseConfig_Sight* SenseConfig = Cast<const UAISenseConfig_Sight>(NewListenerPtr->GetSenseConfig(GetSenseID()));
	check(SenseConfig);
	const FDigestedSightProperties PropertyDigest(*SenseConfig);
	DigestedProperties.Add(NewListener.GetListenerID(), PropertyDigest);

	GenerateQueriesForListener(NewListener, PropertyDigest);
}

void UAISense_Sight::GenerateQueriesForListener(const FPerceptionListener& Listener, const FDigestedSightProperties& PropertyDigest, const TFunction<void(FAISightQuery&)>& OnAddedFunc/*= nullptr */)
{
	bool bNewQueriesAdded = false;
	const IGenericTeamAgentInterface* ListenersTeamAgent = Listener.GetTeamAgent();
	const AActor* Avatar = Listener.GetBodyActor();

	// create sight queries with all legal targets
	for (FTargetsContainer::TConstIterator ItTarget(ObservedTargets); ItTarget; ++ItTarget)
	{
		const AActor* TargetActor = ItTarget->Value.GetTargetActor();
		if (TargetActor == nullptr || TargetActor == Avatar)
		{
			continue;
		}

		const FVector TargetLocation = TargetActor->GetActorLocation();
		if (RegisterNewQuery(Listener, ListenersTeamAgent, *TargetActor, ItTarget->Key, TargetLocation, PropertyDigest, OnAddedFunc))
		{
			bNewQueriesAdded = true;
		}
	}

	// sort Sight Queries
	if (bNewQueriesAdded)
	{
		RequestImmediateUpdate();
	}
}

bool UAISense_Sight::RegisterNewQuery(const FPerceptionListener& Listener, const IGenericTeamAgentInterface* ListenersTeamAgent, const AActor& TargetActor, const FAISightTarget::FTargetId& TargetId, const FVector& TargetLocation, const FDigestedSightProperties& PropDigest, const TFunction<void(FAISightQuery&)>& OnAddedFunc)
{
	if (!FAISenseAffiliationFilter::ShouldSenseTeam(ListenersTeamAgent, TargetActor, PropDigest.AffiliationFlags))
	{
		return false;
	}

	// create a sight query
	const float Importance = CalcQueryImportance(Listener, TargetLocation, PropDigest.SightRadiusSq);
	const bool bInRange = Importance > 0.0f;
	if (!bInRange)
	{
		bSightQueriesOutOfRangeDirty = true;
	}

	FAISightQuery& AddedQuery = bInRange ? SightQueriesInRange.AddDefaulted_GetRef() : SightQueriesOutOfRange.AddDefaulted_GetRef();
	AddedQuery.ObserverId = Listener.GetListenerID();
	AddedQuery.TargetId = TargetId;
	AddedQuery.Importance = Importance;

	if (OnAddedFunc)
	{
		OnAddedFunc(AddedQuery);
	}

	return true;
}

void UAISense_Sight::OnListenerUpdateImpl(const FPerceptionListener& UpdatedListener)
{
	SCOPE_CYCLE_COUNTER(STAT_AI_Sense_Sight_ListenerUpdate);

	// first, naive implementation:
	// 1. remove all queries by this listener
	// 2. proceed as if it was a new listener

	// see if this listener is a Target as well
	const FAISightTarget::FTargetId AsTargetId = UpdatedListener.GetBodyActorUniqueID();
	FAISightTarget* AsTarget = ObservedTargets.Find(AsTargetId);
	if (AsTarget != nullptr)
	{
		if (AsTarget->Target.IsValid())
		{
			// if still a valid target then backup list of observers for which the listener was visible to restore in the newly created queries
			TSet<FPerceptionListenerID> LastVisibleObservers;
			RemoveAllQueriesToTarget(AsTargetId, [&LastVisibleObservers](const FAISightQuery& Query)
			{
				if (Query.GetLastResult())
				{
					LastVisibleObservers.Add(Query.ObserverId);
				}
			});

			RegisterTarget(*(AsTarget->Target.Get()), [&LastVisibleObservers](FAISightQuery& Query)
			{
				Query.SetLastResult(LastVisibleObservers.Contains(Query.ObserverId));
			});
		}
		else
		{
			RemoveAllQueriesToTarget(AsTargetId);
		}
	}

	const FPerceptionListenerID ListenerID = UpdatedListener.GetListenerID();

	if (UpdatedListener.HasSense(GetSenseID()))
	{
		// if still a valid sense then backup list of targets that were visible by the listener to restore in the newly created queries
		TSet<FAISightTarget::FTargetId> LastVisibleTargets;
		RemoveAllQueriesByListener(UpdatedListener, [&LastVisibleTargets](const FAISightQuery& Query)
		{
			if (Query.GetLastResult())
			{
				LastVisibleTargets.Add(Query.TargetId);
			}
		});

		const UAISenseConfig_Sight* SenseConfig = Cast<const UAISenseConfig_Sight>(UpdatedListener.Listener->GetSenseConfig(GetSenseID()));
		check(SenseConfig);
		FDigestedSightProperties& PropertiesDigest = DigestedProperties.FindOrAdd(ListenerID);
		PropertiesDigest = FDigestedSightProperties(*SenseConfig);

		GenerateQueriesForListener(UpdatedListener, PropertiesDigest, [&LastVisibleTargets](FAISightQuery& Query)
		{
			Query.SetLastResult(LastVisibleTargets.Contains(Query.TargetId));
		});
	}
	else
	{
		// remove all queries
		RemoveAllQueriesByListener(UpdatedListener);

		DigestedProperties.Remove(ListenerID);
	}
}

void UAISense_Sight::OnListenerConfigUpdated(const FPerceptionListener& UpdatedListener)
{
	bool bSkipListenerUpdate = false;
	const FPerceptionListenerID ListenerID = UpdatedListener.GetListenerID();

	FDigestedSightProperties* PropertiesDigest = DigestedProperties.Find(ListenerID);
	if (PropertiesDigest)
	{
		// The only parameter we need to rebuild all the queries for this listener is if the affiliation mask changed, otherwise there is nothing to update.
		const UAISenseConfig_Sight* SenseConfig = CastChecked<const UAISenseConfig_Sight>(UpdatedListener.Listener->GetSenseConfig(GetSenseID()));
		FDigestedSightProperties NewPropertiesDigest(*SenseConfig);
		bSkipListenerUpdate = NewPropertiesDigest.AffiliationFlags == PropertiesDigest->AffiliationFlags;
		*PropertiesDigest = NewPropertiesDigest;
	}

	if (!bSkipListenerUpdate)
	{
		Super::OnListenerConfigUpdated(UpdatedListener);
	}
}


void UAISense_Sight::OnListenerRemovedImpl(const FPerceptionListener& RemovedListener)
{
	RemoveAllQueriesByListener(RemovedListener);

	DigestedProperties.FindAndRemoveChecked(RemovedListener.GetListenerID());

	// note: there use to be code to remove all queries _to_ listener here as well
	// but that was wrong - the fact that a listener gets unregistered doesn't have to
	// mean it's being removed from the game altogether.
}

void UAISense_Sight::RemoveAllQueriesByListener(const FPerceptionListener& Listener, const TFunction<void(const FAISightQuery&)>& OnRemoveFunc/*= nullptr */)
{
	SCOPE_CYCLE_COUNTER(STAT_AI_Sense_Sight_RemoveByListener);
	UE_MT_SCOPED_WRITE_ACCESS(QueriesListAccessDetector);

	const FPerceptionListenerID ListenerId = Listener.GetListenerID();

	auto RemoveQuery = [&ListenerId, &OnRemoveFunc](TArray<FAISightQuery>& SightQueries, const int32 QueryIndex)->EReverseForEachResult
	{
		const FAISightQuery& SightQuery = SightQueries[QueryIndex];

		if (SightQuery.ObserverId == ListenerId)
		{
			if (OnRemoveFunc)
			{
				OnRemoveFunc(SightQuery);
			}
			SightQueries.RemoveAtSwap(QueryIndex, EAllowShrinking::No);
			return EReverseForEachResult::Modified;
		}
		return EReverseForEachResult::UnTouched;
	};
	ReverseForEach(SightQueriesInRange, RemoveQuery);
	if(ReverseForEach(SightQueriesOutOfRange, RemoveQuery) == EReverseForEachResult::Modified)
	{
		bSightQueriesOutOfRangeDirty = true;
	}
	ReverseForEach(SightQueriesPending, RemoveQuery);
}

void UAISense_Sight::RemoveAllQueriesToTarget(const FAISightTarget::FTargetId& TargetId, const TFunction<void(const FAISightQuery&)>& OnRemoveFunc/*= nullptr */)
{
	UE_MT_SCOPED_WRITE_ACCESS(QueriesListAccessDetector);
	RemoveAllQueriesToTarget_Internal(TargetId, OnRemoveFunc);
}

void UAISense_Sight::RemoveAllQueriesToTarget_Internal(const FAISightTarget::FTargetId& TargetId, const TFunction<void(const FAISightQuery&)>& OnRemoveFunc/*= nullptr */)
{
	SCOPE_CYCLE_COUNTER(STAT_AI_Sense_Sight_RemoveToTarget);

	auto RemoveQuery = [&TargetId, &OnRemoveFunc](TArray<FAISightQuery>& SightQueries, const int32 QueryIndex)->EReverseForEachResult
	{
		const FAISightQuery& SightQuery = SightQueries[QueryIndex];

		if (SightQuery.TargetId == TargetId)
		{
			if (OnRemoveFunc)
			{
				OnRemoveFunc(SightQuery);
			}
			SightQueries.RemoveAtSwap(QueryIndex, EAllowShrinking::No);
			return EReverseForEachResult::Modified;
		}
		return EReverseForEachResult::UnTouched;
	};
	ReverseForEach(SightQueriesInRange, RemoveQuery);
	if (ReverseForEach(SightQueriesOutOfRange, RemoveQuery) == EReverseForEachResult::Modified)
	{
		bSightQueriesOutOfRangeDirty = true;
	}
	ReverseForEach(SightQueriesPending, RemoveQuery);
}

void UAISense_Sight::OnListenerForgetsActor(const FPerceptionListener& Listener, AActor& ActorToForget)
{
	const FPerceptionListenerID ListenerId = Listener.GetListenerID();
	const uint32 TargetId = ActorToForget.GetUniqueID();
	
	auto ForgetPreviousResult = [&ListenerId, &TargetId](FAISightQuery& SightQuery)->EForEachResult
	{
		if (SightQuery.ObserverId == ListenerId && SightQuery.TargetId == TargetId)
		{
			// assuming one query per observer-target pair
			SightQuery.ForgetPreviousResult();
			return EForEachResult::Break;
		}
		return EForEachResult::Continue;
	};

	if (ForEach(SightQueriesInRange, ForgetPreviousResult) == EForEachResult::Continue)
	{
		if (ForEach(SightQueriesOutOfRange, ForgetPreviousResult) == EForEachResult::Continue)
		{
			ForEach(SightQueriesPending, ForgetPreviousResult);
		}
	}
}

void UAISense_Sight::OnListenerForgetsAll(const FPerceptionListener& Listener)
{
	UE_MT_SCOPED_WRITE_ACCESS(QueriesListAccessDetector);

	const FPerceptionListenerID ListenerId = Listener.GetListenerID();

	auto ForgetPreviousResult = [&ListenerId](FAISightQuery& SightQuery)->EForEachResult
	{
		if (SightQuery.ObserverId == ListenerId)
		{
			SightQuery.ForgetPreviousResult();
		}
		return EForEachResult::Continue;
	};

	ForEach(SightQueriesInRange, ForgetPreviousResult);
	ForEach(SightQueriesOutOfRange, ForgetPreviousResult);
	ForEach(SightQueriesPending, ForgetPreviousResult);
}

#if WITH_GAMEPLAY_DEBUGGER_MENU
void UAISense_Sight::DescribeSelfToGameplayDebugger(const UAIPerceptionSystem& PerceptionSystem, FGameplayDebuggerCategory& DebuggerCategory) const
{
	const int32 TotalQueriesCount = SightQueriesInRange.Num() + SightQueriesOutOfRange.Num() + SightQueriesPending.Num();
	DebuggerCategory.AddTextLine(
		FString::Printf(TEXT("%s: %d Targets, %d Queries (InRange:%d, OutOfRange:%d, Pending:%d)"),
			*GetSenseID().Name.ToString(),
			ObservedTargets.Num(),
			TotalQueriesCount,
			SightQueriesInRange.Num(),
			SightQueriesOutOfRange.Num(),
			SightQueriesPending.Num())
		);
}
#endif // WITH_GAMEPLAY_DEBUGGER_MENU
