// Copyright Epic Games, Inc. All Rights Reserved.

#include "Perception/AISense_Team.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AISense_Team)

//----------------------------------------------------------------------//
// 
//----------------------------------------------------------------------//
FAITeamStimulusEvent::FAITeamStimulusEvent(AActor* InBroadcaster, AActor* InEnemy, const FVector& InLastKnowLocation, float EventRange, float PassedInfoAge, float InStrength)
	: LastKnowLocation(InLastKnowLocation), RangeSq(FMath::Square(EventRange)), InformationAge(PassedInfoAge), Strength(InStrength), Broadcaster(InBroadcaster), Enemy(InEnemy)
{
	CacheBroadcastLocation();

	TeamIdentifier = FGenericTeamId::GetTeamIdentifier(InBroadcaster);
}

//----------------------------------------------------------------------//
// 
//----------------------------------------------------------------------//
UAISense_Team::UAISense_Team(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer)
{
}

float UAISense_Team::Update()
{
	AIPerception::FListenerMap& ListenersMap = *GetListeners();
	
	for (AIPerception::FListenerMap::TIterator ListenerIt(ListenersMap); ListenerIt; ++ListenerIt)
	{
		FPerceptionListener& Listener = ListenerIt->Value;

		if (Listener.HasSense(GetSenseID()) == false)
		{
			// skip listeners not interested in this sense
			continue;
		}

		for (const FAITeamStimulusEvent& Event : RegisteredEvents)
		{
			// @todo implement some kind of TeamIdentifierType that would supply comparison operator 
			if (Listener.TeamIdentifier != Event.TeamIdentifier 
				|| FVector::DistSquared(Event.GetBroadcastLocation(), Listener.CachedLocation) > Event.RangeSq)
			{
				continue;
			}
			
			Listener.RegisterStimulus(Event.Enemy, FAIStimulus(*this, Event.Strength, Event.LastKnowLocation, Event.GetBroadcastLocation(), FAIStimulus::SensingSucceeded).SetStimulusAge(Event.InformationAge));
		}
	}

	RegisteredEvents.Reset();

	// return decides when next tick is going to happen
	return SuspendNextUpdate;
}

void UAISense_Team::RegisterEvent(const FAITeamStimulusEvent& Event)
{
	RegisteredEvents.Add(Event);

	RequestImmediateUpdate();
}

