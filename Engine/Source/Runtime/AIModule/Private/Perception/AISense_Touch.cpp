// Copyright Epic Games, Inc. All Rights Reserved.

#include "Perception/AISense_Touch.h"
#include "GameFramework/Controller.h"
#include "GameFramework/Pawn.h"
#include "Perception/AIPerceptionListenerInterface.h"
#include "Perception/AIPerceptionComponent.h"
#include "Perception/AISenseConfig_Touch.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AISense_Touch)

//----------------------------------------------------------------------//
// FAITouchEvent
//----------------------------------------------------------------------//
IAIPerceptionListenerInterface* FAITouchEvent::GetTouchedActorAsPerceptionListener() const
{
	IAIPerceptionListenerInterface* Listener = nullptr;
	if (TouchReceiver)
	{
		Listener = Cast<IAIPerceptionListenerInterface>(TouchReceiver);
		if (Listener == nullptr)
		{
			const APawn* ListenerAsPawn = Cast<APawn>(TouchReceiver);
			if (ListenerAsPawn)
			{
				Listener = Cast<IAIPerceptionListenerInterface>(ListenerAsPawn->GetController());
			}
		}
	}
	return Listener;
}

//----------------------------------------------------------------------//
// FDigestedHearingProperties
//----------------------------------------------------------------------//
UAISense_Touch::FDigestedTouchProperties::FDigestedTouchProperties(const UAISenseConfig_Touch& SenseConfig)
{
	AffiliationFlags = SenseConfig.DetectionByAffiliation.GetAsFlags();
}

UAISense_Touch::FDigestedTouchProperties::FDigestedTouchProperties()
{
	AffiliationFlags = FAISenseAffiliationFilter::DetectAllFlags();
}

//----------------------------------------------------------------------//
// UAISense_Touch
//----------------------------------------------------------------------//
UAISense_Touch::UAISense_Touch(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer)
{
	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		OnNewListenerDelegate.BindUObject(this, &UAISense_Touch::OnNewListenerImpl);
		OnListenerUpdateDelegate.BindUObject(this, &UAISense_Touch::OnListenerUpdateImpl);
		OnListenerRemovedDelegate.BindUObject(this, &UAISense_Touch::OnListenerRemovedImpl);
	}
}

void UAISense_Touch::OnNewListenerImpl(const FPerceptionListener& NewListener)
{
	UAIPerceptionComponent* ListenerPtr = NewListener.Listener.Get();
	check(ListenerPtr);
	const UAISenseConfig_Touch* SenseConfig = Cast<const UAISenseConfig_Touch>(ListenerPtr->GetSenseConfig(GetSenseID()));
	check(SenseConfig);
	const FDigestedTouchProperties PropertyDigest(*SenseConfig);
	DigestedProperties.Add(NewListener.GetListenerID(), PropertyDigest);
}

void UAISense_Touch::OnListenerUpdateImpl(const FPerceptionListener& UpdatedListener)
{
	const FPerceptionListenerID ListenerID = UpdatedListener.GetListenerID();
	
	if (UpdatedListener.HasSense(GetSenseID()))
	{
		const UAISenseConfig_Touch* SenseConfig = Cast<const UAISenseConfig_Touch>(UpdatedListener.Listener->GetSenseConfig(GetSenseID()));
		check(SenseConfig);
		FDigestedTouchProperties& PropertiesDigest = DigestedProperties.FindOrAdd(ListenerID);
		PropertiesDigest = FDigestedTouchProperties(*SenseConfig);
	}
	else
	{
		DigestedProperties.Remove(ListenerID);
	}
}

void UAISense_Touch::OnListenerRemovedImpl(const FPerceptionListener& RemovedListener)
{
	DigestedProperties.FindAndRemoveChecked(RemovedListener.GetListenerID());
}

float UAISense_Touch::Update()
{
	AIPerception::FListenerMap& ListenersMap = *GetListeners();

	for (const FAITouchEvent& Event : RegisteredEvents)
	{
		if (Event.TouchReceiver != nullptr && Event.OtherActor != nullptr)
		{
			IAIPerceptionListenerInterface* PerceptionListener = Event.GetTouchedActorAsPerceptionListener();
			if (PerceptionListener != nullptr)
			{
				const UAIPerceptionComponent* PerceptionComponent = PerceptionListener->GetPerceptionComponent();
				if (PerceptionComponent != nullptr && ListenersMap.Contains(PerceptionComponent->GetListenerId()))
				{
					// this has to succeed, will assert a failure
					FPerceptionListener& Listener = ListenersMap[PerceptionComponent->GetListenerId()];
					if (Listener.HasSense(GetSenseID()))
					{
						const FDigestedTouchProperties* PropDigest = DigestedProperties.Find(Listener.GetListenerID());
						if (PropDigest && FAISenseAffiliationFilter::ShouldSenseTeam(Listener.TeamIdentifier, Event.TeamIdentifier, PropDigest->AffiliationFlags) == false)
						{
							continue;
						}
						Listener.RegisterStimulus(Event.OtherActor, FAIStimulus(*this, 1.f, Event.Location, Event.Location));
					}
				}
			}
		}
	}

	RegisteredEvents.Reset();

	// return decides when next tick is going to happen
	return SuspendNextUpdate;
}

void UAISense_Touch::RegisterEvent(const FAITouchEvent& Event)
{
	RegisteredEvents.Add(Event);

	RequestImmediateUpdate();
}

void UAISense_Touch::ReportTouchEvent(UObject* WorldContextObject, AActor* TouchReceiver, AActor* OtherActor, FVector Location)
{
	UAIPerceptionSystem* PerceptionSystem = UAIPerceptionSystem::GetCurrent(WorldContextObject);
	if (PerceptionSystem)
	{
		const FAITouchEvent Event(TouchReceiver, OtherActor, Location);
		PerceptionSystem->OnEvent(Event);
	}
}

