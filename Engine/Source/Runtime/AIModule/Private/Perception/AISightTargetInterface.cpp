// Copyright Epic Games, Inc. All Rights Reserved.

#include "Perception/AISightTargetInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AISightTargetInterface)

UAISightTargetInterface::UAISightTargetInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

