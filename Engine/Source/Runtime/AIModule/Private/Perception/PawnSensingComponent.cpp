// Copyright Epic Games, Inc. All Rights Reserved.

#include "Perception/PawnSensingComponent.h"
#include "EngineGlobals.h"
#include "TimerManager.h"
#include "CollisionQueryParams.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/Controller.h"
#include "GameFramework/PlayerController.h"
#include "Engine/Engine.h"
#include "EngineUtils.h"
#include "AIController.h"
#include "Components/PawnNoiseEmitterComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PawnSensingComponent)

DECLARE_CYCLE_STAT(TEXT("Sensing"),STAT_AI_Sensing,STATGROUP_AI);

UPawnSensingComponent::UPawnSensingComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	LOSHearingThreshold = 2800.f;
	HearingThreshold = 1400.0f;
	SightRadius = 5000.0f;
	PeripheralVisionAngle = 90.f;
	PeripheralVisionCosine = FMath::Cos(FMath::DegreesToRadians(PeripheralVisionAngle));
	
	SensingInterval = 0.5f;
	HearingMaxSoundAge = 1.f;

	bOnlySensePlayers = true;
	bHearNoises = true;
	bSeePawns = true;

	PrimaryComponentTick.bCanEverTick = false;
	bWantsInitializeComponent = true;
	bAutoActivate = false;

	bEnableSensingUpdates = true;
}


void UPawnSensingComponent::SetPeripheralVisionAngle(const float NewPeripheralVisionAngle)
{
	PeripheralVisionAngle = NewPeripheralVisionAngle;
	PeripheralVisionCosine = FMath::Cos(FMath::DegreesToRadians(PeripheralVisionAngle));
}


void UPawnSensingComponent::InitializeComponent()
{
	Super::InitializeComponent();
	SetPeripheralVisionAngle(PeripheralVisionAngle);
	
	if (bEnableSensingUpdates)
	{
		bEnableSensingUpdates = false; // force an update
		SetSensingUpdatesEnabled(true);
	}
}


void UPawnSensingComponent::SetSensingUpdatesEnabled(const bool bEnabled)
{
	if (bEnableSensingUpdates != bEnabled)
	{
		bEnableSensingUpdates = bEnabled;

		if (bEnabled && SensingInterval > 0.f)
		{
			// Stagger initial updates so all sensors do not update at the same time (to avoid hitches).
			const float InitialDelay = (SensingInterval * FMath::SRand()) + KINDA_SMALL_NUMBER;
			SetTimer(InitialDelay);
		}
		else
		{
			SetTimer(0.f);
		}
	}	
}


void UPawnSensingComponent::SetTimer(const float TimeInterval)
{
	// Only necessary to update if we are the server
	AActor* const Owner = GetOwner();
	if (IsValid(Owner) && GEngine->GetNetMode(GetWorld()) < NM_Client)
	{
		Owner->GetWorldTimerManager().SetTimer(TimerHandle_OnTimer, this, &UPawnSensingComponent::OnTimer, TimeInterval, false);
	}
}


void UPawnSensingComponent::SetSensingInterval(const float NewSensingInterval)
{
	if (SensingInterval != NewSensingInterval)
	{
		SensingInterval = NewSensingInterval;

		AActor* const Owner = GetOwner();
		if (IsValid(Owner))
		{
			if (SensingInterval <= 0.f)
			{
				SetTimer(0.f);
			}
			else if (bEnableSensingUpdates)
			{
				float CurrentElapsed = Owner->GetWorldTimerManager().GetTimerElapsed(TimerHandle_OnTimer);
				CurrentElapsed = FMath::Max(0.f, CurrentElapsed);

				if (CurrentElapsed < SensingInterval)
				{
					// Extend lifetime by remaining time.
					SetTimer(SensingInterval - CurrentElapsed);
				}
				else if (CurrentElapsed > SensingInterval)
				{
					// Basically fire next update, because time has already expired.
					// Don't want to fire immediately in case an update tries to change the interval, looping endlessly.
					SetTimer(KINDA_SMALL_NUMBER);
				}
			}		
		}
	}
}


void UPawnSensingComponent::OnTimer()
{
	SCOPE_CYCLE_COUNTER(STAT_AI_Sensing);

	AActor* const Owner = GetOwner();
	if (!IsValid(Owner) || !IsValid(Owner->GetWorld()))
	{
		return;
	}

	if (CanSenseAnything())
	{
		UpdateAISensing();
	}
	
	if (bEnableSensingUpdates)
	{
		SetTimer(SensingInterval);
	}
};


AActor* UPawnSensingComponent::GetSensorActor() const
{
	AActor* SensorActor = GetOwner();
	AController* Controller = Cast<AController>(SensorActor);
	if (IsValid(Controller))
	{
		// In case the owner is a controller, use the controlled pawn as the sensing location.
		SensorActor = Controller->GetPawn();
	}

	if (!IsValid(SensorActor))
	{
		return NULL;
	}

	return SensorActor;
}

AController* UPawnSensingComponent::GetSensorController() const
{
	AActor* SensorActor = GetOwner();
	AController* Controller = Cast<AController>(SensorActor);

	if (IsValid(Controller))
	{
		return Controller;
	}
	else
	{
		APawn* Pawn = Cast<APawn>(SensorActor);
		if (IsValid(Pawn) && IsValid(Pawn->GetController()))
		{
			return Pawn->GetController();
		}
	}

	return NULL;
}

bool UPawnSensingComponent::IsSensorActor(const AActor* Actor) const
{
	return (Actor == GetSensorActor());
}

bool UPawnSensingComponent::HasLineOfSightTo(const AActor* Other) const
{
	AController* SensorController = GetSensorController();
	if (SensorController == NULL)
	{
		return false;
	}

	return SensorController->LineOfSightTo(Other, FVector::ZeroVector, true);
}

bool UPawnSensingComponent::CanSenseAnything() const
{
	return ((bHearNoises && OnHearNoise.IsBound()) ||
			(bSeePawns && OnSeePawn.IsBound()));
}


void UPawnSensingComponent::UpdateAISensing()
{
	AActor* const Owner = GetOwner();
	check(IsValid(Owner));
	check(IsValid(Owner->GetWorld()));

	if (bOnlySensePlayers)
	{
		for (FConstPlayerControllerIterator Iterator = Owner->GetWorld()->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			APlayerController* PC = Iterator->Get();
			if (IsValid(PC))
			{
				APawn* Pawn = PC->GetPawn();
				if (IsValid(Pawn) && !IsSensorActor(Pawn))
				{
					SensePawn(*Pawn);
				}
			}
		}
	}
	else
	{
		for (APawn* Pawn : TActorRange<APawn>(Owner->GetWorld()))
		{
			if (!IsSensorActor(Pawn))
			{
				SensePawn(*Pawn);
			}
		}
	}	
}


void UPawnSensingComponent::SensePawn(APawn& Pawn)
{
	// Visibility checks
	bool bHasSeenPawn = false;
	bool bHasFailedLineOfSightCheck = false;
	if (bSeePawns && ShouldCheckVisibilityOf(&Pawn))
	{
		if (CouldSeePawn(&Pawn, true))
		{
			if (HasLineOfSightTo(&Pawn))
			{
				BroadcastOnSeePawn(Pawn);
				bHasSeenPawn = true;
			}
			else
			{
				bHasFailedLineOfSightCheck = true;
			}
		}
	}

	// Sound checks
	// No need to 'hear' something if you've already seen it!
	if (bHasSeenPawn)
	{
		return;
	}

	// Might not be able to hear or react to the sound at all...
	if (!bHearNoises || !OnHearNoise.IsBound())
	{
		return;
	}

	const UPawnNoiseEmitterComponent* NoiseEmitterComponent = Pawn.GetPawnNoiseEmitterComponent();
	if (NoiseEmitterComponent && ShouldCheckAudibilityOf(&Pawn))
	{
		// ToDo: This should all still be refactored more significantly.  There's no reason that we should have to
		// explicitly check "local" and "remote" (i.e. Pawn-emitted and other-source-emitted) sounds separately here.
		// The noise emitter should handle all of those details for us so the sensing component doesn't need to know about
		// them at all!
		if (IsNoiseRelevant(Pawn, *NoiseEmitterComponent, true) && CanHear(Pawn.GetActorLocation(), NoiseEmitterComponent->GetLastNoiseVolume(true), bHasFailedLineOfSightCheck))
		{
			BroadcastOnHearLocalNoise(Pawn, Pawn.GetActorLocation(), NoiseEmitterComponent->GetLastNoiseVolume(true));
		}
		else if (IsNoiseRelevant(Pawn, *NoiseEmitterComponent, false) && CanHear(NoiseEmitterComponent->LastRemoteNoisePosition, NoiseEmitterComponent->GetLastNoiseVolume(false), false))
		{
			BroadcastOnHearRemoteNoise(Pawn, NoiseEmitterComponent->LastRemoteNoisePosition, NoiseEmitterComponent->GetLastNoiseVolume(false));
		}
	}
}


void UPawnSensingComponent::BroadcastOnSeePawn(APawn& Pawn)
{
	OnSeePawn.Broadcast(&Pawn);
}

void UPawnSensingComponent::BroadcastOnHearLocalNoise(APawn& Instigator, const FVector& Location, float Volume)
{
	OnHearNoise.Broadcast(&Instigator, Location, Volume);
}

void UPawnSensingComponent::BroadcastOnHearRemoteNoise(APawn& Instigator, const FVector& Location, float Volume)
{
	OnHearNoise.Broadcast(&Instigator, Location, Volume);
}


bool UPawnSensingComponent::CouldSeePawn(const APawn *Other, bool bMaySkipChecks) const
{
	if (!Other)
	{
		return false;
	}

	const AActor* Owner = GetOwner();
	if (!Owner)
	{
		return false;
	}

	FVector const OtherLoc = Other->GetActorLocation();
	FVector const SensorLoc = GetSensorLocation();
	FVector const SelfToOther = OtherLoc - SensorLoc;

	// check max sight distance
	FVector::FReal const SelfToOtherDistSquared = SelfToOther.SizeSquared();
	if (SelfToOtherDistSquared > FMath::Square(SightRadius))
	{
		return false;
	}

	// may skip if more than some fraction of maxdist away (longer time to acquire)
	if (bMaySkipChecks && (FMath::Square(FMath::FRand()) * SelfToOtherDistSquared > FMath::Square(0.4f * SightRadius)))
	{
		return false;
	}

// 	UE_LOGF(LogPath, Warning, "DistanceToOtherSquared = %f, SightRadiusSquared: %f", SelfToOtherDistSquared, FMath::Square(SightRadius));

	// check field of view
	FVector const SelfToOtherDir = SelfToOther.GetSafeNormal();
	FVector const MyFacingDir = GetSensorRotation().Vector();

// 	UE_LOGF(LogPath, Warning, "DotProductFacing: %f, PeripheralVisionCosine: %f", SelfToOtherDir | MyFacingDir, PeripheralVisionCosine);

	return ((SelfToOtherDir | MyFacingDir) >= PeripheralVisionCosine);
}


bool UPawnSensingComponent::IsNoiseRelevant(const APawn& Pawn, const UPawnNoiseEmitterComponent& NoiseEmitterComponent, bool bSourceWithinNoiseEmitter) const
{
	// If sound has no volume, it's not relevant.
	if (NoiseEmitterComponent.GetLastNoiseVolume(bSourceWithinNoiseEmitter) <= 0.f)
	{
		return false;
	}

	float LastNoiseTime = NoiseEmitterComponent.GetLastNoiseTime(bSourceWithinNoiseEmitter);
	// If the sound is too old, it's not relevant.
	if (Pawn.GetWorld()->TimeSince(LastNoiseTime) > HearingMaxSoundAge)
	{
		return false;
	}

	// If there's no sensor actor or sensor was created since the noise was emitted, it's irrelevant.
	AActor* SensorActor = GetSensorActor();
	if (!SensorActor || (LastNoiseTime < SensorActor->CreationTime))
	{
		return false;
	}

	return true;
}

FVector UPawnSensingComponent::GetSensorLocation() const
{
	FVector SensorLocation(FVector::ZeroVector);
	const AActor* SensorActor = GetSensorActor();

	if (SensorActor != NULL)
	{
		FRotator ViewRotation;
		SensorActor->GetActorEyesViewPoint(SensorLocation, ViewRotation);
	}

	return SensorLocation;
}

FRotator UPawnSensingComponent::GetSensorRotation() const
{
	FRotator SensorRotation(FRotator::ZeroRotator);
	const AActor* SensorActor = GetSensorActor();
	
	if (SensorActor != NULL)
	{
		SensorRotation = SensorActor->GetActorRotation();
	}
	
	return SensorRotation;
}

bool UPawnSensingComponent::CanHear(const FVector& NoiseLoc, float Loudness, bool bFailedLOS) const
{
	if (Loudness <= 0.f)
	{
		return false;
	}

	const AActor* const Owner = GetOwner();
	if (!IsValid(Owner))
	{
		return false;
	}

	FVector const HearingLocation = GetSensorLocation();

	FVector::FReal const LoudnessAdjustedDistSq = (HearingLocation - NoiseLoc).SizeSquared()/(Loudness*Loudness);
	if (LoudnessAdjustedDistSq <= FMath::Square(HearingThreshold))
	{
		// Hear even occluded sounds within HearingThreshold
		return true;
	}

	// check if sound close enough to do LOS check, and LOS hasn't already failed
	if (bFailedLOS || (LoudnessAdjustedDistSq > FMath::Square(LOSHearingThreshold)))
	{
		return false;
	}

	// check if sound is occluded
	return !Owner->GetWorld()->LineTraceTestByChannel(HearingLocation, NoiseLoc, ECC_Visibility, FCollisionQueryParams(SCENE_QUERY_STAT(CanHear), true, Owner));
}


bool UPawnSensingComponent::ShouldCheckVisibilityOf(APawn *Pawn) const
{
	const bool bPawnIsPlayer = (Pawn->GetController() && Pawn->GetController()->PlayerState);
	if (!bSeePawns || (bOnlySensePlayers && !bPawnIsPlayer))
	{
		return false;
	}

	if (bPawnIsPlayer && AAIController::AreAIIgnoringPlayers())
	{
		return false;
	}

	if (Pawn->IsHidden())
	{
		return false;
	}

	return true;
}


bool UPawnSensingComponent::ShouldCheckAudibilityOf(APawn* Pawn) const
{
	const bool bPawnIsPlayer = (Pawn->GetController() && Pawn->GetController()->PlayerState);
	if (!bHearNoises || (bOnlySensePlayers && !bPawnIsPlayer))
	{
		return false;
	}

	if (bPawnIsPlayer && AAIController::AreAIIgnoringPlayers())
	{
		return false;
	}

	return true;
}


