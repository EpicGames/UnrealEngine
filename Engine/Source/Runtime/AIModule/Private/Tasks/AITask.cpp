// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/AITask.h"
#include "GameFramework/Pawn.h"
#include "AIController.h"
#include "AIResources.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AITask)


UAITask::UAITask(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Priority = uint8(EAITaskPriority::AutonomousAI);
}

AAIController* UAITask::GetAIControllerForActor(AActor* Actor)
{
	AAIController* Result = Cast<AAIController>(Actor);

	if (Result == nullptr)
	{
		APawn* AsPawn = Cast<APawn>(Actor);
		if (AsPawn != nullptr)
		{
			Result = Cast<AAIController>(AsPawn->GetController());
		}
	}

	return Result;
}

void UAITask::Activate()
{
	Super::Activate();

	if (OwnerController == nullptr)
	{
		AActor* OwnerActor = GetOwnerActor();
		if (OwnerActor)
		{
			OwnerController = GetAIControllerForActor(OwnerActor);
		}
		else
		{
			// ERROR!
		}
	}
}

void UAITask::InitAITask(AAIController& AIOwner, IGameplayTaskOwnerInterface& InTaskOwner, uint8 InPriority)
{
	OwnerController = &AIOwner;
	InitTask(InTaskOwner, InPriority);

	if (InPriority == uint8(EAITaskPriority::AutonomousAI))
	{
		AddRequiredResource<UAIResource_Logic>();
	}
}

void UAITask::InitAITask(AAIController& AIOwner, IGameplayTaskOwnerInterface& InTaskOwner)
{
	InitAITask(AIOwner, InTaskOwner, InTaskOwner.GetGameplayTaskDefaultPriority());
}

void UAITask::RequestAILogicLocking()
{
	AddClaimedResource<UAIResource_Logic>();
}

