// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/AITask_LockLogic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AITask_LockLogic)

UAITask_LockLogic::UAITask_LockLogic(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

