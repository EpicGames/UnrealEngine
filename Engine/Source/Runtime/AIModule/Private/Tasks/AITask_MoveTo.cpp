// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/AITask_MoveTo.h"
#include "UObject/Package.h"
#include "TimerManager.h"
#include "AISystem.h"
#include "AIController.h"
#include "VisualLogger/VisualLogger.h"
#include "AIResources.h"
#include "GameplayTasksComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AITask_MoveTo)

UAITask_MoveTo::UAITask_MoveTo(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsPausable = true;
	MoveRequestID = FAIRequestID::InvalidRequest;

	MoveRequest.SetAcceptanceRadius(GET_AI_CONFIG_VAR(AcceptanceRadius));
	MoveRequest.SetReachTestIncludesAgentRadius(GET_AI_CONFIG_VAR(bFinishMoveOnGoalOverlap));
	MoveRequest.SetAllowPartialPath(GET_AI_CONFIG_VAR(bAcceptPartialPaths));
	MoveRequest.SetUsePathfinding(true);

	AddRequiredResource(UAIResource_Movement::StaticClass());
	AddClaimedResource(UAIResource_Movement::StaticClass());
	
	MoveResult = EPathFollowingResult::Invalid;
	bUseContinuousTracking = false;
}

UAITask_MoveTo* UAITask_MoveTo::AIMoveTo(AAIController* Controller, FVector InGoalLocation, AActor* InGoalActor,
	float AcceptanceRadius, EAIOptionFlag::Type StopOnOverlap, EAIOptionFlag::Type AcceptPartialPath,
	bool bUsePathfinding, bool bLockAILogic, bool bUseContinuousGoalTracking, EAIOptionFlag::Type ProjectGoalOnNavigation,
	EAIOptionFlag::Type RequireNavigableEndLocation)
{
	UAITask_MoveTo* MyTask = Controller ? UAITask::NewAITask<UAITask_MoveTo>(*Controller, EAITaskPriority::High) : nullptr;
	if (MyTask)
	{
		FAIMoveRequest MoveReq;
		if (InGoalActor)
		{
			MoveReq.SetGoalActor(InGoalActor);
		}
		else
		{
			MoveReq.SetGoalLocation(InGoalLocation);
		}

		MoveReq.SetAcceptanceRadius(AcceptanceRadius);
		MoveReq.SetReachTestIncludesAgentRadius(FAISystem::PickAIOption(StopOnOverlap, MoveReq.IsReachTestIncludingAgentRadius()));
		MoveReq.SetAllowPartialPath(FAISystem::PickAIOption(AcceptPartialPath, MoveReq.IsUsingPartialPaths()));
		MoveReq.SetUsePathfinding(bUsePathfinding);
		MoveReq.SetProjectGoalLocation(FAISystem::PickAIOption(ProjectGoalOnNavigation, MoveReq.IsProjectingGoal()));
		MoveReq.SetRequireNavigableEndLocation(FAISystem::PickAIOption(RequireNavigableEndLocation, MoveReq.IsNavigableEndLocationRequired()));
		if (Controller)
		{
			MoveReq.SetNavigationFilter(Controller->GetDefaultNavigationFilterClass());
			MoveReq.SetCanStrafe(Controller->bAllowStrafe);
		}

		MyTask->SetUp(Controller, MoveReq);
		MyTask->SetContinuousGoalTracking(bUseContinuousGoalTracking);

		if (bLockAILogic)
		{
			MyTask->RequestAILogicLocking();
		}
	}

	return MyTask;
}

void UAITask_MoveTo::SetUp(AAIController* Controller, const FAIMoveRequest& InMoveRequest)
{
	OwnerController = Controller;
	MoveRequest = InMoveRequest;
}

void UAITask_MoveTo::SetContinuousGoalTracking(bool bEnable)
{
	bUseContinuousTracking = bEnable;
}

void UAITask_MoveTo::FinishMoveTask(EPathFollowingResult::Type InResult)
{
	if (MoveRequestID.IsValid())
	{
		UPathFollowingComponent* PFComp = OwnerController ? OwnerController->GetPathFollowingComponent() : nullptr;
		if (PFComp && PFComp->GetStatus() != EPathFollowingStatus::Idle)
		{
			ResetObservers();
			PFComp->AbortMove(*this, FPathFollowingResultFlags::OwnerFinished, MoveRequestID);
		}
	}

	MoveResult = InResult;
	EndTask();

	if (InResult == EPathFollowingResult::Invalid)
	{
		OnRequestFailed.Broadcast();
	}
	else
	{
		OnMoveFinished.Broadcast(InResult, OwnerController);
	}

	if (OnMoveTaskFinished.IsBound())
	{
		OnMoveTaskFinished.Broadcast(InResult, OwnerController);
	}
}

void UAITask_MoveTo::Activate()
{
	Super::Activate();

	UE_CVLOG(bUseContinuousTracking, GetGameplayTasksComponent(), LogGameplayTasks, Log, TEXT("Continuous goal tracking requested, moving to: %s"),
		MoveRequest.IsMoveToActorRequest() ? TEXT("actor => looping successful moves!") : TEXT("location => will NOT loop"));

	MoveRequestID = FAIRequestID::InvalidRequest;
	ConditionalPerformMove();
}

void UAITask_MoveTo::ConditionalPerformMove()
{
	if (MoveRequest.IsUsingPathfinding() && OwnerController && OwnerController->ShouldPostponePathUpdates())
	{
		UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Log, TEXT("%s> can't path right now, waiting..."), *GetName());
		OwnerController->GetWorldTimerManager().SetTimer(MoveRetryTimerHandle, this, &UAITask_MoveTo::ConditionalPerformMove, 0.2f, false);
	}
	else
	{
		MoveRetryTimerHandle.Invalidate();
		PerformMove();
	}
}

void UAITask_MoveTo::PerformMove()
{
	UPathFollowingComponent* PFComp = OwnerController ? OwnerController->GetPathFollowingComponent() : nullptr;
	if (PFComp == nullptr)
	{
		FinishMoveTask(EPathFollowingResult::Invalid);
		return;
	}

	ResetObservers();
	ResetTimers();

	// start new move request
	FNavPathSharedPtr FollowedPath;
	const FPathFollowingRequestResult ResultData = OwnerController->MoveTo(MoveRequest, &FollowedPath);

	switch (ResultData.Code)
	{
	case EPathFollowingRequestResult::Failed:
		FinishMoveTask(EPathFollowingResult::Invalid);
		break;

	case EPathFollowingRequestResult::AlreadyAtGoal:
		MoveRequestID = ResultData.MoveId;
		OnRequestFinished(ResultData.MoveId, FPathFollowingResult(EPathFollowingResult::Success, FPathFollowingResultFlags::AlreadyAtGoal));
		break;

	case EPathFollowingRequestResult::RequestSuccessful:
		MoveRequestID = ResultData.MoveId;
		PathFinishDelegateHandle = PFComp->OnRequestFinished.AddUObject(this, &UAITask_MoveTo::OnRequestFinished);
		SetObservedPath(FollowedPath);

		if (IsFinished())
		{
			UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Error, TEXT("%s> re-Activating Finished task!"), *GetName());
		}
		break;

	default:
		checkNoEntry();
		break;
	}
}

void UAITask_MoveTo::Pause()
{
	if (OwnerController && MoveRequestID.IsValid())
	{
		OwnerController->PauseMove(MoveRequestID);
	}

	ResetTimers();
	Super::Pause();
}

void UAITask_MoveTo::Resume()
{
	Super::Resume();

	if (!MoveRequestID.IsValid() || (OwnerController && !OwnerController->ResumeMove(MoveRequestID)))
	{
		UE_CVLOG(MoveRequestID.IsValid(), GetGameplayTasksComponent(), LogGameplayTasks, Log, TEXT("%s> Resume move failed, starting new one."), *GetName());
		ConditionalPerformMove();
	}
}

void UAITask_MoveTo::SetObservedPath(FNavPathSharedPtr InPath)
{
	if (PathUpdateDelegateHandle.IsValid() && Path.IsValid())
	{
		Path->RemoveObserver(PathUpdateDelegateHandle);
	}

	PathUpdateDelegateHandle.Reset();
	
	Path = InPath;
	if (Path.IsValid())
	{
		// disable auto repaths, it will be handled by move task to include ShouldPostponePathUpdates condition
		Path->EnableRecalculationOnInvalidation(false);
		PathUpdateDelegateHandle = Path->AddObserver(FNavigationPath::FPathObserverDelegate::FDelegate::CreateUObject(this, &UAITask_MoveTo::OnPathEvent));
	}
}

void UAITask_MoveTo::ResetObservers()
{
	if (Path.IsValid())
	{
		Path->DisableGoalActorObservation();
	}

	if (PathFinishDelegateHandle.IsValid())
	{
		UPathFollowingComponent* PFComp = OwnerController ? OwnerController->GetPathFollowingComponent() : nullptr;
		if (PFComp)
		{
			PFComp->OnRequestFinished.Remove(PathFinishDelegateHandle);
		}

		PathFinishDelegateHandle.Reset();
	}

	if (PathUpdateDelegateHandle.IsValid())
	{
		if (Path.IsValid())
		{
			Path->RemoveObserver(PathUpdateDelegateHandle);
		}

		PathUpdateDelegateHandle.Reset();
	}
}

void UAITask_MoveTo::ResetTimers()
{
	if (IsValid(OwnerController) && IsValid(OwnerController->GetWorld()))
	{
		// Remove all timers including the ones that might have been set with SetTimerForNextTick 
		OwnerController->GetWorldTimerManager().ClearAllTimersForObject(this);
	}
	MoveRetryTimerHandle.Invalidate();
	PathRetryTimerHandle.Invalidate();
}

void UAITask_MoveTo::OnDestroy(bool bInOwnerFinished)
{
	Super::OnDestroy(bInOwnerFinished);
	
	ResetObservers();
	ResetTimers();

	if (MoveRequestID.IsValid())
	{
		UPathFollowingComponent* PFComp = OwnerController ? OwnerController->GetPathFollowingComponent() : nullptr;
		if (PFComp && PFComp->GetStatus() != EPathFollowingStatus::Idle)
		{
			PFComp->AbortMove(*this, FPathFollowingResultFlags::OwnerFinished, MoveRequestID);
		}
	}

	// clear the shared pointer now to make sure other systems
	// don't think this path is still being used
	Path = nullptr;
}

void UAITask_MoveTo::OnRequestFinished(FAIRequestID RequestID, const FPathFollowingResult& Result)
{
	if (RequestID == MoveRequestID)
	{
		if (Result.HasFlag(FPathFollowingResultFlags::UserAbort) && Result.HasFlag(FPathFollowingResultFlags::NewRequest) && !Result.HasFlag(FPathFollowingResultFlags::ForcedScript))
		{
			UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Log, TEXT("%s> ignoring OnRequestFinished, move was aborted by new request"), *GetName());
		}
		else
		{
			// reset request Id, FinishMoveTask doesn't need to update path following's state
			MoveRequestID = FAIRequestID::InvalidRequest;

			if (bUseContinuousTracking && MoveRequest.IsMoveToActorRequest() && Result.IsSuccess())
			{
				UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Log, TEXT("%s> received OnRequestFinished and goal tracking is active! Moving again in next tick"), *GetName());
				GetWorld()->GetTimerManager().SetTimerForNextTick(this, &UAITask_MoveTo::PerformMove);
			}
			else
			{
				FinishMoveTask(Result.Code);
			}
		}
	}
	else if (IsActive())
	{
		UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Warning, TEXT("%s> received OnRequestFinished with not matching RequestID!"), *GetName());
	}
}

void UAITask_MoveTo::OnPathEvent(FNavigationPath* InPath, ENavPathEvent::Type Event)
{
	const static UEnum* NavPathEventEnum = StaticEnum<ENavPathEvent::Type>();
	UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Log, TEXT("%s> Path event: %s"), *GetName(), *NavPathEventEnum->GetNameStringByValue(Event));

	switch (Event)
	{
	case ENavPathEvent::NewPath:
	case ENavPathEvent::UpdatedDueToGoalMoved:
	case ENavPathEvent::UpdatedDueToNavigationChanged:
		if (InPath && InPath->IsPartial() && !MoveRequest.IsUsingPartialPaths())
		{
			UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Log, TEXT(">> partial path is not allowed, aborting"));
			UPathFollowingComponent::LogPathHelper(OwnerController, InPath, MoveRequest.GetGoalActor());
			FinishMoveTask(EPathFollowingResult::Aborted);
		}
#if ENABLE_VISUAL_LOG
		else if (!IsActive())
		{
			UPathFollowingComponent::LogPathHelper(OwnerController, InPath, MoveRequest.GetGoalActor());
		}
#endif // ENABLE_VISUAL_LOG
		break;

	case ENavPathEvent::Invalidated:
		ConditionalUpdatePath();
		break;

	case ENavPathEvent::Cleared:
	case ENavPathEvent::RePathFailed:
		UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Log, TEXT(">> no path, aborting!"));
		FinishMoveTask(EPathFollowingResult::Aborted);
		break;

	case ENavPathEvent::MetaPathUpdate:
	default:
		break;
	}
}

void UAITask_MoveTo::ConditionalUpdatePath()
{
	// mark this path as waiting for repath so that PathFollowingComponent doesn't abort the move while we 
	// micro manage repathing moment
	// note that this flag fill get cleared upon repathing end
	if (Path.IsValid())
	{
		Path->SetManualRepathWaiting(true);
	}

	if (MoveRequest.IsUsingPathfinding() && OwnerController && OwnerController->ShouldPostponePathUpdates())
	{
		UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Log, TEXT("%s> can't path right now, waiting..."), *GetName());
		OwnerController->GetWorldTimerManager().SetTimer(PathRetryTimerHandle, this, &UAITask_MoveTo::ConditionalUpdatePath, 0.2f, false);
	}
	else
	{
		PathRetryTimerHandle.Invalidate();
		
		ANavigationData* NavData = Path.IsValid() ? Path->GetNavigationDataUsed() : nullptr;
		if (NavData)
		{
			NavData->RequestRePath(Path, ENavPathUpdateType::NavigationChanged);
		}
		else
		{
			UE_VLOG(GetGameplayTasksComponent(), LogGameplayTasks, Log, TEXT("%s> unable to repath, aborting!"), *GetName());
			FinishMoveTask(EPathFollowingResult::Aborted);
		}
	}
}

