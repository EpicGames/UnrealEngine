// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/AITask_RunEQS.h"
#include "AIController.h"
#include "VisualLogger/VisualLogger.h"
#include "GameplayTasksComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AITask_RunEQS)

UAITask_RunEQS::UAITask_RunEQS(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsPausable = false;
	EQSFinishedDelegate = FQueryFinishedSignature::CreateUObject(this, &UAITask_RunEQS::OnEQSRequestFinished);
}

UAITask_RunEQS* UAITask_RunEQS::RunEQS(AAIController* Controller, UEnvQuery* QueryTemplate)
{
	if (QueryTemplate == nullptr || Controller == nullptr)
	{
		return nullptr;
	}

	UAITask_RunEQS* MyTask = UAITask::NewAITask<UAITask_RunEQS>(*Controller, EAITaskPriority::High);
	if (MyTask)
	{
		MyTask->EQSRequest.QueryTemplate = QueryTemplate;
	}

	return MyTask;
}

void UAITask_RunEQS::Activate()
{
	if (EQSRequest.QueryTemplate && OwnerController 
		&& OwnerController->GetBlackboardComponent()
		&& OwnerController->GetPawn())
	{
		Super::Activate();
		EQSRequest.Execute(*OwnerController->GetPawn(), OwnerController->GetBlackboardComponent(), EQSFinishedDelegate);
	}
}

void UAITask_RunEQS::OnEQSRequestFinished(TSharedPtr<FEnvQueryResult> Result)
{
	if (IsFinished() == false)
	{
		QueryResult = Result;

		NotificationDelegate.ExecuteIfBound(Result);

		EndTask();
	}
}

