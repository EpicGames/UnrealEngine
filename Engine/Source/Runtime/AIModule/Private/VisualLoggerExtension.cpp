// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualLoggerExtension.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "CanvasItem.h"
#include "Engine/Canvas.h"
#include "DrawDebugHelpers.h"
#include "EnvironmentQuery/EnvQueryDebugHelpers.h"
#include "EnvironmentQuery/EQSRenderingComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VisualLoggerExtension)

#if ENABLE_VISUAL_LOG
void FVisualLoggerExtension::DisableEQSRendering(AActor* HelperActor)
{
#if USE_EQS_DEBUGGER
	if (HelperActor)
	{
		SelectedEQSId = INDEX_NONE;
		UEQSRenderingComponent* EQSRenderComp = HelperActor->FindComponentByClass<UEQSRenderingComponent>();
		if (EQSRenderComp)
		{
			EQSRenderComp->SetHiddenInGame(true);
			EQSRenderComp->Deactivate();
			EQSRenderComp->ClearStoredDebugData();
		}
	}
#endif
}

extern RENDERCORE_API FTexture* GWhiteTexture;

void FVisualLoggerExtension::ResetData(IVisualLoggerEditorInterface* EdInterface)
{
	DisableEQSRendering(EdInterface->GetHelperActor());
}

void FVisualLoggerExtension::OnItemsSelectionChanged(IVisualLoggerEditorInterface* EdInterface)
{
	DrawData(EdInterface, nullptr); //it'll reset current data first
}

void FVisualLoggerExtension::OnLogLineSelectionChanged(IVisualLoggerEditorInterface* EdInterface, TSharedPtr<struct FLogEntryItem> SelectedItem, int64 UserData)
{
	SelectedEQSId = SelectedItem.IsValid() ? IntCastChecked<int32>(UserData) : INDEX_NONE;
	EdInterface->GetHelperActor()->MarkComponentsRenderStateDirty();
	DrawData(EdInterface, NULL); //we have to refresh rendering components
}

void FVisualLoggerExtension::DrawData(IVisualLoggerEditorInterface* EdInterface, UCanvas* Canvas)
{
	if (Canvas == nullptr)
	{
		// disable and refresh EQS rendering
		DisableEQSRendering(EdInterface->GetHelperActor());
	}

	UWorld* World = EdInterface->GetWorld();
	AActor* VisLogHelperActor = EdInterface->GetHelperActor();
	if (World == nullptr || VisLogHelperActor == nullptr)
	{
		return;
	}

	int32 EQSRenderingComponentIndex = 0;
	TArray<FName> RowNames = EdInterface->GetSelectedRows();
	for (const FName& RowName : RowNames)
	{
		if (EdInterface->GetSelectedItemIndex(RowName) != INDEX_NONE )
		{
			if (EdInterface->IsItemVisible(RowName, EdInterface->GetSelectedItemIndex(RowName)) == false)
			{
				continue;
			}

			const FVisualLogDevice::FVisualLogEntryItem& EntryItem = EdInterface->GetSelectedItem(RowName);
			if (Canvas == nullptr)
			{
				for (auto& Component : EQSRenderingComponents)
				{
					if (Component.IsValid())
					{
						Component->SetHiddenInGame(true);
						Component->Deactivate();
						Component->ClearStoredDebugData();
					}
				}
			}
			for (const auto& CurrentData : EntryItem.Entry.DataBlocks)
			{
				const FName TagName = CurrentData.TagName;
				if (EdInterface->MatchCategoryFilters(TagName.ToString()))
				{
					UEQSRenderingComponent* EQSRenderComp = (Canvas == nullptr && EQSRenderingComponents.IsValidIndex(EQSRenderingComponentIndex) ? EQSRenderingComponents[EQSRenderingComponentIndex].Get() : nullptr);
					if (EQSRenderComp == nullptr && Canvas == nullptr)
					{
						EQSRenderComp = NewObject<UEQSRenderingComponent>(VisLogHelperActor);
						EQSRenderComp->bDrawOnlyWhenSelected = false;
						EQSRenderComp->RegisterComponent();
						EQSRenderComp->SetHiddenInGame(false);
						EQSRenderComp->Activate();
						EQSRenderComp->MarkRenderStateDirty();
						EQSRenderingComponents.Add(EQSRenderComp);
					}
					DrawData(World, EQSRenderComp, Canvas, VisLogHelperActor, TagName, CurrentData, EntryItem.Entry.TimeStamp);
				}
				EQSRenderingComponentIndex++;
			}
		}
	}
}

void FVisualLoggerExtension::DrawData(UWorld* InWorld, class UEQSRenderingComponent* EQSRenderComp, UCanvas* Canvas, AActor* HelperActor, const FName& TagName, const FVisualLogDataBlock& DataBlock, double Timestamp)
{
#if USE_EQS_DEBUGGER
	if (TagName == *EVisLogTags::TAG_EQS)
	{
		EQSDebug::FQueryData DebugData;
		UEnvQueryDebugHelpers::BlobArrayToDebugData(DataBlock.Data, DebugData, false);
		if (EQSRenderComp && !Canvas && (SelectedEQSId == DebugData.Id || SelectedEQSId == INDEX_NONE))
		{
			EQSRenderComp->SetHiddenInGame(false);
			EQSRenderComp->Activate();
			EQSRenderComp->StoreDebugData(DebugData);
		}

		/** find and draw item selection */
		int32 BestItemIndex = INDEX_NONE;
		if (SelectedEQSId != INDEX_NONE && DebugData.Id == SelectedEQSId && Canvas)
		{
			FVector FireDir = Canvas->SceneView->GetViewDirection();
			FVector CamLocation = Canvas->SceneView->ViewMatrices.GetViewOrigin();

			FVector::FReal bestAim = 0.;
			for (int32 Index = 0; Index < DebugData.RenderDebugHelpers.Num(); ++Index)
			{
				auto& CurrentItem = DebugData.RenderDebugHelpers[Index];

				const FVector AimDir = CurrentItem.Location - CamLocation;
				const FVector::FReal FireDist = AimDir.Size();

				FVector::FReal newAim = FireDir | AimDir;
				newAim = newAim / FireDist;
				if (newAim > bestAim)
				{
					BestItemIndex = Index;
					bestAim = newAim;
				}
			}

			if (BestItemIndex != INDEX_NONE)
			{
				DrawDebugSphere(InWorld, DebugData.RenderDebugHelpers[BestItemIndex].Location, DebugData.RenderDebugHelpers[BestItemIndex].Radius, 8, FColor::Red, false);
				int32 FailedTestIndex = DebugData.RenderDebugHelpers[BestItemIndex].FailedTestIndex;
				if (FailedTestIndex != INDEX_NONE)
				{
					FString FailInfo = FString::Printf(TEXT("Selected item failed with test %d: %s (%s)\n'%s' with score %3.3f")
						, FailedTestIndex
						, *DebugData.Tests[FailedTestIndex].ShortName
						, *DebugData.Tests[FailedTestIndex].Detailed
						, *DebugData.RenderDebugHelpers[BestItemIndex].AdditionalInformation, DebugData.RenderDebugHelpers[BestItemIndex].FailedScore
						);
					float OutX, OutY;
					Canvas->StrLen(GEngine->GetSmallFont(), FailInfo, OutX, OutY);

					FCanvasTileItem TileItem(FVector2D(10, 10), GWhiteTexture, FVector2D(Canvas->SizeX, 2*OutY), FColor(0, 0, 0, 200));
					TileItem.BlendMode = SE_BLEND_Translucent;
					Canvas->DrawItem(TileItem, 0, Canvas->SizeY - 2*OutY);

					FCanvasTextItem TextItem(FVector2D::ZeroVector, FText::FromString(FailInfo), GEngine->GetSmallFont(), FLinearColor::White);
					TextItem.Depth = 1.1f;
					TextItem.EnableShadow(FColor::Black, FVector2D(1, 1));
					Canvas->DrawItem(TextItem, 5, Canvas->SizeY - 2*OutY);

				}
			}
		}


	}
#endif
}
#endif //ENABLE_VISUAL_LOG

