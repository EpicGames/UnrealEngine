// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// various parameters will be exposed to config at some point
namespace FAIConfig
{
	namespace Navigation
	{
		// distance to focal point along current velocity vector
		AIMODULE_API extern float FocalPointDistance;
	}
};
