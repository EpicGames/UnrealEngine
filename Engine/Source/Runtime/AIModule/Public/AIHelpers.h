// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AITypes.h"
#include "Containers/Array.h"
#include "Math/MathFwd.h"
#include "Math/Quat.h"
#include "Math/Rotator.h"
#include "Misc/Optional.h"
#include "Templates/SubclassOf.h"

class AActor;
class UActorComponent;

namespace FAISystem
{
	AIMODULE_API FVector FindClosestLocation(const FVector& Origin, const TArray<FVector>& Locations);

	//----------------------------------------------------------------------//
	// CheckIsTargetInSightCone
	//                     F
	//                   *****  
	//              *             *
	//          *                     *
	//       *                           *
	//     *                               *
	//   *                                   * 
	//    \                                 /
	//     \                               /
	//      \                             /
	//       \             X             /
	//        \                         /
	//         \          ***          /
	//          \     *    N    *     /
	//           \ *               * /
	//            N                 N
	//            
	//           
	//           
	//           
	//
	// 
	//                     B 
	//
	// X = StartLocation
	// B = Backward offset
	// N = Near Clipping Radius (from the StartLocation adjusted by Backward offset)
	// F = Far Clipping Radius (from the StartLocation adjusted by Backward offset)
	//----------------------------------------------------------------------//
	AIMODULE_API bool CheckIsTargetInSightCone(const FVector& StartLocation, const FVector& ConeDirectionNormal, float PeripheralVisionAngleCos,
											   float ConeDirectionBackwardOffset, float NearClippingRadiusSq, float const FarClippingRadiusSq, const FVector& TargetLocation);
}

namespace UE::AI
{
	/**
	 * This method will extract the yaw radian from the specified vector (The vector does not need to be normalized)
	 * if it is not possible to compute yaw, the function will return an invalid value	 *
	 */
	UE_DEPRECATED(5.2, "Deprecated as returning float instead of FVector::FReal. Use FVector::ToOrientationRotator()::Yaw etc.")
	extern AIMODULE_API TOptional<float> GetYawFromVector(const FVector& Vector);

	/**
	 * This method will extract the yaw radian from the specified rotator
	 * if it is not possible to compute yaw, the function will not return an invalid value
	 */
	UE_DEPRECATED(5.2, "Deprecated as returning float instead of FVector::FReal. Use FRotator::Yaw etc.")
	extern AIMODULE_API TOptional<float> GetYawFromRotator(const FRotator& Rotator);

	/**
	 * This method will extract the yaw radian from the specified quaternion
	 * if it is not possible to compute yaw, the function will not return an invalid value
	 */
	UE_DEPRECATED(5.2, "Deprecated as returning float instead of FVector::FReal. Use FQuat::Rotator()::Yaw etc.")
	extern AIMODULE_API TOptional<float> GetYawFromQuaternion(const FQuat& Quaternion);

	UE_DEPRECATED(5.3, "Please use UE::BlueprintTools::GetActorClassDefaultComponents() instead")
	extern AIMODULE_API void GetActorClassDefaultComponents(const TSubclassOf<AActor>& ActorClass, TArray<UActorComponent*>& OutComponents, const TSubclassOf<UActorComponent>& InComponentClass = TSubclassOf<UActorComponent>());
} // UE::AI
