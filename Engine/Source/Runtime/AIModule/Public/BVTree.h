// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Math/Box.h"

/**
* BVTree - Bounding Volumes Tree
*
* A binary search tree where each node has a Bounding Volume which is either an element (leaf nodes)
* or a volume encompassing all of its children (non-leaf nodes).
*
* It provides fast gathering of all bounding volumes overlapping a given FBox (see GetOverlapping).
* 
* Users are expected to provide an implementation of CalcElementBounds for ElementType:
*	static FBox CalcElementBounds(const FElement& Element) const
*/
template<typename InElementType, typename InAllocator = FDefaultAllocator>
struct TBVTree
{
	typedef InElementType FElement;

	struct FElementBox : public FBox
	{
		// index to TBVTree.Elements
		int ElementIndex;

		FElementBox() : FBox(ForceInit), ElementIndex(-1)
		{}

		FElementBox(const FBox& Box) : FBox(Box), ElementIndex(-1)
		{}
	};

	TBVTree() {}

	TBVTree(const TArray<FElement>& InElements)
	{
		Create(InElements);
	}

	TBVTree(TArray<FElement>&& InElements)
	{
		Create(MoveTemp(InElements));
	}

	// Recreate the tree based on the current content of Elements.
	// Callers are expected to have filled the GetElements() array before calling this function.
	void RecreateTree()
	{
		Nodes.Reset();
		NodeBoundingBoxes.Reset();
		CreateCommonInternal();
	}

	void RecreateTree(const TArray<FElement>& InElements)
	{
		Reset();
		Create(InElements);
	}

	void RecreateTree(TArray<FElement>&& InElements)
	{
		Reset();
		Create(MoveTemp(InElements));
	}

	void GetOverlapping(const FBox& Box, TArray<FElement>& OutOverlappingElements) const
	{
		const int LimitIndex = Nodes.Num();
		int NodeIndex = 0;
		
		while (NodeIndex < LimitIndex)
		{
			const bool bOverlap = Box.Intersect(NodeBoundingBoxes[NodeIndex]);
			const int16 ElementIndex = Nodes[NodeIndex];
			const bool bLeafNode = (ElementIndex >= 0);

			if (bLeafNode && bOverlap)
			{
				OutOverlappingElements.Add(Elements[ElementIndex]);
			}

			NodeIndex += (bOverlap || bLeafNode) ? 1 : -ElementIndex;
		}
	}

	const TArray<int16, InAllocator>& GetNodes() const { return Nodes; }
	const TArray<FBox, InAllocator>& GetBoundingBoxes() const { return NodeBoundingBoxes; }
	const TArray<FElement>& GetElements() const { return Elements; }
	TArray<FElement>& GetElements() { return Elements; }

	bool IsEmpty() const { return Nodes.Num() == 0; }

protected:
	void Subdivide(TArray<FElementBox>& ElementBBoxes, const int StartIndex, const int LimitIndex, int& CurrentNode)
	{
		const int Count = LimitIndex - StartIndex;
		const int ThisCurrentNode = CurrentNode;

		int16& NodeIndex = Nodes[CurrentNode];
		FBox& NodeBounds = NodeBoundingBoxes[CurrentNode++];

		if (Count == 1)
		{
			// Leaf node
			NodeBounds = ElementBBoxes[StartIndex];
			NodeIndex = ElementBBoxes[StartIndex].ElementIndex;
		}
		else
		{
			// Needs splitting
			const FBox TempNodeBounds = CalcNodeBounds(ElementBBoxes, StartIndex, LimitIndex);
			NodeBounds = TempNodeBounds;

			const int NumChildNodes = Count * 2 - 1;

			// A negative index means this is a non-leaf node, and the value is the number of nodes
			// to skip to exit the current branch and jump to the next unvisited node during a search.
			NodeIndex = -NumChildNodes;

			const int Axis = GetLongestAxis(NodeBounds);

			struct FAxisSort
			{
				const int Axis;
				FAxisSort(const int InAxis) : Axis(InAxis) {}

				bool operator()(const FElementBox& A, const FElementBox& B) const
				{
					return ((&A.Min.X)[Axis] < (&B.Min.X)[Axis]);
				}
			};

			Algo::Sort(MakeArrayView(ElementBBoxes.GetData() + StartIndex, Count), FAxisSort(Axis));
			
			const int SplitIndex = StartIndex + Count / 2;

			// Left
			Subdivide(ElementBBoxes, StartIndex, SplitIndex, CurrentNode);
			// Right
			Subdivide(ElementBBoxes, SplitIndex, LimitIndex, CurrentNode);
		}
	}

	void Reset()
	{
		Nodes.Reset();
		NodeBoundingBoxes.Reset();
		Elements.Reset();
	}

	void Create(const TArray<FElement>& InElements)
	{
		Elements = InElements;

		CreateCommonInternal();
	}


	void Create(TArray<FElement>&& InElements)
	{
		Elements = MoveTemp(InElements);

		CreateCommonInternal();
	}

	//assumes Elements has been set up
	void CreateCommonInternal()
	{
		if (Elements.Num())
		{
			const int NodesCount = 2 * Elements.Num() - 1;
			Nodes.AddUninitialized(NodesCount);
			NodeBoundingBoxes.AddUninitialized(NodesCount);

			TArray<FElementBox> ElementBBoxes;
			ElementBBoxes.AddUninitialized(Elements.Num());

			int Index = 0;
			for (const FElement& Element : Elements)
			{
				ElementBBoxes[Index] = CalcElementBounds(Element);
				ElementBBoxes[Index].ElementIndex = Index;
				++Index;
			}

			int CurrentNode = 0;
			Subdivide(ElementBBoxes, 0, Elements.Num(), CurrentNode);
		}
	}

	static FBox CalcNodeBounds(const TArray<FElementBox>& ElementBBoxes, const int StartIndex, const int LimitIndex)
	{
		FBox Extends(ForceInit);
		for (int Index = StartIndex; Index < LimitIndex; ++Index)
		{
			Extends += ElementBBoxes[Index];
		}
		return Extends;
	}

	static int GetLongestAxis(const FBox& NodeBounds)
	{
		const float MaxX = NodeBounds.Max.X - NodeBounds.Min.X;
		const float MaxY = NodeBounds.Max.Y - NodeBounds.Min.Y;
		const float MaxZ = NodeBounds.Max.Z - NodeBounds.Min.Z;

		return (MaxX > MaxY && MaxX > MaxZ)
			? 0
			: ((MaxY > MaxZ) ? 1 : 2);
	}

	// you have to supply this yourself
	static FBox CalcElementBounds(const FElement& /*Element*/);
	
private:
	
	TArray<int16, InAllocator> Nodes;
	TArray<FBox, InAllocator> NodeBoundingBoxes;
	TArray<FElement> Elements;
};

// static FBox CalcElementBounds(const FElement& /*Element*/) const;
