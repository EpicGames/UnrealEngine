// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"

class APawn;
class UBehaviorTree;
class UBehaviorTreeComponent;

struct FBehaviorTreeDelegates
{
	/** delegate type for tree execution events (Params: const UBehaviorTreeComponent* OwnerComp, const UBehaviorTree* TreeAsset) */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnTreeStarted, const UBehaviorTreeComponent&, const UBehaviorTree& );

	/** delegate type for locking AI debugging tool on pawn */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnTreeDebugTool, const APawn*);

	/** Called when the behavior tree starts execution. */
	static AIMODULE_API FOnTreeStarted OnTreeStarted;

	/** Called when the AI debug tool highlights a pawn. */
	static AIMODULE_API FOnTreeDebugTool OnDebugSelected;

	/** Called when the AI debug tool locks on pawn. */
	static AIMODULE_API FOnTreeDebugTool OnDebugLocked;
};
