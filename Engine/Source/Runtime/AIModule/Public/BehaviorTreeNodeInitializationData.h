// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "BehaviorTree/BehaviorTreeManager.h"

class UBTNode;
class UBTCompositeNode;

struct FBehaviorTreeNodeInitializationData
{
	UBTNode* Node;
	UBTCompositeNode* ParentNode;
	uint16 ExecutionIndex;
	uint16 DataSize;
	uint16 SpecialDataSize;
	uint8 TreeDepth;

	FBehaviorTreeNodeInitializationData() {}
	FBehaviorTreeNodeInitializationData(UBTNode* InNode, UBTCompositeNode* InParentNode,
		uint16 InExecutionIndex, uint8 InTreeDepth, uint16 NodeMemory, uint16 SpecialNodeMemory = 0)
		: Node(InNode), ParentNode(InParentNode), ExecutionIndex(InExecutionIndex), TreeDepth(InTreeDepth)
	{
		SpecialDataSize = UBehaviorTreeManager::GetAlignedDataSize(SpecialNodeMemory);

		const uint16 NodeMemorySize = NodeMemory + SpecialDataSize;
		DataSize = (NodeMemorySize <= 2) ? NodeMemorySize : UBehaviorTreeManager::GetAlignedDataSize(NodeMemorySize);
	}

	struct FMemorySort
	{
		inline bool operator()(const FBehaviorTreeNodeInitializationData& A, const FBehaviorTreeNodeInitializationData& B) const
		{
			return A.DataSize > B.DataSize;
		}
	};
};
