// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Class.h"

class AActor;
class UActorComponent;
class UBehaviorTreeComponent;
class UBlackboardData;
class UBTNode;

namespace BlueprintNodeHelpers
{
	AIMODULE_API FString CollectPropertyDescription(const UObject* Ob, const UClass* StopAtClass, const TArray<FProperty*>& InPropertiesToSkip);
	AIMODULE_API FString CollectPropertyDescription(const UObject* Ob, const UClass* StopAtClass, TFunctionRef<bool(FProperty* /*TestProperty*/)> ShouldSkipProperty);
	AIMODULE_API void CollectPropertyData(const UObject* Ob, const UClass* StopAtClass, TArray<FProperty*>& PropertyData);
	AIMODULE_API uint16 GetPropertiesMemorySize(const TArray<FProperty*>& PropertyData);

	AIMODULE_API void CollectBlackboardSelectors(const UObject* Ob, const UClass* StopAtClass, TArray<FName>& KeyNames);
	AIMODULE_API void ResolveBlackboardSelectors(UObject& Ob, const UClass& StopAtClass, const UBlackboardData& BlackboardAsset);
	AIMODULE_API bool HasAnyBlackboardSelectors(const UObject* Ob, const UClass* StopAtClass);

	AIMODULE_API FString DescribeProperty(const FProperty* Prop, const uint8* PropertyAddr);
	AIMODULE_API void DescribeRuntimeValues(const UObject* Ob, const TArray<FProperty*>& PropertyData, TArray<FString>& RuntimeValues);

	AIMODULE_API void CopyPropertiesToContext(const TArray<FProperty*>& PropertyData, uint8* ObjectMemory, uint8* ContextMemory);
	AIMODULE_API void CopyPropertiesFromContext(const TArray<FProperty*>& PropertyData, uint8* ObjectMemory, uint8* ContextMemory);

	AIMODULE_API bool FindNodeOwner(AActor* OwningActor, UBTNode* Node, UBehaviorTreeComponent*& OwningComp, int32& OwningInstanceIdx);

	AIMODULE_API void AbortLatentActions(UActorComponent& OwnerOb, const UObject& Ob);

	inline bool HasBlueprintFunction(FName FuncName, const UObject& Object, const UClass& StopAtClass)
	{
		const UFunction* Function = Object.GetClass()->FindFunctionByName(FuncName);
		ensure(Function);
		return (Function != nullptr) && (Function->GetOuter() != &StopAtClass);
	}

	inline FString GetNodeName(const UObject& NodeObject)
	{
		return NodeObject.GetClass()->GetName().LeftChop(2);
	}
}
