// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Engine/World.h"
	

AIMODULE_API DECLARE_LOG_CATEGORY_EXTERN(LogManagerInstanceTracker, Warning, All);

template<typename FManager>
struct TDefaultManagerInstanceTracker
{
	TMap<UWorld*, FManager*> WorldToInstanceMap;
	bool bCreateIfMissing = false;

	UE_DEPRECATED(5.1, "TDefaultManagerInstanceTracker is deprecated and will be removed. Consider inheriting from a subsystem instead.")
	TDefaultManagerInstanceTracker()
	{
	}

	UE_DEPRECATED(5.1, "TDefaultManagerInstanceTracker is deprecated and will be removed. Consider inheriting from a subsystem instead.")
	FManager* GetManagerInstance(UWorld& World)
	{
		FManager** FoundInstance = WorldToInstanceMap.Find(&World);
		if (FoundInstance)
		{
			return *FoundInstance;
		}
		else if (World.bIsTearingDown)
		{
			return nullptr;
		}

		FManager* NewInstance = nullptr;
		if (bCreateIfMissing)
		{
			NewInstance = NewObject<FManager>(&World);
			NewInstance->AddToRoot();
			WorldToInstanceMap.Add(&World, NewInstance);
			if (!FWorldDelegates::OnPostWorldCleanup.IsBoundToObject(this))
			{
				FWorldDelegates::OnPostWorldCleanup.AddRaw(this, &TDefaultManagerInstanceTracker::OnPostWorldCleanup);
			}
		}
		else
		{
			UE_LOGF(LogManagerInstanceTracker, Warning
				, "Calling FManager::GetCurrent while no instance for world %ls has been created, and bCreateIfMissing == false"
				, *World.GetName());
		}

		return NewInstance;
	}

	void OnPostWorldCleanup(UWorld* World, bool /*bSessionEnded*/, bool /*bCleanupResources*/)
	{
		FManager** FoundInstance = WorldToInstanceMap.Find(World);
		if (FoundInstance)
		{
			(*FoundInstance)->RemoveFromRoot();
			WorldToInstanceMap.Remove(World);
		}
	}
};
