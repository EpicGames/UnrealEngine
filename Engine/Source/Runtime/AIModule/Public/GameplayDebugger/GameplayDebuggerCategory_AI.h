// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_GAMEPLAY_DEBUGGER_MENU
#include "GameplayDebuggerCategory.h"

class AActor;
class AAIController;
class APawn;
class APlayerController;
class FDebugRenderSceneProxy;
class FPoly;
class UPrimitiveComponent;
struct FDebugDrawDelegateHelper;
struct FNavigationPath;

class FGameplayDebuggerCategory_AI : public FGameplayDebuggerCategory
{
public:
	FGameplayDebuggerCategory_AI();

	virtual void CollectData(APlayerController* OwnerPC, AActor* DebugActor) override;
	virtual void OnDataPackReplicated(int32 DataPackId) override;
	virtual void DrawData(APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext) override;
	virtual FDebugRenderSceneProxy* CreateDebugSceneProxy(const UPrimitiveComponent* InComponent, FDebugDrawDelegateHelper*& OutDelegateHelper) override;

	AIMODULE_API static TSharedRef<FGameplayDebuggerCategory> MakeInstance();

protected:
	void CollectPathData(AAIController* DebugAI);
	void DrawPath(UWorld* World);
	void DrawPawnIcons(UWorld* World, AActor* DebugActor, APawn* SkipPawn, FGameplayDebuggerCanvasContext& CanvasContext);
	void DrawOverheadInfo(AActor& DebugActor, FGameplayDebuggerCanvasContext& CanvasContext);

	struct FRepData
	{
		FString ControllerName;
		FString PawnName;
		FString MovementBaseInfo;
		FString MovementModeInfo;
		FString PathFollowingInfo;
		int32 NextPathPointIndex;
		FVector PathGoalLocation;
		FString CurrentAITask;
		FString CurrentAIState;
		FString CurrentAIAssets;
		FString NavDataInfo;
		FString MontageInfo;
		FString TaskQueueInfo;
		FString TickingTaskInfo;
		int16 NumTasksInQueue;
		int16 NumTickingTasks;
		uint32 bHasController : 1;
		uint32 bPathHasGoalActor : 1;
		uint32 bIsUsingPathFollowing : 1;
		uint32 bIsUsingCharacter : 1;
		uint32 bIsUsingBehaviorTree : 1;
		uint32 bIsUsingGameplayTasks : 1;

		FRepData() : NextPathPointIndex(0), NumTasksInQueue(0), NumTickingTasks(0), bHasController(false), bPathHasGoalActor(false),
			bIsUsingPathFollowing(false), bIsUsingCharacter(false), bIsUsingBehaviorTree(false), bIsUsingGameplayTasks(false)
		{
		}

		void Serialize(FArchive& Ar);
	};
	FRepData DataPack;

	struct FRepDataPath
	{
		struct FPoly
		{
			TArray<FVector> Points;
			FColor Color;
		};

		TArray<FPoly> PathCorridor;
		TArray<FVector> PathPoints;

		void Serialize(FArchive& Ar);
	};	
	FRepDataPath PathDataPack;
	int32 PathDataPackId;

private:
	// do NOT read from this pointer, it's only for validating if path has changed and can be invalid!
	FNavigationPath* RawLastPath;
	double LastPathUpdateTime;
};

#endif // WITH_GAMEPLAY_DEBUGGER_MENU
