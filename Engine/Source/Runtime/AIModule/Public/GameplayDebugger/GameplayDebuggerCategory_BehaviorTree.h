// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#if WITH_GAMEPLAY_DEBUGGER_MENU

#include "GameplayDebuggerCategory.h"

class AActor;
class APlayerController;

class FGameplayDebuggerCategory_BehaviorTree : public FGameplayDebuggerCategory
{
public:
	FGameplayDebuggerCategory_BehaviorTree();

	virtual void CollectData(APlayerController* OwnerPC, AActor* DebugActor) override;
	virtual void DrawData(APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext) override;

	AIMODULE_API static TSharedRef<FGameplayDebuggerCategory> MakeInstance();

protected:
	struct FRepData
	{
		FString CompName;
		FString TreeDesc;
		FString BlackboardDesc;

		void Serialize(FArchive& Ar);
	};
	FRepData DataPack;

private:

	/** Bound on Shift- decrease the blackboard display starting index */
	void ScrollUp();

	/** Bound on Shift+ increase the blackboard display starting index */
	void ScrollDown();

	/** Current blackboard display starting index */
	int32 DisplayStartingIndex = 0;

	/** Offset used to move up or down the paging */
	static constexpr int32 PagingOffset = 5;
};

#endif // WITH_GAMEPLAY_DEBUGGER_MENU