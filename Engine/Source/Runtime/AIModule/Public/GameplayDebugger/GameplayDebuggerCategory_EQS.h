// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_GAMEPLAY_DEBUGGER_MENU
#include "GameplayDebuggerCategory.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "EnvironmentQuery/EnvQueryDebugHelpers.h"

class APlayerController;
class UPrimitiveComponent;

class FGameplayDebuggerCategory_EQS : public FGameplayDebuggerCategory
{
public:
	FGameplayDebuggerCategory_EQS();

	virtual void CollectData(APlayerController* OwnerPC, AActor* DebugActor) override;
	virtual void OnDataPackReplicated(int32 DataPackId) override;
	virtual void DrawData(APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext) override;
	virtual FDebugRenderSceneProxy* CreateDebugSceneProxy(const UPrimitiveComponent* InComponent, FDebugDrawDelegateHelper*& OutDelegateHelper) override;

	AIMODULE_API static TSharedRef<FGameplayDebuggerCategory> MakeInstance();

protected:
#if USE_EQS_DEBUGGER
	struct FRepData
	{
		TArray<EQSDebug::FQueryData> QueryDebugData;
		
		void Serialize(FArchive& Ar);
	};
	FRepData DataPack;

	int32 DrawLookedAtItem(const EQSDebug::FQueryData& QueryData, APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext) const;
	void DrawDetailedItemTable(const EQSDebug::FQueryData& QueryData, int32 LookedAtItemIndex, FGameplayDebuggerCanvasContext& CanvasContext) const;
	void DrawDetailedItemRow(const EQSDebug::FItemData& ItemData, const TArray<uint8>& TestRelevancy, float MaxScore, FGameplayDebuggerCanvasContext& CanvasContext) const;
#endif

	void CycleShownQueries();
	void ToggleDetailView();

	uint32 bDrawLabels : 1;
	uint32 bDrawFailedItems : 1;
	uint32 bShowDetails : 1;

	int32 MaxItemTableRows;
	int32 MaxQueries;
	int32 ShownQueryIndex;
};

#endif // WITH_GAMEPLAY_DEBUGGER_MENU
