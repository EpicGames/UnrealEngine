// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_GAMEPLAY_DEBUGGER_MENU
#include "GameplayDebuggerCategory.h"

class APlayerController;

class FGameplayDebuggerCategory_NavLocalGrid : public FGameplayDebuggerCategory
{
public:
	FGameplayDebuggerCategory_NavLocalGrid();

	virtual void CollectData(APlayerController* OwnerPC, AActor* DebugActor) override;
	virtual void DrawData(APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext) override;
	virtual FDebugRenderSceneProxy* CreateDebugSceneProxy(const UPrimitiveComponent* InComponent, FDebugDrawDelegateHelper*& OutDelegateHelper) override;
	virtual void OnDataPackReplicated(int32 DataPackId) override;

	AIMODULE_API static TSharedRef<FGameplayDebuggerCategory> MakeInstance();

	struct FRepData
	{
		struct FGridData
		{
			FBox Bounds;
			float CellSize;
			int32 NumRows;
			int32 NumCols;
			TArray<uint8> Cells;
		};

		TArray<FGridData> Grids;
		int32 NumSources;
		int32 VersionNum;

		FRepData() : NumSources(0), VersionNum(0) {}
		void Serialize(FArchive& Ar);
	};

	struct FRepAgentData
	{
		TArray<int32> PathCells;
		int32 GridIdx;

		void Serialize(FArchive& Ar);
	};

protected:
	FRepData DataPack;
	FRepAgentData AgentDataPack;
};

#endif // WITH_GAMEPLAY_DEBUGGER_MENU
