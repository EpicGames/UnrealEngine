// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_GAMEPLAY_DEBUGGER_MENU
#include "GameplayDebuggerCategory.h"
#include "NavMesh/NavMeshRenderingComponent.h"

class APlayerController;
class UNavigationSystemV1;

class FGameplayDebuggerCategory_Navmesh : public FGameplayDebuggerCategory
{
public:
	AIMODULE_API FGameplayDebuggerCategory_Navmesh();

	AIMODULE_API virtual void CollectData(APlayerController* OwnerPC, AActor* DebugActor) override;
	AIMODULE_API virtual void DrawData(APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext) override;
	AIMODULE_API virtual FDebugRenderSceneProxy* CreateDebugSceneProxy(const UPrimitiveComponent* InComponent, FDebugDrawDelegateHelper*& OutDelegateHelper) override;
	AIMODULE_API virtual void OnDataPackReplicated(int32 DataPackId) override;

	static TSharedRef<FGameplayDebuggerCategory> MakeInstance();

protected:

	void CycleNavData();
	void CycleActorReference();
	void ToggleLockedReferenceLocation();

	/** Called on the server to collect data for the specified navigation data relative to RefPawn.
	 * RefPawn will be used to define the location where we need to collect the data and then call CollectNavigationData's version that receive a location as parameter. */
	AIMODULE_API virtual void CollectNavigationData(const UNavigationSystemV1* NavSys, const ANavigationData* NavData, const APawn* RefPawn);
	/** Called on the server to collect data for the specified navigation data around a location. */
	AIMODULE_API virtual void CollectNavigationData(const UNavigationSystemV1* NavSys, const ANavigationData* NavData, const FVector& RefLocation);
	AIMODULE_API void RetrieveRelativeTilesToDisplay(TArray<FIntPoint>& OutTileDelta);

	struct FRepData
	{
		void Serialize(FArchive& Ar);

		FString NavDataName;
		FString NavBuildLockStatusDesc;
		FString SupportedAgents;
		FVector LockedReferenceLocation = FNavigationSystem::InvalidLocation;
		int32 NumDirtyAreas = 0;
		int32 NumSuspendedDirtyAreas = 0;
		int32 NumRunningTasks = 0;
		int32 NumRemainingTasks = 0;
		bool bCanChangeReference = false;
		bool bCanCycleNavigationData = false;
		bool bIsUsingPlayerActor = false;
		bool bReferenceTooFarFromNavData = false;
		bool bIsNavBuildLocked = false;
		bool bIsNavOctreeLocked = false;
		bool bIsNavDataRebuildingSuspended = false;
	};

	FNavMeshSceneProxyData NavmeshRenderData;
	FRepData DataPack;
	
	enum class EActorReferenceMode : uint8
	{
		PlayerActorOnly,
		PlayerActor,
		DebugActor
	};
	EActorReferenceMode ActorReferenceMode = EActorReferenceMode::DebugActor;

	int32 NavDataIndexToDisplay = INDEX_NONE;
	bool bSwitchToNextNavigationData = false;
	TWeakObjectPtr<const APawn> PrevDebugActorReference;
	bool bToggleLockedReferenceLocation = false;
	TOptional<FVector> LockedReferenceLocation;
};

#endif // WITH_GAMEPLAY_DEBUGGER_MENU
