// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_GAMEPLAY_DEBUGGER_MENU
#include "GameplayDebuggerCategory.h"

class AActor;
class APlayerController;

class FGameplayDebuggerCategory_Perception : public FGameplayDebuggerCategory
{
public:
	FGameplayDebuggerCategory_Perception();

	virtual void CollectData(APlayerController* OwnerPC, AActor* DebugActor) override;

	AIMODULE_API static TSharedRef<FGameplayDebuggerCategory> MakeInstance();
};

#endif // WITH_GAMEPLAY_DEBUGGER_MENU
