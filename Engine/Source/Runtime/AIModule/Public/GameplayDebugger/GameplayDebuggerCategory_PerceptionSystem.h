// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_GAMEPLAY_DEBUGGER_MENU
#include "GameplayDebuggerCategory.h"

class AActor;
class APlayerController;

class FGameplayDebuggerCategory_PerceptionSystem : public FGameplayDebuggerCategory
{
public:
	FGameplayDebuggerCategory_PerceptionSystem();

	AIMODULE_API static TSharedRef<FGameplayDebuggerCategory> MakeInstance();

protected:
	virtual void CollectData(APlayerController* OwnerPC, AActor* DebugActor) override;
};

#endif // WITH_GAMEPLAY_DEBUGGER_MENU
