// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"

namespace UE::AI
{
	/** @return 2D cross product. Using X and Y components of 3D vectors. */
	inline FVector::FReal Cross2D(const FVector& A, const FVector& B)
	{
		return A.X * B.Y - A.Y * B.X;
	}

	/** @return 2D cross product. */
	inline FVector2D::FReal Cross2D(const FVector2D& A, const FVector2D& B)
	{
		return A.X * B.Y - A.Y * B.X;
	}

	/** @return 2D area of triangle. Using X and Y components of 3D vectors. */
	inline FVector::FReal TriArea2D(const FVector& A, const FVector& B, const FVector& C)
	{
		const FVector AB = B - A;
		const FVector AC = C - A;
		return (AC.X * AB.Y - AB.X * AC.Y) * 0.5;
	}

	/** @return 2D area of triangle. */
	inline FVector2D::FReal TriArea2D(const FVector2D& A, const FVector2D& B, const FVector2D& C)
	{
		const FVector2D AB = B - A;
		const FVector2D AC = C - A;
		return (AC.X * AB.Y - AB.X * AC.Y) * 0.5;
	}

	/** @return value in range [0..1] of the 'Point' project on segment 'Start-End'. Using X and Y components of 3D vectors. */
	inline FVector2D::FReal ProjectPointOnSegment2D(const FVector Point, const FVector Start, const FVector End)
	{
		using FReal = FVector::FReal;
		
		const FVector2D Seg(End - Start);
		const FVector2D Dir(Point - Start);
		const FReal D = Seg.SquaredLength();
		const FReal T = FVector2D::DotProduct(Seg, Dir);
		
		if (T < 0.0)
		{
			return 0.0;
		}
		else if (T > D)
		{
			return 1.0;
		}
		
		return D > UE_KINDA_SMALL_NUMBER ? (T / D) : 0.0;
	}

	/** @return value of the 'Point' project on infinite line defined by segment 'Start-End'. Using X and Y components of 3D vectors. */
	inline FVector::FReal ProjectPointOnLine2D(const FVector Point, const FVector Start, const FVector End)
	{
		using FReal = FVector::FReal;
		
		const FVector2D Seg(End - Start);
		const FVector2D Dir(Point - Start);
		const FReal D = Seg.SquaredLength();
		const FReal T = FVector2D::DotProduct(Seg, Dir);
		return D > UE_KINDA_SMALL_NUMBER ? (T / D) : 0.0;
	}

	/** @return signed distance of the 'Point' to infinite line defined by segment 'Start-End'. Using X and Y components of 3D vectors. */
	inline FVector::FReal SignedDistancePointLine2D(const FVector Point, const FVector Start, const FVector End)
	{
		using FReal = FVector::FReal;
		
		const FVector2D Seg(End - Start);
		const FVector2D Dir(Point - Start);
		const FReal Nom = Cross2D(Seg, Dir);
		const FReal Den = Seg.SquaredLength();
		const FReal Dist = Den > UE_KINDA_SMALL_NUMBER ? (Nom / FMath::Sqrt(Den)) : 0.0;
		return Dist;
	}

	/**
	 * Intersects infinite lines defined by segments A and B in 2D. Using X and Y components of 3D vectors.
	 * @param StartA start point of segment A
	 * @param EndA end point of segment A
	 * @param StartB start point of segment B
	 * @param EndB end point of segment B
	 * @param OutTA intersection value along segment A
	 * @param OutTB intersection value along segment B
	 * @return if segments A and B intersect in 2D
	 */
	inline bool IntersectLineLine2D(const FVector& StartA, const FVector& EndA, const FVector& StartB, const FVector& EndB, FVector2D::FReal& OutTA, FVector2D::FReal& OutTB)
	{
		using FReal = FVector::FReal;

		const FVector U = EndA - StartA;
		const FVector V = EndB - StartB;
		const FVector W = StartA - StartB;
		
		const FReal D = Cross2D(U, V);
		if (FMath::Abs(D) < UE_KINDA_SMALL_NUMBER)
		{
			OutTA = 0.0;
			OutTB = 0.0;
			return false;
		}
		
		OutTA = Cross2D(V, W) / D;
		OutTB = Cross2D(U, W) / D;
		
		return true;
	}

	/**
	 * Calculates intersection of segment Start-End with convex polygon Poly in 2D. Using X and Y components of 3D vectors.
	 * @param Start start point of the segment
	 * @param End end point of the segment
	 * @param Poly convex polygon
	 * @param OutTMin value along the segment of the first intersection point [0..1]
	 * @param OutTMax value along the segment of the second intersection point [0..1]
	 * @param OutSegMin index of the polygon segment of the first intersection point
	 * @param OutSegMax index of the polygon segment of the second intersection point
	 * @return true if the segment inside or intersects with the polygon.
	 */
	extern AIMODULE_API bool IntersectSegmentPoly2D(const FVector& Start, const FVector& End, TConstArrayView<FVector> Poly,
								FVector2D::FReal& OutTMin, FVector2D::FReal& OutTMax, int32& OutSegMin, int32& OutSegMax);

	/**
	 * Interpolates bilinear patch A,B,C,D. U interpolates from A->B, and C->D, and V interpolates from AB->CD.
	 * @param UV interpolation coordinates [0..1] range
	 * @param VertexA first corner
	 * @param VertexB second corner
	 * @param VertexC third corner
	 * @param VertexD fourth corner
	 * @return interpolated value.
	 */
	inline FVector Bilinear(const FVector2D UV, const FVector VertexA, const FVector VertexB, const FVector VertexC, const FVector VertexD)
	{
		const FVector AB = FMath::Lerp(VertexA, VertexB, UV.X);
		const FVector CD = FMath::Lerp(VertexD, VertexC, UV.X);
		return FMath::Lerp(AB, CD, UV.Y);
	}

	/**
	 * Finds the UV coordinates of the 'Point' on bilinear patch A,B,C,D. U interpolates from A->B, and C->D, and V interpolates from AB->CD.
	 * @param Point location inside or close to the bilinear patch
	 * @param VertexA first corner
	 * @param VertexB second corner
	 * @param VertexC third corner
	 * @param VertexD fourth corner
	 * @return UV interpolation coordinates of the 'Point'.
	 */
	extern AIMODULE_API FVector2D InvBilinear2D(const FVector Point, const FVector VertexA, const FVector VertexB, const FVector VertexC, const FVector VertexD);

	/**
	 * Finds the UV coordinates of the 'Point' on bilinear patch A,B,C,D. U interpolates from A->B, and C->D, and V interpolates from AB->CD.
	 * The UV coordinate is clamped to [0..1] range after inversion.
	 * @param Point location inside or close to the bilinear patch
	 * @param VertexA first corner
	 * @param VertexB second corner
	 * @param VertexC third corner
	 * @param VertexD fourth corner
	 * @return UV interpolation coordinates of the 'Point' in [0..1] range.
	 */
	inline FVector2D InvBilinear2DClamped(const FVector Point, const FVector VertexA, const FVector VertexB, const FVector VertexC, const FVector VertexD)
	{
		return InvBilinear2D(Point, VertexA, VertexB, VertexC, VertexD).ClampAxes(0.0, 1.0);
	}

}; // UE::AI