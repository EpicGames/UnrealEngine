// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AI/NavigationSystemBase.h" // Needed for LogAStar

struct FGraphAStarDefaultPolicy
{
	static const int32 NodePoolSize = 64;
	static const int32 OpenSetSize = 64;
	static const int32 FatalPathLength = 10000;
	static const bool bReuseNodePoolInSubsequentSearches = false;
};

enum EGraphAStarResult
{
	SearchFail,
	SearchSuccess,
	GoalUnreachable,
	InfiniteLoop
};

inline const int32 NO_COUNT = INT_MAX;

// To get AStar Graph tracing, enable this define
#define ENABLE_GRAPH_ASTAR_LOGGING 0
#if ENABLE_GRAPH_ASTAR_LOGGING
	#define UE_GRAPH_ASTAR_LOG(Verbosity, Format, ...) UE_LOG(LogAStar, Verbosity, Format, __VA_ARGS__)
#else
	#define UE_GRAPH_ASTAR_LOG(...)
#endif

/**
 *	Default A* node class.
 *	Extend this class and pass as a parameter to FGraphAStar for additional functionality
 */
template<typename TGraph>
struct FGraphAStarDefaultNode
{
	typedef typename TGraph::FNodeRef FGraphNodeRef;

	const FGraphNodeRef NodeRef;
	FGraphNodeRef ParentRef;
	FVector::FReal TraversalCost;
	FVector::FReal TotalCost;
	int32 SearchNodeIndex;
	int32 ParentNodeIndex;
	uint8 bIsOpened : 1;
	uint8 bIsClosed : 1;

	inline FGraphAStarDefaultNode(const FGraphNodeRef& InNodeRef)
		: NodeRef(InNodeRef)
		, ParentRef(TIsPointer<FGraphNodeRef>::Value ? (FGraphNodeRef)0 : (FGraphNodeRef)INDEX_NONE)
		, TraversalCost(TNumericLimits<FVector::FReal>::Max())
		, TotalCost(TNumericLimits<FVector::FReal>::Max())
		, SearchNodeIndex(INDEX_NONE)
		, ParentNodeIndex(INDEX_NONE)
		, bIsOpened(false)
		, bIsClosed(false)
	{}

	inline void MarkOpened() { bIsOpened = true; }
	inline void MarkNotOpened() { bIsOpened = false; }
	inline void MarkClosed() { bIsClosed = true; }
	inline void MarkNotClosed() { bIsClosed = false; }
	inline bool IsOpened() const { return bIsOpened; }
	inline bool IsClosed() const { return bIsClosed; }
};

#define DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_IMPL( TemplateClass, TemplateClassParameter, ConditionalReturnType, ConditionalFunctionName, DefaultImpl ) \
struct CQuery##ConditionalFunctionName	\
{	\
	template<typename TemplateClass> auto Requires(TemplateClassParameter& Obj) -> decltype(Obj.ConditionalFunctionName());	\
};	\
template <typename TemplateClass> \
static inline decltype(auto) ConditionalFunctionName(TemplateClassParameter & Obj) \
{ \
	if constexpr (TModels_V<CQuery##ConditionalFunctionName, TemplateClass>) \
	{ \
		return Obj.ConditionalFunctionName(); \
	} \
	else \
	{ \
		return (ConditionalReturnType)(DefaultImpl); \
	} \
}
#define DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION( ConditionalReturnType, ConditionalFunctionName, DefaultImpl )  DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_IMPL( TemplateClass, TemplateClass, ConditionalReturnType, ConditionalFunctionName, DefaultImpl )
#define DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_CONST( ConditionalReturnType, ConditionalFunctionName, DefaultImpl ) DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_IMPL( TemplateClass, const TemplateClass, ConditionalReturnType, ConditionalFunctionName, DefaultImpl )

#define DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_1PARAM_IMPL( TemplateClass, TemplateClassParameter, ConditionalReturnType, ConditionalFunctionName, ConditionalParamType1, DefaultImpl) \
struct CQuery##ConditionalFunctionName	\
{	\
	template<typename TemplateClass> auto Requires(TemplateClassParameter& Obj, ConditionalParamType1 Param1) -> decltype(Obj.ConditionalFunctionName(Param1));	\
};	\
template <typename TemplateClass> \
static inline decltype(auto) ConditionalFunctionName(TemplateClassParameter & Obj, ConditionalParamType1 Param1) \
{ \
	if constexpr (TModels_V<CQuery##ConditionalFunctionName, TemplateClass>) \
	{ \
		return Obj.ConditionalFunctionName(Param1); \
	} \
	else \
	{ \
		return (ConditionalReturnType)(DefaultImpl); \
	} \
}
#define DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_1PARAM( ConditionalReturnType, ConditionalFunctionName, ConditionalParamType1, DefaultImpl) DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_1PARAM_IMPL( TemplateClass, TemplateClass, ConditionalReturnType, ConditionalFunctionName, ConditionalParamType1, DefaultImpl) 
#define DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_1PARAM_CONST( ConditionalReturnType, ConditionalFunctionName, ConditionalParamType1, DefaultImpl) DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_1PARAM_IMPL( TemplateClass, const TemplateClass, ConditionalReturnType, ConditionalFunctionName, ConditionalParamType1, DefaultImpl) 

#define DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_2PARAMS_IMPL( TemplateClass, TemplateClassParameter, ConditionalReturnType, ConditionalFunctionName, ConditionalParamType1, ConditionalParamType2, DefaultImpl) \
struct CQuery##ConditionalFunctionName	\
{	\
	template<typename TemplateClass> auto Requires(TemplateClassParameter& Obj, ConditionalParamType1 Param1, ConditionalParamType2 Param2) -> decltype(Obj.ConditionalFunctionName(Param1,Param2));	\
};	\
template <typename TemplateClass> \
static inline decltype(auto) ConditionalFunctionName(TemplateClassParameter & Obj, ConditionalParamType1 Param1, ConditionalParamType2 Param2) \
{ \
	if constexpr (TModels_V<CQuery##ConditionalFunctionName, TemplateClass>) \
	{ \
		return Obj.ConditionalFunctionName(Param1, Param2); \
	} \
	else \
	{ \
		return (ConditionalReturnType)(DefaultImpl); \
	} \
}
#define DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_2PARAMS( ConditionalReturnType, ConditionalFunctionName, ConditionalParamType1, ConditionalParamType2, DefaultImpl) DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_2PARAMS_IMPL( TemplateClass, TemplateClass, ConditionalReturnType, ConditionalFunctionName, ConditionalParamType1, ConditionalParamType2, DefaultImpl)
#define DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_2PARAMS_CONST( ConditionalReturnType, ConditionalFunctionName, ConditionalParamType1, ConditionalParamType2, DefaultImpl) DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_2PARAMS_IMPL(TemplateClass, const TemplateClass, ConditionalReturnType, ConditionalFunctionName, ConditionalParamType1, ConditionalParamType2, DefaultImpl)

#define DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_CUSTOM( ConditionalFunctionName, QueryReturnType, QueryFunctionName, QueryParam, QueryDefaultImpl, QueryImpl) \
struct CQuery##QueryFunctionName	\
{	\
	template<typename TemplateClass> auto Requires(const TemplateClass& Obj) -> decltype(Obj.ConditionalFunctionName());	\
};	\
template <typename TemplateClass> \
static inline QueryReturnType QueryFunctionName(const TemplateClass& Obj, QueryParam) \
{ \
	if constexpr (TModels_V<CQuery##QueryFunctionName, TemplateClass>) \
	{ \
		return QueryImpl; \
	} \
	else \
	{ \
		return QueryDefaultImpl; \
	} \
}

template<bool DoRangeCheck>
class FRangeChecklessAllocator : public FDefaultAllocator
{
public:

	/** Set to false if you don't want to lose performance on range checks in performance-critical pathfinding code. */
	enum { RequireRangeCheck = DoRangeCheck };
};
template <> struct TAllocatorTraits<FRangeChecklessAllocator<true>> : TAllocatorTraits<FDefaultAllocator> {};
template <> struct TAllocatorTraits<FRangeChecklessAllocator<false>> : TAllocatorTraits<FDefaultAllocator> {};

/**
 *	Generic graph A* implementation
 *
 *	TGraph holds graph representation. Needs to implement functions:
 *		bool IsValidRef(FNodeRef NodeRef) const;															- returns whether given node identyfication is correct
 *		FNodeRef GetNeighbour(const FSearchNode& NodeRef, const int32 NeighbourIndex) const;				- returns neighbour ref
 *
 *	it also needs to specify node type
 *		FNodeRef		- type used as identification of nodes in the graph
 *
 *	TQueryFilter (FindPath's parameter) filter class is what decides which graph edges can be used and at what cost. It needs to implement following functions:
 *		FVector::FReal GetHeuristicScale() const;															- used as GetHeuristicCost's multiplier
 *		FVector::FReal GetHeuristicCost(const FSearchNode& StartNode, const FSearchNode& EndNode) const;	- estimate of cost from StartNode to EndNode from a search node
 *		FVector::FReal GetTraversalCost(const FSearchNode& StartNode, const FSearchNode& EndNode) const;	- real cost of traveling from StartNode directly to EndNode from a search node
 *		bool IsTraversalAllowed(const FNodeRef NodeA, const FNodeRef NodeB) const;							- whether traversing given edge is allowed from a NodeRef
 *		bool WantsPartialSolution() const;																	- whether to accept solutions that do not reach the goal
 *
 *		// Backward compatible methods, please use the FSearchNode version. If the FSearchNode version are implemented, these methods will not be called at all.
 *		FNodeRef GetNeighbour(const FNodeRef NodeRef, const int32 NeighbourIndex) const;					- returns neighbour ref
 *		FVector::FReal GetHeuristicCost(const FNodeRef StartNodeRef, const FNodeRef EndNodeRef) const;		- estimate of cost from StartNode to EndNode from a NodeRef
 *		FVector::FReal GetTraversalCost(const FNodeRef StartNodeRef, const FNodeRef EndNodeRef) const;		- real cost of traveling from StartNode directly to EndNode from a NodeRef
 *
 *		// Optionally implemented methods to parameterize the search
 *		int32 GetNeighbourCount(FNodeRef NodeRef) const;													- returns number of neighbours that the graph node identified with NodeRef has, it is ok if not implemented, the logic will stop calling GetNeighbour once it received an invalid noderef
 *		bool ShouldIgnoreClosedNodes() const;																- whether to revisit closed node or not
 *		bool ShouldIncludeStartNodeInPath() const;															- whether to put the start node in the resulting path
 *		bool ShouldFailOnInvalidEndNode() const;															- whether to early out if the end node is not valid
 *		int32 GetMaxSearchNodes() const;																	- whether to limit the number of search nodes to a maximum
 *		FVector::FReal GetCostLimit() const																	- whether to limit the search to a maximum cost
 */
template<typename TGraph, typename Policy = FGraphAStarDefaultPolicy, typename TSearchNode = FGraphAStarDefaultNode<TGraph>, bool DoRangeCheck = false >
struct FGraphAStar
{
	typedef typename TGraph::FNodeRef FGraphNodeRef;
	typedef TSearchNode FSearchNode;

	using FNodeArray = TArray<FSearchNode, FRangeChecklessAllocator<DoRangeCheck>>;
	using FRangeChecklessSetAllocator = TSetAllocator<TSparseArrayAllocator<FRangeChecklessAllocator<DoRangeCheck>, TInlineAllocator<4, FRangeChecklessAllocator<DoRangeCheck>>>, TInlineAllocator<1, FRangeChecklessAllocator<DoRangeCheck>>>;
	using FNodeMap = TMap<FGraphNodeRef, int32, FRangeChecklessSetAllocator>;
	using FIndexArray = TArray<int32, FRangeChecklessAllocator<DoRangeCheck>>;

	struct FNodeSorter
	{
		const FNodeArray& NodePool;

		FNodeSorter(const FNodeArray& InNodePool)
			: NodePool(InNodePool)
		{}

		inline bool operator()(const int32 A, const int32 B) const
		{
			return NodePool[A].TotalCost < NodePool[B].TotalCost;
		}
	};

	struct FNodePool : FNodeArray
	{
		typedef FNodeArray Super;
		FNodeMap NodeMap;

		FNodePool()
		{
			Super::Reserve(Policy::NodePoolSize);
			NodeMap.Reserve(FMath::RoundUpToPowerOfTwo(Policy::NodePoolSize / 4));
		}

		inline FSearchNode& Add(const FSearchNode& SearchNode)
		{
			FSearchNode& NewNode = Super::Emplace_GetRef(SearchNode);
			NewNode.SearchNodeIndex = UE_PTRDIFF_TO_INT32(&NewNode - Super::GetData());
			NodeMap.Add(SearchNode.NodeRef, NewNode.SearchNodeIndex);
			return NewNode;
		}

		inline FSearchNode& FindOrAdd(const FGraphNodeRef NodeRef)
		{
			// first find if node already exist in node map
			const int32 NotInMapIndex = -1;
			int32& Index = NodeMap.FindOrAdd(NodeRef, NotInMapIndex);
			if (Index != NotInMapIndex)
			{
				return (*this)[Index];
			}

			// node not found, add it and setup index in node map
			FSearchNode& NewNode = Super::Emplace_GetRef(NodeRef);
			NewNode.SearchNodeIndex = UE_PTRDIFF_TO_INT32(&NewNode - Super::GetData());
			Index = NewNode.SearchNodeIndex;

			return NewNode;
		}

		inline FSearchNode* Find(const FGraphNodeRef NodeRef)
		{
			const int32* IndexPtr = NodeMap.Find(NodeRef);
			return IndexPtr ? &(*this)[*IndexPtr] : nullptr;
		}

		inline void Reset()
		{
			Super::Reset(Policy::NodePoolSize);
			NodeMap.Reset();
		}

		inline void ReinitNodes()
		{
			for (FSearchNode& Node : *this)
			{
				new (&Node) FSearchNode(Node.NodeRef);
			}
		}
	};

	struct FOpenList : FIndexArray
	{
		typedef FIndexArray Super;
		FNodeArray& NodePool;
		const FNodeSorter& NodeSorter;

		FOpenList(FNodeArray& InNodePool, const FNodeSorter& InNodeSorter)
			: NodePool{ InNodePool }, NodeSorter{ InNodeSorter }
		{
			Super::Reserve(Policy::OpenSetSize);
		}

		void Push(FSearchNode& SearchNode)
		{
			Super::HeapPush(SearchNode.SearchNodeIndex, NodeSorter);
			SearchNode.MarkOpened();
		}

		void Modify(FSearchNode& SearchNode)
		{
			for (int32& NodeIndex : *this)
			{
				if (NodeIndex == SearchNode.SearchNodeIndex)
				{
					AlgoImpl::HeapSiftUp(Super::GetData(), 0, UE_PTRDIFF_TO_INT32(&NodeIndex - Super::GetData()), FIdentityFunctor(), NodeSorter);
					return;
				}
			}
			check(false); // We should never reach here.
		}

		int32 PopIndex()
		{
			int32 SearchNodeIndex = INDEX_NONE;

			// During A* we grow the array as needed and it does not make sense to shrink in the process.
			Super::HeapPop(SearchNodeIndex, NodeSorter, EAllowShrinking::No);
			NodePool[SearchNodeIndex].MarkNotOpened();
			return SearchNodeIndex;
		}

		UE_DEPRECATED(5.4, "PopIndex with a boolean bAllowShrinking has been deprecated - please use the version without parameter")
		inline int32 PopIndex(bool bAllowShrinking)
		{
			return PopIndex();
		}
	};

	const TGraph& Graph;
	FNodePool NodePool;
	FNodeSorter NodeSorter;
	FOpenList OpenList;


	// TGraph optionally implemented wrapper methods
	DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_1PARAM_CONST(int32, GetNeighbourCount, const FGraphNodeRef, NO_COUNT);
	DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_1PARAM_CONST(int32, GetNeighbourCountV2, const FSearchNode&, Obj.GetNeighbourCount(Param1.NodeRef));
	DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_2PARAMS_CONST(FGraphNodeRef, GetNeighbour, const FSearchNode&, const int32, Obj.GetNeighbour(Param1.NodeRef,Param2));

	// TQueryFilter optionally implemented wrapper methods
	DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_2PARAMS_CONST(FVector::FReal, GetTraversalCost, const FSearchNode&, const FSearchNode&, Obj.GetTraversalCost(Param1.NodeRef, Param2.NodeRef))
	DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_2PARAMS_CONST(FVector::FReal, GetHeuristicCost, const FSearchNode&, const FSearchNode&, Obj.GetHeuristicCost(Param1.NodeRef, Param2.NodeRef))
	DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_CONST(bool, ShouldIgnoreClosedNodes, false);
	DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_CONST(bool, ShouldIncludeStartNodeInPath, false);
	DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_CONST(bool, ShouldFailOnInvalidEndNode, true);
	DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_CONST(FVector::FReal, GetCostLimit, TNumericLimits<FVector::FReal>::Max());
	// Custom methods implemented over TQueryFilter optionally implemented methods
	DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_CUSTOM(GetMaxSearchNodes, bool, HasReachMaxSearchNodes, uint32 NodeCount, false, NodeCount >= Obj.GetMaxSearchNodes());
	DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_CUSTOM(GetCostLimit, bool, HasExceededCostLimit, FVector::FReal Cost, false, Cost > Obj.GetCostLimit());

	// TResultPathInfo optionally implemented wrapper methods
	DECLARE_OPTIONALLY_IMPLEMENTED_TEMPLATE_CLASS_FUNCTION_2PARAMS(FGraphNodeRef, SetPathInfo, const int32, const FSearchNode&, Obj[Param1] = Param2.NodeRef);

	FGraphAStar(const TGraph& InGraph)
		: Graph{ InGraph }, NodeSorter{ NodePool }, OpenList{ NodePool, NodeSorter }
	{
		NodePool.Reserve(Policy::NodePoolSize);
	}

	/**
	 * Single run of A* loop: get node from open set and process neighbors
	 * returns true if loop should be continued
	 */
	template<typename TQueryFilter>
	UE_DEPRECATED(5.1, "Please use ProcessSingleNode() taking FVector::FReal& OutBestNodeCost instead!")
	inline bool ProcessSingleNode(const FSearchNode& EndNode, const bool bIsBound, const TQueryFilter& Filter, int32& OutBestNodeIndex, float& OutBestNodeCost)
	{
		double BestNodeCost;

		const bool bSucess = ProcessSingleNode(EndNode, bIsBound, Filter, OutBestNodeIndex, BestNodeCost);
		OutBestNodeCost = UE_REAL_TO_FLOAT_CLAMPED_MAX(BestNodeCost);
		return bSucess;
	}

	/** 
	 * Single run of A* loop: get node from open set and process neighbors 
	 * returns true if loop should be continued
	 */
	template<typename TQueryFilter>
	inline bool ProcessSingleNode(const FSearchNode& EndNode, const bool bIsBound, const TQueryFilter& Filter, int32& OutBestNodeIndex, FVector::FReal& OutBestNodeCost)
	{
		// Pop next best node and put it on closed list
		const int32 ConsideredNodeIndex = OpenList.PopIndex();
		FSearchNode& ConsideredNodeUnsafe = NodePool[ConsideredNodeIndex];
		ConsideredNodeUnsafe.MarkClosed();
		UE_GRAPH_ASTAR_LOG(Display, TEXT(" Best node %i (end node %i)"), ConsideredNodeUnsafe.NodeRef, EndNode.NodeRef);

		// We're there, store and move to result composition
		if (bIsBound && (ConsideredNodeUnsafe.NodeRef == EndNode.NodeRef))
		{
			OutBestNodeIndex = ConsideredNodeUnsafe.SearchNodeIndex;
			OutBestNodeCost = 0.;
			return false;
		}

		const FVector::FReal HeuristicScale = Filter.GetHeuristicScale();

		// consider every neighbor of BestNode
		const int32 NeighbourCount = GetNeighbourCountV2(Graph, ConsideredNodeUnsafe);
		UE_GRAPH_ASTAR_LOG(Display, TEXT(" Found %i neighbor"), NeighbourCount);

		for (int32 NeighbourNodeIndex = 0; NeighbourNodeIndex < NeighbourCount; ++NeighbourNodeIndex)
		{
			const auto& NeighbourRef = GetNeighbour(Graph, NodePool[ConsideredNodeIndex], NeighbourNodeIndex);

			// invalid neigbour check
			if (Graph.IsValidRef(NeighbourRef) == false)
			{
				if(NeighbourCount == NO_COUNT)
				{
					// if user did not implemented the GetNeighbourCount method, let stop at the first invalid neighbour
					break;
				}
				else
				{
					// skipping invalid neighbours
					continue;
				}
			}

			// validate and sanitize
			if (NeighbourRef == NodePool[ConsideredNodeIndex].ParentRef
				|| NeighbourRef == NodePool[ConsideredNodeIndex].NodeRef
				|| Filter.IsTraversalAllowed(NodePool[ConsideredNodeIndex].NodeRef, NeighbourRef) == false)
			{
				UE_GRAPH_ASTAR_LOG(Display, TEXT("    Filtered %lld from %lld"), (uint64)NeighbourRef, (uint64)NodePool[ConsideredNodeIndex].NodeRef);
				continue;
			}

			// check against max search nodes limit
			FSearchNode* ExistingNeighbourNode = nullptr;
			if(HasReachMaxSearchNodes(Filter, (uint32)NodePool.Num()))
			{
				// let's skip this one if it is not already in the NodePool
				ExistingNeighbourNode = NodePool.Find(NeighbourRef);
				if (!ExistingNeighbourNode)
				{
					UE_GRAPH_ASTAR_LOG(Display, TEXT("    Reach Limit %lld from %lld"), (uint64)NeighbourRef, (uint64)NodePool[ConsideredNodeIndex].NodeRef);
					continue;
				}
			}
			FSearchNode& NeighbourNode = ExistingNeighbourNode ? *ExistingNeighbourNode : NodePool.FindOrAdd(NeighbourRef);

			// check condition to avoid search of closed nodes even if they could have lower cost
			if (ShouldIgnoreClosedNodes(Filter) && NeighbourNode.IsClosed())
			{
				UE_GRAPH_ASTAR_LOG(Display, TEXT("    Skipping closed %lld from %lld"), (uint64)NeighbourRef, (uint64)NodePool[ConsideredNodeIndex].NodeRef);
				continue;
			}

			// calculate cost and heuristic.
			const FVector::FReal NewTraversalCost = GetTraversalCost(Filter, NodePool[ConsideredNodeIndex], NeighbourNode) + NodePool[ConsideredNodeIndex].TraversalCost;
			const FVector::FReal NewHeuristicCost = bIsBound && (NeighbourNode.NodeRef != EndNode.NodeRef)
				? (GetHeuristicCost(Filter, NeighbourNode, EndNode) * HeuristicScale)
				: 0.;
			const FVector::FReal NewTotalCost = NewTraversalCost + NewHeuristicCost;

			// check against cost limit
			if (HasExceededCostLimit(Filter, NewTotalCost))
			{
				UE_GRAPH_ASTAR_LOG(Display, TEXT("    Skipping reach cost limit %lld from %lld cost %f total %f prev cost %f limit %f"), (uint64)NeighbourRef, (uint64)NodePool[ConsideredNodeIndex].NodeRef, NewTraversalCost, NewTotalCost, NeighbourNode.TotalCost, GetCostLimit(Filter));
				continue;
			}

			// check if this is better then the potential previous approach
			if (NewTotalCost >= NeighbourNode.TotalCost)
			{
				// if not, skip
				UE_GRAPH_ASTAR_LOG(Display, TEXT("    Skipping new cost higher %lld from %lld cost %f total %f prev cost %f"), (uint64)NeighbourRef, (uint64)NodePool[ConsideredNodeIndex].NodeRef, NewTraversalCost, NewTotalCost, NeighbourNode.TotalCost);
				continue;
			}

			// fill in
			NeighbourNode.TraversalCost = NewTraversalCost;
			NeighbourNode.TotalCost = NewTotalCost;
			NeighbourNode.ParentRef = NodePool[ConsideredNodeIndex].NodeRef;
			NeighbourNode.ParentNodeIndex = NodePool[ConsideredNodeIndex].SearchNodeIndex;
			NeighbourNode.MarkNotClosed();

			if (NeighbourNode.IsOpened() == false)
			{
			    UE_GRAPH_ASTAR_LOG(Display, TEXT("    Pushing %lld from %lld cost %f total %f"), (uint64)NeighbourRef, (uint64)NodePool[ConsideredNodeIndex].NodeRef, NewTraversalCost, NewTotalCost);
				OpenList.Push(NeighbourNode);
			}
			else
			{
			    UE_GRAPH_ASTAR_LOG(Display, TEXT("    Modifying %lld from %lld cost %f total %f"), (uint64)NeighbourRef, (uint64)NodePool[ConsideredNodeIndex].NodeRef, NewTraversalCost, NewTotalCost);
				OpenList.Modify(NeighbourNode);
			}

			// in case there's no path let's store information on
			// "closest to goal" node
			// using Heuristic cost here rather than Traversal or Total cost
			// since this is what we'll care about if there's no solution - this node 
			// will be the one estimated-closest to the goal
			if (NewHeuristicCost < OutBestNodeCost)
			{
				UE_GRAPH_ASTAR_LOG(Display, TEXT("    New best path %lld from %lld new best heuristic %f prev best heuristic %f"), (uint64)NeighbourRef, (uint64)NodePool[ConsideredNodeIndex].NodeRef, NewHeuristicCost, OutBestNodeCost);
				OutBestNodeCost = NewHeuristicCost;
				OutBestNodeIndex = NeighbourNode.SearchNodeIndex;
			}
		}

		return true;
	}

	/** 
	 *	Performs the actual search.
	 *	@param [OUT] OutPath - on successful search contains a sequence of graph nodes representing 
	 *		solution optimal within given constraints
	 */
	template<typename TQueryFilter, typename TResultPathInfo = TArray<FGraphNodeRef> >
	EGraphAStarResult FindPath(const FSearchNode& StartNode, const FSearchNode& EndNode, const TQueryFilter& Filter, TResultPathInfo& OutPath)
	{
		UE_GRAPH_ASTAR_LOG(Display, TEXT(""));
		UE_GRAPH_ASTAR_LOG(Display, TEXT("Starting FindPath request..."));

		if (!Graph.IsValidRef(StartNode.NodeRef) || (ShouldFailOnInvalidEndNode(Filter) && !Graph.IsValidRef(EndNode.NodeRef)))
		{
			return SearchFail;
		}

		if (StartNode.NodeRef == EndNode.NodeRef)
		{
			UE_GRAPH_ASTAR_LOG(Display, TEXT("Storing path result (length=1)..."));
			OutPath.Reset(1);
        	OutPath.AddZeroed(1);

			FSearchNode InitialNode = StartNode;
			InitialNode.TraversalCost = 0;
			InitialNode.TotalCost = 0;

			UE_GRAPH_ASTAR_LOG(Display, TEXT("  NodeRef %i"), InitialNode.NodeRef);
			SetPathInfo(OutPath, 0, InitialNode);
			return SearchSuccess;
		}

		if (Policy::bReuseNodePoolInSubsequentSearches)
		{
			NodePool.ReinitNodes();
		}
		else
		{
			NodePool.Reset();
		}
		OpenList.Reset();

		// kick off the search with the first node
		int32 BestNodeIndex = INDEX_NONE;
		FVector::FReal BestNodeCost = TNumericLimits<FVector::FReal>::Max();
		{
			// scoping StartPoolNode to make it clear it's not safe to use after ProcessSingleNode due to potential NodePool reallocation
			FSearchNode& StartPoolNode = NodePool.Add(StartNode);
			StartPoolNode.TraversalCost = 0;
			StartPoolNode.TotalCost = GetHeuristicCost(Filter, StartNode, EndNode) * Filter.GetHeuristicScale();

			OpenList.Push(StartPoolNode);

			BestNodeIndex = StartPoolNode.SearchNodeIndex;
			BestNodeCost = StartPoolNode.TotalCost;
		}

		const int32 StartPoolSearchNodeIndex = NodePool[BestNodeIndex].SearchNodeIndex;
		const FGraphNodeRef StartPoolNodeRef = NodePool[BestNodeIndex].NodeRef;

		EGraphAStarResult Result = EGraphAStarResult::SearchSuccess;
		const bool bIsBound = true;
		
		bool bProcessNodes = true;
		while (OpenList.Num() > 0 && bProcessNodes)
		{
			bProcessNodes = ProcessSingleNode(EndNode, bIsBound, Filter, BestNodeIndex, BestNodeCost);
		}

		// check if we've reached the goal
		if (BestNodeCost != 0.)
		{
			Result = EGraphAStarResult::GoalUnreachable;
		}

		// no point to waste perf creating the path if querier doesn't want it
		if (Result == EGraphAStarResult::SearchSuccess || Filter.WantsPartialSolution())
		{
			// store the path. Note that it will be reversed!
			int32 SearchNodeIndex = BestNodeIndex;
			int32 PathLength = ShouldIncludeStartNodeInPath(Filter) && BestNodeIndex != StartPoolSearchNodeIndex ? 1 : 0;
			do 
			{
				PathLength++;
				SearchNodeIndex = NodePool[SearchNodeIndex].ParentNodeIndex;
			} while (NodePool.IsValidIndex(SearchNodeIndex) && NodePool[SearchNodeIndex].NodeRef != StartPoolNodeRef && ensure(PathLength < Policy::FatalPathLength));
			
			if (PathLength >= Policy::FatalPathLength)
			{
				Result = EGraphAStarResult::InfiniteLoop;
			}

			OutPath.Reset(PathLength);
			OutPath.AddZeroed(PathLength);

			// store the path
			UE_GRAPH_ASTAR_LOG(Display, TEXT("Storing path result (length=%i)..."), PathLength);
			SearchNodeIndex = BestNodeIndex;
			int32 ResultNodeIndex = PathLength - 1;
			do
			{
				UE_GRAPH_ASTAR_LOG(Display, TEXT("  NodeRef %i"), NodePool[SearchNodeIndex].NodeRef);
				SetPathInfo(OutPath, ResultNodeIndex--, NodePool[SearchNodeIndex]);
				SearchNodeIndex = NodePool[SearchNodeIndex].ParentNodeIndex;
			} while (ResultNodeIndex >= 0);
		}
		
		return Result;
	}

	/** Floods node pool until running out of either free nodes or open set  */
	template<typename TQueryFilter>
	EGraphAStarResult FloodFrom(const FSearchNode& StartNode, const TQueryFilter& Filter)
	{
		if (!(Graph.IsValidRef(StartNode.NodeRef)))
		{
			return SearchFail;
		}

		NodePool.Reset();
		OpenList.Reset();

		// kick off the search with the first node
		int32 BestNodeIndex = INDEX_NONE;
		FVector::FReal BestNodeCost = TNumericLimits<FVector::FReal>::Max();
		{
			// scoping StartPoolNode to make it clear it's not safe to use after ProcessSingleNode due to potential NodePool reallocation
			FSearchNode& StartPoolNode = NodePool.Add(StartNode);
			StartPoolNode.TraversalCost = 0;
			StartPoolNode.TotalCost = 0;

			OpenList.Push(StartPoolNode);

			BestNodeIndex = StartPoolNode.SearchNodeIndex;
			BestNodeCost = StartPoolNode.TotalCost;
		}
		
		const FSearchNode FakeEndNode = StartNode;
		const bool bIsBound = false;

		bool bProcessNodes = true;
		while (OpenList.Num() > 0 && bProcessNodes)
		{
			bProcessNodes = ProcessSingleNode(FakeEndNode, bIsBound, Filter, BestNodeIndex, BestNodeCost);
		}

		return EGraphAStarResult::SearchSuccess;
	}

	template<typename TQueryFilter>
	bool HasReachMaxSearchNodes(const TQueryFilter& Filter) const
	{
		return HasReachMaxSearchNodes(Filter, (uint32)NodePool.Num());
	}
};