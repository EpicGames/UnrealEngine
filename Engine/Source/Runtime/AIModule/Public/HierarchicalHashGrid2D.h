// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Containers/StaticArray.h"
#include "Math/Box.h"

/**
 * Hierarchical Hash Grid in 2D.
 *
 * Items are added to the "infinite" grid at specific level based on their size. The grid size, and size ratio between levels 
 * are defined as the template parameter. This allows some computation to be optimized based on the values.
 *
 * Items that cannot be fitted in the grid are added to a spill list, which contents are included in all queries.
 * Each item is added to just one grid cell and thus can overlap some neighbour cells up to half cell size at that grid level.
 * This is compensated during the query by expanding the query box.
 *
 * Potential optimizations:
 * - (X, Y, Level) could probably be int16.
 * - Level ratio could be a shift so LevelDown() could become just shift.
 * - FloorToInt ends up calling floorf, which will be a function call.
 * 	 int FloorToInt(float a) { return (int)a + ((int)a > a ? -1 : 0); }
 * 	 auto vectorizes nicely on clang, but not on VC.
 * 
 * - Add helper function to allow to tweak the cell size to reset the grid when spill list gets too large
 */

// LWC_TODO_AI Note we are using int32 here for X and Y which does mean that we could overflow the limit of an int for LWCoords
// unless fairly large grid cell sizes are used.As WORLD_MAX is currently in flux until we have a better idea of what we are
// going to be able to support its probably not worth investing time in this potential issue right now.
template <int32 InNumLevels = 3, int32 InLevelRatio = 4, typename InItemIDType = uint32>
class THierarchicalHashGrid2D
{
public:

	typedef InItemIDType ItemIDType;
	static const int32 NumLevels = InNumLevels;		/** Number of grid levels in the grid. */
	static const int32 LevelRatio = InLevelRatio;	/** Ratio in cells between consecutive levels. */

	/** Cells are located based on their hash. Each cell stores linked list of items at that location and how many children
	  * are under the same location in the finer grid levels.
	  */
	struct FCell
	{
		FCell() : X(0), Y(0), Level(0) {}
		FCell(const int32 InX, const int32 InY, const int32 InLevel) : X(InX), Y(InY), Level(InLevel) {}

		int32 X, Y, Level;			/** Location of the cell */
		int32 First = INDEX_NONE;	/** First item in the item linked list. (index to Items array, or INDEX_NONE) */
		int32 ChildItemCount = 0;	/** Num of children in the child cells. */

		/** Returns hash based on cell location. The hash function is called a lot, favor speed over quality. */
		friend uint32 GetTypeHash(const FCell& Cell)
		{
			constexpr uint32 H1 = 0x8da6b343;	// Arbitrary big primes.
			constexpr uint32 H2 = 0xd8163841;
			constexpr uint32 H3 = 0xcb1ab31f;
			return (H1 * uint32(Cell.X) + H2 * uint32(Cell.Y) + H3 * uint32(Cell.Level));
		}

		// Need for TSet<>
		bool operator==(const FCell& RHS) const
		{
			return X == RHS.X && Y == RHS.Y && Level == RHS.Level;
		}
	};

	/** Specifies cell location within the grid at specific level. */
	struct FCellLocation
	{
		FCellLocation() = default;
		FCellLocation(const int32 InX, const int32 InY, const int32 InLevel) : X(InX), Y(InY), Level(InLevel) {}

		bool operator==(const FCellLocation& RHS) const
		{
			return X == RHS.X && Y == RHS.Y && Level == RHS.Level;
		}

		bool operator!=(const FCellLocation& RHS) const
		{
			return !(*this == RHS);
		}

		int32 X = 0;
		int32 Y = 0;
		int32 Level = INDEX_NONE;
	};

	/** Item stored in a grid cell. */
	struct FItem
	{
		ItemIDType ID = 0;			/** External item ID */
		int32 Next = INDEX_NONE;	/** Next item in the item linked list (index to Items array, or INDEX_NONE). */
	};

	/** Rectangle bounds, coordinates inclusive. */
	struct FCellRect
	{
		FCellRect() = default;
		FCellRect(const int32 InMinX, const int32 InMinY, const int32 InMaxX, const int32 InMaxY) : MinX(InMinX), MinY(InMinY), MaxX(InMaxX), MaxY(InMaxY) {}

		int32 MinX = 0;
		int32 MinY = 0;
		int32 MaxX = 0;
		int32 MaxY = 0;
	};

	/** Iterator state over a rectangle */
	struct FCellRectIterator
	{
		// Use int64 to prevent overflow when FCellRect::MaxX or FCellRect::MaxY are equal to the maximum value of int32.
		int64 X = 0;		/** Current X */
		int64 Y = 0;		/** Current Y */
		int32 Level = 0;	/** Current level */
		FCellRect Rect;		/** Area to iterate over */
	};


	/** Constructor, initializes the grid for specific cell size.
	  * @param InCellSize - new finest level cell size of the grid
	  */
	THierarchicalHashGrid2D(const float InCellSize = 500.f)
		: SpillList(INDEX_NONE)
	{
		check(InCellSize > 0.0f);

		float CurrCellSize = InCellSize;
		for (int32 i = 0; i < NumLevels; i++)
		{
			CellSize[i] = CurrCellSize;
			InvCellSize[i] = 1.0f / CurrCellSize;
			CurrCellSize *= (float)(LevelRatio);
		}

		for (int32 i = 0; i < NumLevels; i++)
		{
			LevelItemCount[i] = 0;
		}
	}

	/** Resets and initializes the grid for specific cell size.
	  * @param InCellSize - new finest level cell size of the grid
	  */
	void Initialize(const float InCellSize)
	{
		check(InCellSize > 0.0f);

		Reset();

		float CurrCellSize = InCellSize;
		for (int32 i = 0; i < NumLevels; i++)
		{
			CellSize[i] = CurrCellSize;
			InvCellSize[i] = 1.0f / CurrCellSize;
			CurrCellSize *= (float)(LevelRatio);
		}
	}

	/** Reset the grid to empty. */
	void Reset()
	{
		SpillList = INDEX_NONE;
		Cells.Reset();
		Items.Reset();

		for (int32 i = 0; i < NumLevels; i++)
		{
			LevelItemCount[i] = 0;
		}
	}

	/** Adds item to the grid. 
	  * @param ID - External ID used to identify the item.
	  * @param Bounds - Bounding box of the item. 
	  * @return Cell location of the item, can be used later to remove the item.
	  */
	FCellLocation Add(const ItemIDType ID, const FBox& Bounds)
	{
		const FCellLocation Location = CalcCellLocation(Bounds);
		Add(ID, Location);
		return Location;
	}

	/** Adds item to the grid.
	  * @param ID - External ID used to identify the item.
	  * @param Location - Cell location where the item should be added.
	  * @return Cell location of the item, can be used later to remove the item.
	  */
	void Add(const ItemIDType ID, const FCellLocation& Location)
	{
		const int32 Idx = Items.AddUninitialized().Index;
		FItem& Item = Items[Idx];
		Item.ID = ID;

		if (Location.Level == INDEX_NONE)
		{
			// Could not fit into any of the grids, add to spill list.
			Item.Next = SpillList;
			SpillList = Idx;
		}
		else
		{
			// Add to cell at specific level
			FCell& Cell = FindOrAddCell(Location.X, Location.Y, Location.Level);
			Item.Next = Cell.First;
			Cell.First = Idx;

			// Update per level counts
			LevelItemCount[Location.Level]++;

			// Update child counts
			FCellLocation ParentLocation = Location;
			while (ParentLocation.Level < NumLevels - 1)
			{
				ParentLocation.X = LevelDown(ParentLocation.X);
				ParentLocation.Y = LevelDown(ParentLocation.Y);
				ParentLocation.Level++;
				FCell& ParentCell = FindOrAddCell(ParentLocation.X, ParentLocation.Y, ParentLocation.Level);
				ParentCell.ChildItemCount++;
			}
		}
	}

	/** Removes item based on the bounding box it was added with.
	  * @param ID - External ID used to identify the item.
	  * @param OldBounds - The same bounding box the item was previously added or moved with.
	  */
	void Remove(const ItemIDType ID, const FBox& OldBounds)
	{
		const FCellLocation OldLocation = CalcCellLocation(OldBounds);
		Remove(ID, OldLocation);
	}

	/** Removes item based on the cell location it was added with.
	  * @param ID - External ID used to identify the item.
	  * @param OldLocation - The same cell location the item was previously added or moved with.
	  */
	void Remove(const ItemIDType ID, const FCellLocation& OldLocation)
	{
		if (OldLocation.Level == INDEX_NONE)
		{
			// Remove from spill list.
			int32 PrevIdx = INDEX_NONE;
			for (int32 Idx = SpillList; Idx != INDEX_NONE; PrevIdx = Idx, Idx = Items[Idx].Next)
			{
				if (Items[Idx].ID == ID)
				{
					if (PrevIdx == INDEX_NONE)
					{
						SpillList = Items[Idx].Next;
					}
					else
					{
						Items[PrevIdx].Next = Items[Idx].Next;
					}
					Items.RemoveAtUninitialized(Idx);
					break;
				}
			}
		}
		else
		{
			// Remove from cell
			if (FCell* Cell = FindCellMutable(OldLocation.X, OldLocation.Y, OldLocation.Level))
			{
				int32 PrevIdx = INDEX_NONE;
				for (int32 Idx = Cell->First; Idx != INDEX_NONE; PrevIdx = Idx, Idx = Items[Idx].Next)
				{
					if (Items[Idx].ID == ID)
					{
						if (PrevIdx == INDEX_NONE)
						{
							Cell->First = Items[Idx].Next;
						}
						else
						{
							Items[PrevIdx].Next = Items[Idx].Next;
						}
						Items.RemoveAtUninitialized(Idx);
						break;
					}
				}

				// Update per level counts
				LevelItemCount[OldLocation.Level]--;

				// Update child counts
				FCellLocation ParentLocation = OldLocation;
				while (ParentLocation.Level < NumLevels - 1)
				{
					ParentLocation.X = LevelDown(ParentLocation.X);
					ParentLocation.Y = LevelDown(ParentLocation.Y);
					ParentLocation.Level++;
					FCell* ParentCell = FindCellMutable(ParentLocation.X, ParentLocation.Y, ParentLocation.Level);
					if (ParentCell)
					{
						ParentCell->ChildItemCount--;
					}
				}
			}
		}
	}

	/** Moves item based on previous bounding box and new bounding box.
	  * @param ID - External ID used to identify the item.
	  * @param OldBounds - The same bounding box the item was previously added or moved with.
	  * @param NewBounds - New bounds of the item
	  * @return The cell location where the item was added to.
	  */
	FCellLocation Move(const ItemIDType ID, const FBox& OldBounds, const FBox& NewBounds)
	{
		const FCellLocation OldLocation = CalcCellLocation(OldBounds);
		return Move(ID, OldLocation, NewBounds);
	}

	/** Moves item based on previous cell location and new bounding box.
	  * @param ID - External ID used to identify the item.
	  * @param OldLocation - The same cell location the item was previously added or moved with.
	  * @param NewBounds - New bounds of the item
	  * @return The cell location where the item was added to.
	  */
	FCellLocation Move(const ItemIDType ID, const FCellLocation& OldLocation, const FBox& NewBounds)
	{
		const FCellLocation NewLocation = CalcCellLocation(NewBounds);
		if (NewLocation != OldLocation)
		{
			Remove(ID, OldLocation);
			Add(ID, NewLocation);
		}
		return NewLocation;
	}

	/** Returns items that potentially touch the bounds. Operates on grid level, can have false positives.
	  * This can be faster than Query() for small query box sizes (i.e. max dimension CellSize*4) due to simpler logic.
	  * @param Bounds - Query bounding box.
	  * @param OutResults - Result of the query, IDs of potentially overlapping items.
	  */
	template <typename OutT>
	void QuerySmall(const FBox& Bounds, OutT& OutResults) const
	{
		// Return items from buckets on all levels
		for (int32 Level = 0; Level < NumLevels; Level++)
		{
			// Update per level counts
			if (LevelItemCount[Level] == 0)
			{
				continue;
			}

			// Finest level query rect
			FCellRect Rect = CalcQueryBounds(Bounds, Level);

			// Use int64 to prevent overflow when FCellRect::MaxX or FCellRect::MaxY are equal to the maximum value of int32.
			for (int64 Y = Rect.MinY; Y <= Rect.MaxY; Y++)
			{
				for (int64 X = Rect.MinX; X <= Rect.MaxX; X++)
				{
					if (const FCell* Cell = FindCell(IntCastChecked<int32>(X), IntCastChecked<int32>(Y), Level))
					{
						for (int32 Idx = Cell->First; Idx != INDEX_NONE; Idx = Items[Idx].Next)
						{
							OutResults.Add(Items[Idx].ID);
						}
					}
				}
			}
		}

		// Everything from spill list
		for (int32 Idx = SpillList; Idx != INDEX_NONE; Idx = Items[Idx].Next)
		{
			OutResults.Add(Items[Idx].ID);
		}
	}

	/** Returns items that potentially touch the bounds. Operates on grid level, can have false positives.
	  * @param Bounds - Query bounding box.
	  * @param OutResults - Result of the query, IDs of potentially overlapping items.
	  */
	template <typename OutT>
	void Query(const FBox& Bounds, OutT& OutResults) const
	{
		FCellRect Rects[NumLevels];
		FCellRectIterator Iters[NumLevels];
		int32 IterIdx = 0;

		// Calculate cell bounds for each level, keep track of the coarsest level that has any items, we'll start from that.
		for (int32 Level = 0; Level < NumLevels; Level++)
		{
			Rects[Level] = CalcQueryBounds(Bounds, Level);
		}

		// The idea of the iterator below is that it iterates over rectangle cells recursively towards finer levels, depth first.
		// The previous level's iterator is kept in the Iters stack, and we can pop and continue that once the finer level is completed.
		// Finer iterator rectangles is clamped against that levels tight bounds so that unnecessary cells are not visited.
		// Coarser levels of the grid also store data if the finer levels under them has any items. This is used to skip iterating
		// lower levels at certain locations completely. This can be big advantage in larger query boxes, compared to iterating all cells as in QuerySmall().

		// Init coarsest iterator
		const int32 StartLevel = NumLevels - 1;
		Iters[IterIdx].Level = StartLevel;
		Iters[IterIdx].Rect = Rects[StartLevel];
		Iters[IterIdx].X = Iters[IterIdx].Rect.MinX;
		Iters[IterIdx].Y = Iters[IterIdx].Rect.MinY;
		IterIdx++;

		while (IterIdx > 0)
		{
			FCellRectIterator& Iter = Iters[IterIdx - 1];
			// Check if the iterator has finished
			if (Iter.X > Iter.Rect.MaxX)
			{
				Iter.X = Iter.Rect.MinX;
				Iter.Y++;
				if (Iter.Y > Iter.Rect.MaxY)
				{
					IterIdx--;
					continue;
				}
			}

			if (const FCell* Cell = FindCell( IntCastChecked<int32>(Iter.X) , IntCastChecked<int32>(Iter.Y), Iter.Level))
			{
				// Collect items from this cell.
				for (int32 Idx = Cell->First; Idx != INDEX_NONE; Idx = Items[Idx].Next)
				{
					OutResults.Add(Items[Idx].ID);
				}

				// Advance to region at finer level if it has any items.
				if (Cell->ChildItemCount > 0)
				{
					check(Iter.Level > 0);
					const int FinerLevel = Iter.Level - 1;
					// The iteration rect is intersection between current coarse cell and finer level bounds (which is more accurate).
					const FCellRect& FinerLevelRect = Rects[FinerLevel];
					const int32 MinX = ClampInt32(Iter.X * LevelRatio);
					const int32 MinY = ClampInt32(Iter.Y * LevelRatio);
					const int32 MaxX = ClampInt32(Iter.X * LevelRatio + LevelRatio - 1);
					const int32 MaxY = ClampInt32(Iter.Y * LevelRatio + LevelRatio - 1);
					FCellRect CurrentRect(MinX, MinY, MaxX, MaxY);
					FCellRect NewIterRect = IntersectRect(CurrentRect, FinerLevelRect);
					// Advance if rect is not empty.
					if (NewIterRect.MaxX >= NewIterRect.MinX && NewIterRect.MaxY >= NewIterRect.MinY)
					{
						FCellRectIterator& FinerIter = Iters[IterIdx];
						FinerIter.Rect = NewIterRect;
						FinerIter.X = NewIterRect.MinX;
						FinerIter.Y = NewIterRect.MinY;
						FinerIter.Level = FinerLevel;
						IterIdx++;
					}
				}
			}

			// Advance iteration
			Iter.X++;
		}

		// Everything from spill list
		for (int32 Idx = SpillList; Idx != INDEX_NONE; Idx = Items[Idx].Next)
		{
			OutResults.Add(Items[Idx].ID);
		}
	}

	/** Calculates cell based query rectangle. The bounds are expanded by half grid cell size because the items are stored for only one cell
	  * based on their center and side. For that reason the items can overlap the neighbor cells by half the cell size.
	  * @param Bounds - Query bounding box to quantize.
	  * @param Level - Which level of the tree the to calculate the bounds for
	  * @return Quantized rectangle representing the cell bounds at specific level of the tree, coordinates inclusive.
	  */
	FCellRect CalcQueryBounds(const FBox& Bounds, const int32 Level) const
	{
		FCellRect Result;
		Result.MinX = ClampInt32(FMath::FloorToInt(Bounds.Min.X * InvCellSize[Level] - 0.5f));
		Result.MinY = ClampInt32(FMath::FloorToInt(Bounds.Min.Y * InvCellSize[Level] - 0.5f));
		Result.MaxX = ClampInt32(FMath::FloorToInt(Bounds.Max.X * InvCellSize[Level] + 0.5f));
		Result.MaxY = ClampInt32(FMath::FloorToInt(Bounds.Max.Y * InvCellSize[Level] + 0.5f));
		return Result;
	}

	/** Returns intersection of the two cell bounding rectangles.
	  * @param Left - left hand side rectangle
	  * @param Right - right hand side rectangle
	  * @return Intersecting are between left and right.
	  */
	FCellRect IntersectRect(const FCellRect& Left, const FCellRect& Right) const
	{
		FCellRect Result;
		Result.MinX = FMath::Max(Left.MinX, Right.MinX);
		Result.MinY = FMath::Max(Left.MinY, Right.MinY);
		Result.MaxX = FMath::Min(Left.MaxX, Right.MaxX);
		Result.MaxY = FMath::Min(Left.MaxY, Right.MaxY);
		return Result;
	}

	/** Levels down a coordinate using floor rounding.
	  * @param X - Coordinate to level down
	  * @return Coordinate at coarser level.
	  */
	static int32 LevelDown(int32 X)
	{
		X -= X < 0 ? (LevelRatio - 1) : 0;
		return X / LevelRatio;
	}

	/** Finds grid level where the bounds fit inside a cell.
	  * @param Bounds - Bounding box of the item.
	  * @return Returns cell location of the item, or location at Level == INDEX_NONE if the item cannot fit in the grid.
	  */
	FCellLocation CalcCellLocation(const FBox& Bounds) const
	{
		FCellLocation Location(0, 0, 0);

		const FVector Center = Bounds.GetCenter();
		const FVector::FReal Diameter = FMath::Max(Bounds.Max.X - Bounds.Min.X, Bounds.Max.Y - Bounds.Min.Y);
		for (Location.Level = 0; Location.Level < NumLevels; Location.Level++)
		{
			const int32 DiameterCells = ClampInt32(FMath::CeilToInt(Diameter * InvCellSize[Location.Level]));
			// note that it's fine for DiameterCells to equal 0 - that would happen for 0-sized items (valid location, no extent).
			if (DiameterCells <= 1)
			{
				Location.X = ClampInt32(FMath::FloorToInt(Center.X * InvCellSize[Location.Level]));
				Location.Y = ClampInt32(FMath::FloorToInt(Center.Y * InvCellSize[Location.Level]));
				break;
			}
		}

		if (Location.Level == NumLevels)
		{
			// Could not fit into any of the levels.
			Location.X = 0;
			Location.Y = 0;
			Location.Level = INDEX_NONE;
		}

		return Location;
	}

	/** Finds the world box from a cell.
	 * @params CellLocation - Cell location.
	 * @return Returns the bounds of the cell.
	 */
	FBox CalcCellBounds(const FCellLocation& CellLocation) const
	{
		const FVector::FReal Size = CellSize[CellLocation.Level];
		const FVector::FReal X = static_cast<FVector::FReal>(CellLocation.X) * Size;
		const FVector::FReal Y = static_cast<FVector::FReal>(CellLocation.Y) * Size;
		return FBox(FVector(X, Y, 0.0), FVector(X + Size, Y + Size, 0.0));
	}

	/** Returns a cell for specific location and level, creates new cell if it does not exist.
	  * @param X - Cell X coordinate.
	  * @param Y - Cell Y coordinate.
	  * @param Level - Grid Level.
	  * @return Reference to cell at specified location.
	  */
	FCell& FindOrAddCell(const int X, const int Y, const int Level)
	{
		FCell CellToFind(X, Y, Level);
		const uint32 Hash = GetTypeHash(CellToFind);
		FCell* Cell = Cells.FindByHash(Hash, CellToFind);
		if (Cell != nullptr)
		{
			return *Cell;
		}
		FSetElementId Index = Cells.AddByHash(Hash, FCell(X, Y, Level));
		return Cells[Index];
	}

	/** Returns a cell for specific location and level.
	  * @param X - Cell X coordinate.
	  * @param Y - Cell Y coordinate.
	  * @param Level - Grid Level.
	  * @return Pointer to cell at specified location, or return nullptr if the cell does not exist.
	  */
	FCell* FindCellMutable(const int X, const int Y, const int Level)
	{
		FCell CellToFind(X, Y, Level);
		const uint32 Hash = GetTypeHash(CellToFind);
		return Cells.FindByHash(Hash, CellToFind);
	}

	/** Returns a cell for specific location and level.
	  * @param X - Cell X coordinate.
	  * @param Y - Cell Y coordinate.
	  * @param Level - Grid Level.
	  * @return Pointer to cell at specified location, or return nullptr if the cell does not exist.
	  */
	const FCell* FindCell(const int X, const int Y, const int Level) const
	{
		FCell CellToFind(X, Y, Level);
		const uint32 Hash = GetTypeHash(CellToFind);
		return Cells.FindByHash(Hash, CellToFind);
	}

	/** @return Cell size at specific grid level. */
	float GetCellSize(const int32 Level) const { return CellSize[Level]; }

	/** @return 1/cellSize at specific grid level. */
	float GetInvCellSize(const int32 Level) const { return InvCellSize[Level]; }

	/** @return Array containing all the items. */
	int32 GetLevelItemCount(const int32 Level) const { return LevelItemCount[Level]; }

	/** @return Set containing all the cells. */
	const TSet<FCell>& GetCells() const { return Cells; }

	/** @return Sparse array containing all the items. */
	const TSparseArray<FItem>& GetItems() const { return Items; }

	/** @return Index in items of the first item on the spill list. */
	int32 GetFirstSpillListItem() const { return SpillList; } 

protected:

	/** @return given int64 clamped in int32 range. */
	static constexpr int32 ClampInt32(const int64 Value)
	{
		return static_cast<int32>(FMath::Clamp(
			Value,
			static_cast<int64>(std::numeric_limits<int32>::lowest()),
			static_cast<int64>(std::numeric_limits<int32>::max())));
	}
	
	TStaticArray<float, NumLevels> CellSize;		/** Lowest level cell size */
	TStaticArray<float, NumLevels> InvCellSize;		/** 1 / CellSize */
	TStaticArray<int32, NumLevels> LevelItemCount;	/** Number items per level, can be used to skip certain levels. */
	TSet<FCell> Cells;								/** TSet uses hash buckets to locate items. */
	TSparseArray<FItem> Items;						/** TSparseArray uses freelist to recycled used items. */
	int32 SpillList = INDEX_NONE;					/** Linked list of items that did not fit into any of the levels */
};
