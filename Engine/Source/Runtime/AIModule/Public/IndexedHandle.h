// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Array.h"
#include <atomic>
#include "IndexedHandle.generated.h"

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#define UE_DO_INDEXED_HANDLE_MANAGER_ID 1
#else
#define UE_DO_INDEXED_HANDLE_MANAGER_ID 0
#endif


/**
 * Index based handle that doesn't use a serial number.
 * For fast access in to index based data structures when we don't expect handles to be able to be stale
 */
USTRUCT()
struct FSimpleIndexedHandleBase
{
	GENERATED_BODY()

	FSimpleIndexedHandleBase() = default;

	/** @param InIndex - passing INDEX_NONE will make this handle Invalid */
	FSimpleIndexedHandleBase(int32 InIndex) : Index(InIndex)
	{
	};

	bool operator==(const FSimpleIndexedHandleBase& Other) const
	{
		return Index == Other.Index;
	}

	bool operator!=(const FSimpleIndexedHandleBase& Other) const
	{
		return !operator==(Other);
	}

	/** @return INDEX_NONE if invalid or >=0 for a potentially valid handle */
	int32 GetIndex() const { return Index; }

	/** @return true if this handle is valid, this doesn't check that it's in the correct range for the underlying resource it's providing a handle for */
	bool IsValid() const { return Index >= 0; }

	/**
	 * boolean operator useful when embedding declaration in if statement
	 * @return same as IsValid()
	 * @see IsValid
	 */
	operator bool() const { return IsValid(); }

	/** Makes the handle Invalid */
	void Invalidate() { Index = INDEX_NONE; }

	/** @param InIndex - passing INDEX_NONE will make this handle Invalid */
	void SetIndex(int32 InIndex)
	{
		Index = InIndex;
	}

	friend uint32 GetTypeHash(const FSimpleIndexedHandleBase& Handle)
	{
		return static_cast<uint32>(Handle.GetIndex());
	}

protected:
	int32 Index = INDEX_NONE;
};

/** Index based handle that has a serial number to verify stale handles. For fast safe access in to index based data structures */
USTRUCT()
struct FIndexedHandleBase : public FSimpleIndexedHandleBase
{
	GENERATED_BODY()

	FIndexedHandleBase() = default;

	/** @note passing INDEX_NONE as index will make this handle Invalid */
	FIndexedHandleBase(int32 InIndex, uint32 InSerialNumber) : FSimpleIndexedHandleBase(InIndex), SerialNumber(InSerialNumber)
	{
	};

	bool operator==(const FIndexedHandleBase& Other) const
	{
		return FSimpleIndexedHandleBase::operator==(Other) && SerialNumber == Other.SerialNumber;
	}

	bool operator!=(const FIndexedHandleBase& Other) const
	{
		return !operator==(Other);
	}

	uint32 GetSerialNumber() const { return SerialNumber; }
	void SetSerialNumber(uint32 InSerialNumber) { SerialNumber = InSerialNumber; }

	friend uint32 GetTypeHash(const FIndexedHandleBase& Handle)
	{
		return HashCombine(static_cast<uint32>(Handle.GetIndex()), Handle.GetSerialNumber());
	}

protected:
	uint32 SerialNumber = 0;

#if UE_DO_INDEXED_HANDLE_MANAGER_ID
public:
	uint32 ManagerID = 0;
#endif
};

/** Compact Index based handle that has a serial number to verify stale handles. For fast safe access in to index based data structures. */
USTRUCT()
struct FCompactIndexedHandleBase
{
	GENERATED_BODY()

	FCompactIndexedHandleBase() = default;

	/** @note passing INDEX_NONE as index will make this handle Invalid */
	FCompactIndexedHandleBase(const int32 InIndex, const uint32 InSerialNumber)
	{
		check(InIndex <= (int32)MAX_int16);
		Index = (int16)InIndex;
		
		check(InSerialNumber <= (uint32)MAX_uint16);
		SerialNumber = (uint16)InSerialNumber;
	};

	bool operator==(const FCompactIndexedHandleBase& Other) const
	{
		return Index == Other.Index && SerialNumber == Other.SerialNumber;
	}

	bool operator!=(const FCompactIndexedHandleBase& Other) const
	{
		return !operator==(Other);
	}

	/** @return INDEX_NONE if invalid or >=0 for a potentially valid handle */
	int16 GetIndex() const { return Index; }

	/** @return true if this handle is valid, this doesn't check that it's in the correct range for the underlying resource it's providing a handle for */
	bool IsValid() const { return Index >= 0; }

	/** 
     * boolean operator useful when embedding declaration in if statement
	 * @return same as IsValid()
	 * @see IsValid
     */
	operator bool() const { return IsValid(); }

	/** Makes the handle Invalid */
	void Invalidate() { Index = INDEX_NONE; }

	/** @param InIndex - passing INDEX_NONE will make this handle Invalid */
	void SetIndex(int16 InIndex)
	{
		Index = InIndex;
	}

	uint16 GetSerialNumber() const { return SerialNumber; }
	void SetSerialNumber(int16 InSerialNumber) { SerialNumber = InSerialNumber; }

	friend uint32 GetTypeHash(const FCompactIndexedHandleBase& Handle)
	{
		return (static_cast<uint32>(Handle.GetIndex()) << 16) + static_cast<uint32>(Handle.GetSerialNumber());
	}

protected:
	int16 Index = INDEX_NONE;
	uint16 SerialNumber = 0;

#if UE_DO_INDEXED_HANDLE_MANAGER_ID
public:
	uint32 ManagerID = 0;
#endif
};

/** Handle Manager meant for FIndexedHandleBase and FCompactIndexedHandleBase derived classes, handles are given out from a freelist and are zero based and consecutive in nature,
 *  so ideal for being used as indices in to arrays.
 *  Using bOptimizeHandleReuse performance impact on releasing a handle. Because new handles will always use the smallest index, this will help greatly the compaction code(ShrinkHandles) and reduce the total number of handles.
 *  This is very useful when you need to iterate through every valid handles often.
 */
template<typename TIndexedHandle, typename TIndexType, typename TSerialType, bool bOptimizeHandleReuse = false>
struct FIndexedHandleManagerBase
{
protected:
	typedef TArray<TIndexedHandle> FHandleArray;

public:
	FIndexedHandleManagerBase()
	{
#if UE_DO_INDEXED_HANDLE_MANAGER_ID
		ManagerID = ManagerIDCounter.fetch_add(1);
#endif
	}

	TIndexedHandle GetNextHandle()
	{
		TIndexedHandle* Handle = nullptr;

		//if no free handle indices then create one
		if (FreeHandleIndices.Num() == 0)
		{
			checkf(Handles.Num() <= static_cast<typename FHandleArray::SizeType>(TNumericLimits<TIndexType>::Max()), TEXT("Handles are overflowing the numeric limits of the handle TIndexType!"));
			Handle = &Handles.Emplace_GetRef(Handles.Num(), SerialNumberCounter.fetch_add(1));
		}
		else //otherwise use an existing handle index
		{
			const TIndexType Idx = FreeHandleIndices.Pop();
			Handle = &Handles[Idx];
			checkf(!Handle->IsValid(), TEXT("Free handle must be set invalid before reuse"));
			Handle->SetIndex(Idx);
			Handle->SetSerialNumber(SerialNumberCounter.fetch_add(1));
		}

#if UE_DO_INDEXED_HANDLE_MANAGER_ID
		Handle->ManagerID = ManagerID;
#endif

		return *Handle;
	}

	bool RemoveHandle(TIndexedHandle IndexedHandle)
	{
		const bool bIsValid = IsValidHandle(IndexedHandle);

		if (ensureMsgf(bIsValid, TEXT("Trying to Remove Invalid Handle with Index %d, Serial Number %d"), IndexedHandle.GetIndex(), IndexedHandle.GetSerialNumber()))
		{
			if constexpr (bOptimizeHandleReuse)
			{
			    const TIndexType Idx = IndexedHandle.GetIndex();
			    const int32 InsertPosition = Algo::LowerBound(FreeHandleIndices, Idx, [](const TIndexType& Lhs, const TIndexType& Rhs) {return Lhs > Rhs;});
			    FreeHandleIndices.Insert(Idx, InsertPosition);
			}
			else
			{
				FreeHandleIndices.Add(IndexedHandle.GetIndex());
			}

			Handles[IndexedHandle.GetIndex()].Invalidate();

			//increase the serial numbers when a handle is removed
			Handles[IndexedHandle.GetIndex()].SetSerialNumber(SerialNumberCounter.fetch_add(1));
		}

		return bIsValid;
	}

	bool IsValidHandle(TIndexedHandle IndexedHandle) const
	{
#if UE_DO_INDEXED_HANDLE_MANAGER_ID
		if (!ensureMsgf(ManagerID == IndexedHandle.ManagerID, TEXT("ManagerID %d does not match IndexedHandle.ManagerID %d, handles must only be used by the same manager they were created by!"), ManagerID, IndexedHandle.ManagerID))
		{
			return false;
		}
#endif // UE_DO_INDEXED_HANDLE_MANAGER_ID

		return ((IndexedHandle.GetIndex() >= 0) && (IndexedHandle.GetIndex() < Handles.Num())
			&& (IndexedHandle.GetSerialNumber() == Handles[IndexedHandle.GetIndex()].GetSerialNumber())
			&& (IndexedHandle.GetIndex() == Handles[IndexedHandle.GetIndex()].GetIndex()));
	}

	const TArray<TIndexedHandle>& GetHandles() const { return Handles; }

	/** returns the number of used handles, will be greater than or equal to zero */
	int32 CalcNumUsedHandles() const { return Handles.Num() - FreeHandleIndices.Num(); };

	// Defaulted copy constructor, but it acts differently from the copy assignment operator - notably, doesn't allocate a new ManagerID.
	FIndexedHandleManagerBase(const FIndexedHandleManagerBase& Other) = default;

	FIndexedHandleManagerBase& operator=(const FIndexedHandleManagerBase& Other)
	{
		Handles = Other.Handles;
		FreeHandleIndices = Other.FreeHandleIndices;
		//Leave SerialNumberCounter well alone

#if UE_DO_INDEXED_HANDLE_MANAGER_ID
		//assign new ManagerID to the new Manager
		ManagerID = ManagerIDCounter.fetch_add(1);
#endif // UE_DO_INDEXED_HANDLE_MANAGER_ID
		return *this;
	}

	/** Attempts to shrink the Handles array if there are contiguous free slots at the end of the Array
	 *	@return the new Num items in the handles array
	 */
	int32 ShrinkHandles()
	{
		while (Handles.Num() > 0 && FreeHandleIndices.Num() > 0 && FreeHandleIndices.Remove(Handles.Num() - 1))
		{
			Handles.Pop(EAllowShrinking::No);
		}

		return Handles.Num();
	}


	void Reset()
	{
		Handles.Reset();
		FreeHandleIndices.Reset();
		//Don't reset the SerialNumberCounter as we need that to increase from where it left off to keep handle consistency
	}

protected:
	/** Handles stored as a free list sparse array, Handle entries that are free will have a GetIndex() == INDEX_NONE */
	FHandleArray Handles;
	TArray<TIndexType> FreeHandleIndices;

	static std::atomic<TSerialType> SerialNumberCounter;

#if UE_DO_INDEXED_HANDLE_MANAGER_ID
	/** ID used in non shipping / test builds to ensure the same manager that is used to create handles is also used to remove them */
	static std::atomic<uint32> ManagerIDCounter;
	uint32 ManagerID = 0;
#endif // UE_DO_INDEXED_HANDLE_MANAGER_ID
};

template<typename TIndexedHandle, typename TIndexType, typename TSerialType, bool bOptimizeHandleReuse>
std::atomic<TSerialType> FIndexedHandleManagerBase<TIndexedHandle, TIndexType, TSerialType, bOptimizeHandleReuse>::SerialNumberCounter = { 0 };

#if UE_DO_INDEXED_HANDLE_MANAGER_ID
template<typename TIndexedHandle, typename TIndexType, typename TSerialType, bool bOptimizeHandleReuse>
std::atomic<uint32> FIndexedHandleManagerBase<TIndexedHandle, TIndexType, TSerialType, bOptimizeHandleReuse>::ManagerIDCounter = { 0 };
#endif // UE_DO_INDEXED_HANDLE_MANAGER_ID

template<typename TIndexedHandle, bool bOptimizeHandleReuse = false>
struct FIndexedHandleManager : public FIndexedHandleManagerBase<TIndexedHandle, int32, uint32, bOptimizeHandleReuse>
{
};

template<typename TIndexedHandle, bool bOptimizeHandleReuse = false>
struct FCompactIndexedHandleManager : public FIndexedHandleManagerBase<TIndexedHandle, int16, uint16, bOptimizeHandleReuse>
{
};
