// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequentialID.generated.h"

USTRUCT()
struct FSequentialIDBase
{
	GENERATED_BODY()

	FSequentialIDBase() = default;

	explicit FSequentialIDBase(uint32 InID) : Value(InID) {}

	bool operator==(const FSequentialIDBase& Other) const { return Value == Other.Value; }
	bool operator!=(const FSequentialIDBase& Other) const { return Value != Other.Value; }

	bool IsValid() const { return Value != InvalidID; }
	bool IsInvalid() const { return Value == InvalidID; }

	void Invalidate() { Value = InvalidID; }

	uint32 GetValue() const { return Value; }

	FString Describe() const { return FString::Printf(TEXT("Id[%d]"), Value); }

	friend uint32 GetTypeHash(const FSequentialIDBase& SequentialID) { return SequentialID.GetValue(); }

protected:
	static constexpr uint32 InvalidID = 0;

	UPROPERTY()
	uint32 Value = InvalidID;
};
