// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/Archive.h"
#include "Math/NumericLimits.h"

struct FGridSize2D
{
	uint32 Width;
	uint32 Height;

	explicit FGridSize2D(uint32 InWidth = 0, uint32 InHeight = 0)
		: Width(InWidth), Height(InHeight)
	{
	}

	bool operator==(const FGridSize2D& Other) const
	{
		return Width == Other.Width && Height == Other.Height;
	}
};

/**	No virtuals on purpose */
template<typename CellType, int InvalidCellValue = 0>
struct TSimpleCellGrid
{
	typedef CellType FCellType;

	float GridCellSize;
	FBox WorldBounds;
	FVector Origin;
	FVector BoundsSize;
	FGridSize2D GridSize;

protected:
	TArray<FCellType> Cells;

public:
	TSimpleCellGrid()
		: GridCellSize(0.0f)
		, WorldBounds(ForceInitToZero)
		, Origin(FLT_MAX)
		, BoundsSize(0)
	{
	}

	/** Initialize the grid from a bounding box
	 * @param InCellSize Size of a cell
	 * @param Bounds Bounding box the grid needs to encapsulate
	 * @return True if the grid was initialized properly
	 */
	bool Init(const float InCellSize, const FBox& Bounds)
	{
		if (InCellSize <= 0.0f || !Bounds.IsValid)
		{
			return false;
		}
			
		GridCellSize = InCellSize;

		const FVector RealBoundsSize = Bounds.GetSize();
		GridSize = FGridSize2D(IntCastChecked<uint32>(FMath::CeilToInt(RealBoundsSize.X / InCellSize)), IntCastChecked<uint32>(FMath::CeilToInt(RealBoundsSize.Y / InCellSize)));
		BoundsSize = FVector(GridSize.Width * InCellSize, GridSize.Height * InCellSize, RealBoundsSize.Z);
		Origin = FVector(Bounds.Min.X, Bounds.Min.Y, (Bounds.Min.Z + Bounds.Max.Z) * 0.5f);
		UpdateWorldBounds();

		const uint64 TempCellCount = GridSize.Width * GridSize.Height;
		const typename TArray<FCellType>::SizeType CellCount = FMath::Min(TempCellCount, (uint64)TNumericLimits<typename TArray<FCellType>::SizeType>::Max());
		ensureMsgf(CellCount == TempCellCount, TEXT("Grid width and height are too big."));
		Cells.AddDefaulted(CellCount);

		return true;
	}

	/** Initialize the grid
	 * @param InCellSize Size of a cell
	 * @param InGridSize Amount of cells needed in each direction
	 * @param InOrigin World location of the grid
	 * @param VerticalBoundSize Size of the grid above and under the Origin
	 * @return True if the grid was initialized properly
	 */
	bool Init(const float InCellSize, const FGridSize2D& InGridSize, const FVector& InOrigin, const float VerticalBoundSize)
	{
		if (InCellSize < 0 || InGridSize.Height == 0 || InGridSize.Width == 0 || VerticalBoundSize < 0)
		{
			return false;
		}

		GridCellSize = InCellSize;
		GridSize = InGridSize;
		BoundsSize = FVector(GridSize.Width * InCellSize, GridSize.Height * InCellSize, VerticalBoundSize);
		Origin = InOrigin;
		UpdateWorldBounds();

		const uint64 TempCellCount = GridSize.Width * GridSize.Height;
		const typename TArray<FCellType>::SizeType CellCount = FMath::Min(TempCellCount, (uint64)TNumericLimits<typename TArray<FCellType>::SizeType>::Max());
		ensureMsgf(CellCount == TempCellCount, TEXT("Grid width and height are too big."));
		Cells.AddDefaulted(CellCount);

		return true;
	}

	/** Change the vertical position of the grid by providing an interval */
	void SetVerticalInterval(const FFloatInterval& VerticalInterval)
	{
		if (VerticalInterval.IsValid())
		{
			BoundsSize.Z = VerticalInterval.Size();
			Origin.Z = VerticalInterval.Interpolate(0.5);
			UpdateWorldBounds();
		}
	}

	void UpdateWorldBounds()
	{
		WorldBounds = FBox(Origin - FVector(0, 0, BoundsSize.Z / 2), Origin + FVector(BoundsSize.X, BoundsSize.Y, BoundsSize.Z / 2));
	}

	inline bool IsValid() const
	{
		return Cells.Num() && GridCellSize > 0;
	}

	inline bool IsValidIndex(const int32 CellIndex) const
	{
		return Cells.IsValidIndex(CellIndex);
	}

	inline bool IsValidCoord(int32 LocationX, int32 LocationY) const
	{
		return (LocationX >= 0) && (LocationX < (int32)GridSize.Width) && (LocationY >= 0) && (LocationY < (int32)GridSize.Height);
	}

	inline bool IsValidCoord(const FIntVector& CellCoords) const
	{
		return IsValidCoord(CellCoords.X, CellCoords.Y);
	}

	inline uint32 GetAllocatedSize() const
	{
		return IntCastChecked<uint32>(Cells.GetAllocatedSize());
	}

	/** Convert world location to (X,Y) coords on grid, result can be outside grid */
	inline FIntVector GetCellCoordsUnsafe(const FVector& WorldLocation) const
	{
		return FIntVector(
			IntCastChecked<int32>(FMath::TruncToInt((WorldLocation.X - Origin.X) / GridCellSize)),
			IntCastChecked<int32>(FMath::TruncToInt((WorldLocation.Y - Origin.Y) / GridCellSize)),
			0);
	}

	/** Convert world location to (X,Y) coords on grid, result is clamped to grid */
	FIntVector GetCellCoords(const FVector& WorldLocation) const
	{
		const FIntVector UnsafeCoords = GetCellCoordsUnsafe(WorldLocation);
		return FIntVector(FMath::Clamp(UnsafeCoords.X, 0, (int32)GridSize.Width - 1), FMath::Clamp(UnsafeCoords.Y, 0, (int32)GridSize.Height - 1), 0);
	}

	/** Convert cell index to coord X on grid, result can be invalid */
	inline int32 GetCellCoordX(int32 CellIndex) const
	{
		return CellIndex / GridSize.Height;
	}

	/** Convert cell index to coord Y on grid, result can be invalid */
	inline int32 GetCellCoordY(int32 CellIndex) const
	{
		return CellIndex % GridSize.Height;
	}

	/** Convert cell index to (X,Y) coords on grid */
	inline FIntVector GetCellCoords(int32 CellIndex) const
	{
		return FIntVector(GetCellCoordX(CellIndex), GetCellCoordY(CellIndex), 0);
	}

	/** Convert world location to cell index, result can be invalid */
	int32 GetCellIndexUnsafe(const FVector& WorldLocation) const
	{
		const FIntVector CellCoords = GetCellCoordsUnsafe(WorldLocation);
		return GetCellIndexUnsafe(CellCoords.X, CellCoords.Y);
	}

	/** Convert (X,Y) coords on grid to cell index, result can be invalid */
	inline int32 GetCellIndexUnsafe(const FIntVector& CellCoords) const
	{
		return GetCellIndexUnsafe(CellCoords.X, CellCoords.Y);
	}

	/** Convert (X,Y) coords on grid to cell index, result can be invalid */
	inline int32 GetCellIndexUnsafe(int32 LocationX, int32 LocationY) const
	{
		return (LocationX * GridSize.Height) + LocationY;
	}

	/** Convert (X,Y) coords on grid to cell index, returns -1 for position outside grid */
	inline int32 GetCellIndex(int32 LocationX, int32 LocationY) const
	{
		return IsValidCoord(LocationX, LocationY) ? GetCellIndexUnsafe(LocationX, LocationY) : INDEX_NONE;
	}

	/** Convert world location to cell index, returns -1 for position outside grid */
	int32 GetCellIndex(const FVector& WorldLocation) const
	{
		const FIntVector CellCoords = GetCellCoordsUnsafe(WorldLocation);
		return GetCellIndex(CellCoords.X, CellCoords.Y);
	}

	/** Return the bounding box of a cell. */
	inline FBox GetWorldCellBox(int32 CellIndex) const
	{
		return GetWorldCellBox(GetCellCoordX(CellIndex), GetCellCoordY(CellIndex));
	}

	/** Return the bounding box of a cell. */
	inline FBox GetWorldCellBox(int32 LocationX, int32 LocationY) const
	{
		return FBox(
			Origin + FVector(LocationX * GridCellSize, LocationY * GridCellSize, -BoundsSize.Z * 0.5f), 
			Origin + FVector((LocationX + 1) * GridCellSize, (LocationY + 1) * GridCellSize, BoundsSize.Z * 0.5f)
			);
	}

	/** Return the 2D bounding box of a cell. */
	inline FBox2D GetWorldCellBox2D(int32 CellIndex) const
	{
		return GetWorldCellBox2D(GetCellCoordX(CellIndex), GetCellCoordY(CellIndex));
	}

	/** Return the 2D bounding box of a cell. */
	inline FBox2D GetWorldCellBox2D(int32 LocationX, int32 LocationY) const
	{
		return FBox2D(
			FVector2D(Origin) + FVector2D(LocationX * GridCellSize, LocationY * GridCellSize), 
			FVector2D(Origin) + FVector2D((LocationX + 1) * GridCellSize, (LocationY + 1) * GridCellSize)
			);
	}

	/** Return the world bounding box of all cells included in the given rectangle. */
	inline FBox GetWorldCellRectangleBox(const FIntRect& CellRect) const
	{
		return FBox(
			Origin + FVector(CellRect.Min.X * GridCellSize, CellRect.Min.Y * GridCellSize, -BoundsSize.Z * 0.5f), 
			Origin + FVector((CellRect.Max.X+1) * GridCellSize, (CellRect.Max.Y+1) * GridCellSize, BoundsSize.Z * 0.5f)
			);
	}

	/** Compute a rectangle of cells overlapping the given WorldBox. */
	inline FIntRect GetCellRectangleFromBox(const FBox& WorldBox) const
	{
		if (!WorldBox.IsValid)
		{
			return FIntRect();
		}

		const FIntVector CellMin = GetCellCoordsUnsafe(WorldBox.Min);
		const FIntVector CellMax = GetCellCoordsUnsafe(WorldBox.Max);
		return FIntRect(CellMin.X, CellMin.Y, CellMax.X, CellMax.Y);
	}

	/** Return an IntRect that includes all the cells of the grid. Max is inclusive. */
	inline FIntRect GetGridRectangle() const
	{
		return FIntRect(0, 0, GridSize.Width-1, GridSize.Height-1);
	}

	inline FVector GetWorldCellCenter(int32 CellIndex) const
	{
		return GetWorldCellCenter(GetCellCoordX(CellIndex), GetCellCoordY(CellIndex));
	}

	inline FVector GetWorldCellCenter(int32 LocationX, int32 LocationY) const
	{
		return Origin + FVector((LocationX + 0.5f) * GridCellSize, (LocationY + 0.5f) * GridCellSize, 0);
	}

	const FCellType& GetCellAtWorldLocationUnsafe(const FVector& WorldLocation) const
	{
		const int32 CellIndex = GetCellIndexUnsafe(WorldLocation);
		return Cells[CellIndex];
	}

	const FCellType& GetCellAtWorldLocation(const FVector& WorldLocation) const
	{
		static FCellType InvalidCellInstance = FCellType(InvalidCellValue);
		const int32 CellIndex = GetCellIndex(WorldLocation);
		return (CellIndex == INDEX_NONE) ? InvalidCellInstance : Cells[CellIndex];
	}

	inline FCellType& operator[](int32 CellIndex) { return Cells[CellIndex]; }
	inline const FCellType& operator[](int32 CellIndex) const { return Cells[CellIndex]; }

	inline FCellType& GetCellAtIndexUnsafe(int32 CellIndex) { return Cells.GetData()[CellIndex]; }
	inline const FCellType& GetCellAtIndexUnsafe(int32 CellIndex) const { return Cells.GetData()[CellIndex]; }

	FCellType& GetCellAtCoordsUnsafe(int32 LocationX, int32 LocationY) { return GetCellAtIndexUnsafe(GetCellIndexUnsafe(LocationX, LocationY)); }
	const FCellType& GetCellAtCoordsUnsafe(int32 LocationX, int32 LocationY) const { return GetCellAtIndexUnsafe(GetCellIndexUnsafe(LocationX, LocationY)); }

	inline int32 GetCellsCount() const
	{
		return Cells.Num();
	}

	inline int32 Num() const
	{
		return Cells.Num();
	}

	void Serialize(FArchive& Ar)
	{	
		uint32 VersionNum = MAX_uint32;
		Ar << VersionNum;

		if (Ar.IsLoading())
		{
			if (VersionNum == MAX_uint32)
			{
				Ar << GridCellSize;
			}
			else
			{
				GridCellSize = VersionNum * 1.0f;
			}
		}
		else
		{
			Ar << GridCellSize;
		}

		Ar << Origin;
		Ar << BoundsSize;
		Ar << GridSize.Width << GridSize.Height;
	
		UpdateWorldBounds();

		if (VersionNum == MAX_uint32)
		{
			Ar << Cells;
		}
		else
		{
			uint32 DataBytesCount = GetAllocatedSize();
			Ar << DataBytesCount;

			if (DataBytesCount > 0)
			{
				if (Ar.IsLoading())
				{
					const uint64 TempCellCount = GridSize.Width * GridSize.Height;
					const typename TArray<FCellType>::SizeType CellCount = FMath::Min(TempCellCount, (uint64)TNumericLimits<typename TArray<FCellType>::SizeType>::Max());
					ensureMsgf(CellCount == TempCellCount, TEXT("Grid width and height are too big."));
					Cells.SetNum(CellCount);
					
					DataBytesCount = FMath::Clamp(DataBytesCount, 0, CellCount * sizeof(FCellType));
				}

				Ar.Serialize(Cells.GetData(), DataBytesCount);
			}
		}
	}

	void AllocateMemory()
	{
		Cells.SetNum(GridSize.Width * GridSize.Height);
	}

	void FreeMemory()
	{
		Cells.Empty();
	}

	void Zero()
	{
		Cells.Reset();
		Cells.AddZeroed(GridSize.Width * GridSize.Height);
	}

	void CleanUp()
	{
		Cells.Empty();
		GridCellSize = 0;
		Origin = FVector(FLT_MAX);
	}
};
